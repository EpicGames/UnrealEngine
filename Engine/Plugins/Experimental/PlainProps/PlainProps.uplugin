{
	"FileVersion" : 3,
	"FriendlyName" : "PlainProps",
	"Description" : "New Serialization Stack Prototype",
	"Category": "Experimental",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"EnabledByDefault" : false,
	"IsExperimentalVersion": true,
	"SupportedTargetPlatforms": [
		"Win64"
	],
	"Modules": [
		{
			"Name" : "PlainProps",
			"Type" : "Runtime",
			"PlatformAllowList": [ "Win64" ]
		}
	]
}