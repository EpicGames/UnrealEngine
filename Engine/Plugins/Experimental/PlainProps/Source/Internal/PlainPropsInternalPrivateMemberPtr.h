// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PreprocessorHelpers.h"
#include "Templates/Identity.h"

#define PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_BEGIN(Namespace) \
	namespace Namespace \
	{ \
		template <auto Storage, auto PtrToMember> \
		struct TPrivateAccess \
		{ \
			TPrivateAccess() \
			{ \
				*Storage = PtrToMember; \
			} \
			static TPrivateAccess Instance; \
		}; \
		template <auto Storage, auto PtrToMember> \
		TPrivateAccess<Storage, PtrToMember> TPrivateAccess<Storage, PtrToMember>::Instance;

#define PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_END() \
	}

#define PP_DEFINE_PRIVATE_MEMBER_PTR(Class, Member, Type) \
	TIdentity_T<UE_REMOVE_OPTIONAL_PARENS(Type)> UE_REMOVE_OPTIONAL_PARENS(Class)::* _##Member; \
	template struct TPrivateAccess<&_##Member, &UE_REMOVE_OPTIONAL_PARENS(Class)::Member>

