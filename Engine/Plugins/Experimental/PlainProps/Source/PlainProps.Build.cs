// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PlainProps : ModuleRules
{
	public PlainProps(ReadOnlyTargetRules Target) : base(Target)
	{
		bDisableStaticAnalysis = true;

		CppStandard = CppStandardVersion.Cpp20;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CorePreciseFP",
		});
	}
}
