<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">
  
  <!-- // Copyright Epic Games, Inc. All Rights Reserved. -->

  <!--
    Some watch window examples:
    - ((PlainProps::DbgVis::FIdVisualizer*)GDebuggingState->Ptrs[(strstr(GDebuggingState->GuidString, "d4b455b77bab4f1da94498ec086fb4ab") - GDebuggingState->GuidString)/32])
    - PlainProps::GReadSchemas    
    - _ppVisualizer()
    - _ppIndexer()
    - _ppNameType()
    - _ppResolveNestedScope(1)
    - _ppResolveParametricType(1)
    - _ppResolveEnum(1)
    - _ppResolveStruct(1)
    - _ppResolveBatch(1)
    - StaticArrayType
    - StaticArrayType,view(nn)
  -->

  <!-- Helper function to align a uintptr_t Val to some power-of-two Alignment. -->
  <Intrinsic Name="_AlignInt" Expression="(Val + Alignment - 1) &amp; ~(Alignment - 1)">
    <Parameter Name="Val" Type="uintptr_t"/>
    <Parameter Name="Alignment" Type="uintptr_t"/>
  </Intrinsic>


  <!-- Global name and id resolving intrinsics -->

  <!-- See FVisualizerDebuggingState for documentation.--> 
  <Intrinsic Name="_ppVisualizer"
    Expression="((PlainProps::DbgVis::FIdVisualizer*)GDebuggingState->Ptrs[(strstr(GDebuggingState->GuidString, &quot;d4b455b77bab4f1da94498ec086fb4ab&quot;) - GDebuggingState->GuidString)/32])">
  </Intrinsic>

  <Intrinsic Name="_ppIndexer" Expression="_ppVisualizer()->Indexer"/>
  <Intrinsic Name="_ppNameType" Expression="_ppVisualizer()->NameType"/>
  <Intrinsic Name="_ppUseFName" Expression="strncmp(_ppNameType(),&quot;FName&quot;,5) == 0"/>
  <Intrinsic Name="_ppUseFSensitiveName" Expression="strncmp(_ppNameType(),&quot;SensName&quot;,8) == 0"/>
  <Intrinsic Name="_ppUseFAnsiString" Expression="strncmp(_ppNameType(),&quot;AnsiStr&quot;,7) == 0"/>
  <Intrinsic Name="_ppUseFUtf8StringView" Expression="strncmp(_ppNameType(),&quot;Utf8View&quot;,8) == 0"/>

  <Intrinsic Name="_ppResolveNestedScope" Expression="*((PlainProps::FNestedScope*)(_ppIndexer()->NestedScopes.Scopes._GetElementPtr(Idx)))">
    <Parameter Name="Idx" Type="int64"/>
  </Intrinsic>

  <Intrinsic Name="_ppResolveParametricType" Expression="*((PlainProps::FParametricType*)(_ppIndexer()->ParametricTypes.Types._GetElementPtr(Idx)))">
    <Parameter Name="Idx" Type="int64"/>
  </Intrinsic>

  <Intrinsic Name="_ppResolveParameter" Expression="*((PlainProps::FType*)(_ppIndexer()->ParametricTypes.Parameters._GetElementPtr(Idx)))">
    <Parameter Name="Idx" Type="int64"/>
  </Intrinsic>

  <Intrinsic Name="_ppResolveStruct" Expression="*((PlainProps::FType*)(_ppIndexer()->Structs._GetElementPtr(Idx)))">
    <Parameter Name="Idx" Type="int64"/>
  </Intrinsic>

  <Intrinsic Name="_ppResolveEnum" Expression="*((PlainProps::FType*)(_ppIndexer()->Enums._GetElementPtr(Idx)))">
    <Parameter Name="Idx" Type="int64"/>
  </Intrinsic>

  <Intrinsic Name="_ppResolveBatch" Expression="((PlainProps::DbgVis::FSchemaBatch**)(GDebuggingState->Ptrs[(strstr(GDebuggingState->GuidString, &quot;0a05d5a9de4e492d989e7f936cc1c843&quot;) - GDebuggingState->GuidString)/32]))[Idx]">
    <Parameter Name="Idx" Type="uint16"/>
  </Intrinsic>

  <Intrinsic Name="_ppResolveStructSchema" Expression="_ppResolveBatch(BatchIdx)->_ResolveStruct(Idx)">
    <Parameter Name="BatchIdx" Type="uint16"/>
    <Parameter Name="Idx" Type="uint32"/>
  </Intrinsic>

  <Intrinsic Name="_ppResolveEnumSchema" Expression="_ppResolveBatch(BatchIdx)->_ResolveEnum(Idx)">
    <Parameter Name="BatchIdx" Type="uint16"/>
    <Parameter Name="Idx" Type="uint32"/>
  </Intrinsic>

  <!-- Global PlainProps intrinsics -->
  <Intrinsic Name="_ppUsesSuper" Expression="(int32)(Inheritance == PlainProps::DbgVis::ESuper::Used || Inheritance == PlainProps::DbgVis::ESuper::Reused)">
    <Parameter Name="Inheritance" Type="PlainProps::DbgVis::ESuper"/>
  </Intrinsic>

  <!-- PlainPropsTypes.h : Simple Types -->
  
  <Type Name="PlainProps::FLeafType">
    <AlternativeType Name="PlainProps::FUnpackedLeafType"/>
    
    <Intrinsic Name="_IsBool"   Expression="Type == ELeafType::Bool &amp;&amp; Width == ELeafWidth::B8"/>
    <Intrinsic Name="_IsU8"     Expression="Type == ELeafType::IntU &amp;&amp; Width == ELeafWidth::B8"/>
    <Intrinsic Name="_IsU16"    Expression="Type == ELeafType::IntU &amp;&amp; Width == ELeafWidth::B16"/>
    <Intrinsic Name="_IsU32"    Expression="Type == ELeafType::IntU &amp;&amp; Width == ELeafWidth::B32"/>
    <Intrinsic Name="_IsU64"    Expression="Type == ELeafType::IntU &amp;&amp; Width == ELeafWidth::B64"/>
    <Intrinsic Name="_IsS8"     Expression="Type == ELeafType::IntS &amp;&amp; Width == ELeafWidth::B8"/>
    <Intrinsic Name="_IsS16"    Expression="Type == ELeafType::IntS &amp;&amp; Width == ELeafWidth::B16"/>
    <Intrinsic Name="_IsS32"    Expression="Type == ELeafType::IntS &amp;&amp; Width == ELeafWidth::B32"/>
    <Intrinsic Name="_IsS64"    Expression="Type == ELeafType::IntS &amp;&amp; Width == ELeafWidth::B64"/>
    <Intrinsic Name="_IsFloat"  Expression="Type == ELeafType::Float &amp;&amp; Width == ELeafWidth::B32"/>
    <Intrinsic Name="_IsDouble" Expression="Type == ELeafType::Float &amp;&amp; Width == ELeafWidth::B64"/>
    <Intrinsic Name="_IsHex8"   Expression="Type == ELeafType::Hex &amp;&amp; Width == ELeafWidth::B8"/>
    <Intrinsic Name="_IsHex16"  Expression="Type == ELeafType::Hex &amp;&amp; Width == ELeafWidth::B16"/>
    <Intrinsic Name="_IsHex32"  Expression="Type == ELeafType::Hex &amp;&amp; Width == ELeafWidth::B32"/>
    <Intrinsic Name="_IsHex64"  Expression="Type == ELeafType::Hex &amp;&amp; Width == ELeafWidth::B64"/>
    <Intrinsic Name="_IsEnum8"  Expression="Type == ELeafType::Enum &amp;&amp; Width == ELeafWidth::B8"/>
    <Intrinsic Name="_IsEnum16" Expression="Type == ELeafType::Enum &amp;&amp; Width == ELeafWidth::B16"/>
    <Intrinsic Name="_IsEnum32" Expression="Type == ELeafType::Enum &amp;&amp; Width == ELeafWidth::B32"/>
    <Intrinsic Name="_IsEnum64" Expression="Type == ELeafType::Enum &amp;&amp; Width == ELeafWidth::B64"/>
    <Intrinsic Name="_IsUtf8"   Expression="Type == ELeafType::Unicode &amp;&amp; Width == ELeafWidth::B8"/>
    <Intrinsic Name="_IsUtf16"  Expression="Type == ELeafType::Unicode &amp;&amp; Width == ELeafWidth::B16"/>
    <Intrinsic Name="_IsUtf32"  Expression="Type == ELeafType::Unicode &amp;&amp; Width == ELeafWidth::B32"/>
    
    <DisplayString Condition="_ != EMemberKind::Leaf" Optional="1">Invalid</DisplayString>
    <DisplayString Condition="_IsBool()">bool</DisplayString>
    <DisplayString Condition="_IsU8()" >u8</DisplayString>
    <DisplayString Condition="_IsU16()">u16</DisplayString>
    <DisplayString Condition="_IsU32()">u32</DisplayString>
    <DisplayString Condition="_IsU64()">u64</DisplayString>
    <DisplayString Condition="_IsS8()" >s8</DisplayString>
    <DisplayString Condition="_IsS16()">s16</DisplayString>
    <DisplayString Condition="_IsS32()">s32</DisplayString>
    <DisplayString Condition="_IsS64()">s64</DisplayString>
    <DisplayString Condition="_IsFloat()">float</DisplayString>
    <DisplayString Condition="_IsDouble()">double</DisplayString>
    <DisplayString Condition="_IsHex8()">Hex8</DisplayString>
    <DisplayString Condition="_IsHex16()">Hex16</DisplayString>
    <DisplayString Condition="_IsHex32()">Hex32</DisplayString>
    <DisplayString Condition="_IsHex64()">Hex64</DisplayString>
    <DisplayString Condition="_IsEnum8()">Enum8</DisplayString>
    <DisplayString Condition="_IsEnum16()">Enum16</DisplayString>
    <DisplayString Condition="_IsEnum32()">Enum32</DisplayString>
    <DisplayString Condition="_IsEnum64()">Enum64</DisplayString>
    <DisplayString Condition="_IsUtf8()">Utf8</DisplayString>
    <DisplayString Condition="_IsUtf16()">Utf16</DisplayString>
    <DisplayString Condition="_IsUtf32()">Utf32</DisplayString>
    <DisplayString Condition="_ == EMemberKind::Leaf" Optional="1">Fallback{_,en}{Type,en}{Width,en}</DisplayString>
    <DisplayString>Invalid</DisplayString>
  </Type>
  
  <Type Name="PlainProps::FRangeType">
    <DisplayString Condition="_ == EMemberKind::Range">{_,en}{MaxSize,en}</DisplayString>
    <DisplayString>Invalid</DisplayString>
  </Type>
  
  <Type Name="PlainProps::FStructType">
    <DisplayString Condition="_ != EMemberKind::Struct">Invalid</DisplayString>
    <DisplayString Condition="IsDynamic == 0 &amp;&amp; IsSuper == 1">Super{_,en}</DisplayString>
    <DisplayString Condition="IsDynamic == 0">{_,en}</DisplayString>
    <DisplayString Condition="IsDynamic == 1 &amp;&amp; IsSuper == 1">DynamicSuper{_,en}</DisplayString>
    <DisplayString Condition="IsDynamic == 1">Dynamic{_,en}</DisplayString>
    <DisplayString>Invalid</DisplayString>
  </Type>

  <Type Name="PlainProps::FMemberType">
    <Intrinsic Name="_IsLeaf" Expression="Kind == EMemberKind::Leaf"/>
    <Intrinsic Name="_IsRange" Expression="Kind == EMemberKind::Range"/>
    <Intrinsic Name="_IsStruct" Expression="Kind == EMemberKind::Struct"/>
    <Intrinsic Name="_IsEnum" Expression="_IsLeaf() &amp;&amp; Leaf.Type == ELeafType::Enum"/>
    
    <DisplayString Condition="_IsLeaf()">{Leaf}</DisplayString>
    <DisplayString Condition="_IsRange()">{Range}</DisplayString>
    <DisplayString Condition="_IsStruct()">{Struct}</DisplayString>
    <DisplayString>Invalid</DisplayString>
    <Expand>
      <Item Name="Kind">Kind</Item>
      <Item Name="[Leaf]" Condition="_IsLeaf()">Leaf</Item>
      <Item Name="[Range]" Condition="_IsRange()">Range</Item>
      <Item Name="[Struct]" Condition="_IsStruct()">Struct</Item>
    </Expand>
  </Type>

  <Type Name="PlainProps::TOptionalId&lt;*&gt;">
    <Intrinsic Name="_IsSet" Expression="Idx != ~0u"/>
    <Intrinsic Name="_Ptr" Expression="($T1*)this"/> <!-- Exposing _Ptr() fixes "An unspecified error has occurred" in FMemberSchema -->
    <DisplayString Condition="Idx == ~0u">NoId</DisplayString>
    <DisplayString ExcludeView="nn;nr">{*($T1*)this}</DisplayString>
    <DisplayString IncludeView="nn">{*($T1*)this,view(nn)}</DisplayString>
    <DisplayString>{*($T1*)this,view(nr)}</DisplayString>
    <Expand>
      <ExpandedItem Condition="Idx != ~0u" ExcludeView="nn;nr">*($T1*)this</ExpandedItem>
      <ExpandedItem Condition="Idx != ~0u" IncludeView="nn">*($T1*)this,view(nn)</ExpandedItem>
      <ExpandedItem Condition="Idx != ~0u" IncludeView="nr">*($T1*)this,view(nr)</ExpandedItem>
    </Expand>
  </Type>


  <!-- PlainPropsUeCoreBindings.h -->
    
  <Type Name="PlainProps::UE::FSensitiveName">
    <DisplayString Condition="Number > 0" Optional="1">{(FDisplayNameEntryId&amp;)Value}_{Number-1}</DisplayString>
    <DisplayString Condition="Number > 0" Optional="1">{DisplayId}_{Number-1}</DisplayString>
    <DisplayString Optional="1">{(FDisplayNameEntryId&amp;)Value}</DisplayString>
    <DisplayString Optional="1">{DisplayId}</DisplayString>
    <DisplayString>UNEXPECTED: {this,na!}</DisplayString>
  </Type>
  

  <!--
    PlainPropsTypes.h : Name and id resolving types. These support three different views:
      - the default view with global index and name resolving.
      - a no-name "nn" view without glboal name resolving.
      - a no-resolve "nr" view without any global index or name resolving at all.
  -->

  <Type Name="PlainProps::TIdIndexer&lt;*&gt;">
    <Intrinsic Name="_NumNames" Expression="Names._Num()"/>
    <Intrinsic Name="_ResolveName" Expression="*(($T1*)(Names._GetElementPtr(Idx)))">
      <Parameter Name="Idx" Type="int64"/>
    </Intrinsic>
    <DisplayString>{"$T1"}, NumNames={Names._Num()}, Name0={_ResolveName(0)}</DisplayString>
  </Type>

  <Type Name="PlainProps::FNameId">
    <DisplayString Condition="_ppUseFName()" ExcludeView="nn;nr" Optional="1">
        {(*(TIdIndexer&lt;FName&gt;*)_ppIndexer())._ResolveName(Idx)}
    </DisplayString>
    <DisplayString Condition="_ppUseFSensitiveName()" ExcludeView="nn;nr" Optional="1">
        {(*(TIdIndexer&lt;PlainProps::UE::FSensitiveName&gt;*)_ppIndexer())._ResolveName(Idx)}
    </DisplayString>
    <DisplayString Condition="_ppUseFAnsiString()" ExcludeView="nn;nr" Optional="1">
        {(*(TIdIndexer&lt;FAnsiString&gt;*)_ppIndexer())._ResolveName(Idx),sb}
    </DisplayString>
    <DisplayString Condition="_ppUseFUtf8StringView()" ExcludeView="nn;nr" Optional="1">
        {(*(TIdIndexer&lt;TStringView&lt;enum FGenericPlatformTypes::UTF8CHAR&gt; &gt;*)_ppIndexer())._ResolveName(Idx),sb}
    </DisplayString>
    <DisplayString>Name{Idx}</DisplayString>
  </Type>

  <Type Name="PlainProps::FMemberId">
    <DisplayString Condition="Id.Idx == ~0u">Super</DisplayString>
    <DisplayString ExcludeView="nn;nr" Optional="1">{Id}</DisplayString>
    <DisplayString>Member{Id.Idx}</DisplayString>
  </Type>

  <Type Name="PlainProps::FTypenameId">
    <Intrinsic Name="_IsConcrete" Expression="NumParameters == 0"/>
    <Intrinsic Name="_IsParametric" Expression="NumParameters != 0"/>
    <Intrinsic Name="_AsParametric" Expression="*reinterpret_cast&lt;FParametricTypeId*&gt;(this)"/>
    
    <DisplayString Condition="_IsConcrete() &amp;&amp; _ppUseFName()" ExcludeView="nn;nr" Optional="1">
      {(*(TIdIndexer&lt;FName&gt;*)_ppIndexer())._ResolveName(Idx)}
    </DisplayString>
    <DisplayString Condition="_IsConcrete() &amp;&amp; _ppUseFSensitiveName()" ExcludeView="nn;nr" Optional="1">
        {(*(TIdIndexer&lt;PlainProps::UE::FSensitiveName&gt;*)_ppIndexer())._ResolveName(Idx)}
    </DisplayString>
    <DisplayString Condition="_IsConcrete() &amp;&amp; _ppUseFAnsiString()" ExcludeView="nn;nr" Optional="1">
      {(*(TIdIndexer&lt;FAnsiString&gt;*)_ppIndexer())._ResolveName(Idx),sb}
    </DisplayString>
    <DisplayString Condition="_IsConcrete() &amp;&amp; _ppUseFUtf8StringView()" ExcludeView="nn;nr" Optional="1">
      {(*(TIdIndexer&lt;TStringView&lt;enum FGenericPlatformTypes::UTF8CHAR&gt; &gt;*)_ppIndexer())._ResolveName(Idx),sb}
    </DisplayString>
    <DisplayString Condition="_IsConcrete()">ConcreteTypename{Idx}</DisplayString>

    <DisplayString Condition="_IsParametric()" ExcludeView="nn;nr" Optional="1">{_ppResolveParametricType(Idx)}</DisplayString>
    <DisplayString Condition="_IsParametric()" ExcludeView="nr" Optional="1">{_ppResolveParametricType(Idx),view(nn)}</DisplayString>
    <DisplayString Condition="_IsParametric()">{_AsParametric(),view(nr)}</DisplayString>
    
    <DisplayString>Invalid</DisplayString>

    <Expand>
      <Item Name="[AsParametric]" Condition="_IsParametric()" ExcludeView="nn;nr">_AsParametric()</Item>
      <Item Name="[AsParametric]" Condition="_IsParametric()" IncludeView="nn">_AsParametric(),view(nn)</Item>
      <Item Name="[AsParametric]" Condition="_IsParametric()" IncludeView="nr">_AsParametric(),view(nr)</Item>
    </Expand>
  </Type>

  <Type Name="PlainProps::FConcreteTypenameId">
    <DisplayString ExcludeView="nn;nr" Optional="1">{Id}</DisplayString>
    <DisplayString ExcludeView="nr" Optional="1">{Id,view(nn)}</DisplayString>
    <DisplayString>ConcreteTypename{Id.Idx}</DisplayString>
    <Expand>
      <Item Name="Id" ExcludeView="nn;nr">Id</Item>
      <Item Name="Id" IncludeView="nn;nr">Id,view(nn)</Item>
    </Expand>
  </Type>

  <Type Name="PlainProps::FParametricTypeId">
    <DisplayString ExcludeView="nn;nr" Optional="1">{_ppResolveParametricType(Idx)}</DisplayString>
    <DisplayString ExcludeView="nr" Optional="1">{_ppResolveParametricType(Idx),view(nn)}</DisplayString>
    <DisplayString Condition="NumParameters == 1">ParametricType{Idx}&lt;P0&gt;</DisplayString>
    <DisplayString Condition="NumParameters == 2">ParametricType{Idx}&lt;P0,P1&gt;</DisplayString>
    <DisplayString Condition="NumParameters &gt; 2">ParametricType{Idx}&lt;P0,...,P{NumParameters-1}&gt;</DisplayString>
    <DisplayString>Invalid</DisplayString>
    <Expand>
      <Item Name="[Parameters]" ExcludeView="nn;nr">_ppResolveParametricType(Idx)</Item>
      <Item Name="[Parameters]" IncludeView="nn">_ppResolveParametricType(Idx),view(nn)</Item>
    </Expand>
  </Type>

  <Type Name="PlainProps::FParametricType">
    <DisplayString Condition="Name._IsSet()" ExcludeView="nn;nr" Optional="1">{Name}&lt;{Parameters}&gt;</DisplayString>
    <DisplayString Condition="Name._IsSet()" ExcludeView="nr" Optional="1">{Name,view(nn)}&lt;{Parameters,view(nn)}&gt;</DisplayString>
    <DisplayString Condition="Name._IsSet()">{Name,view(nr)}&lt;{Parameters,view(nr)}&gt;</DisplayString>
    <DisplayString ExcludeView="nn;nr" Optional="1">[{Parameters}]</DisplayString>
    <DisplayString ExcludeView="nr" Optional="1">[{Parameters,view(nn)}]</DisplayString>
    <DisplayString>[{Parameters,view(nr)}]</DisplayString>
  </Type>

  <Type Name="PlainProps::FParametricTypeView">
    <DisplayString Condition="Name._IsSet()" ExcludeView="nn;nr" Optional="1">{Name}&lt;{Parameters,[NumParameters]na}&gt;</DisplayString>
    <DisplayString Condition="Name._IsSet()" ExcludeView="nr" Optional="1">{Name,view(nn)}&lt;{Parameters,[NumParameters]view(nn)}&gt;</DisplayString>
    <DisplayString Condition="Name._IsSet()">{Name,view(nr)}&lt;{Parameters,[NumParameters]view(nr)}&gt;</DisplayString>
    <DisplayString ExcludeView="nn;nr" Optional="1">[{Parameters,[NumParameters]}]na</DisplayString>
    <DisplayString ExcludeView="nr" Optional="1">[{Parameters,[NumParameters]view(nn)}]</DisplayString>
    <DisplayString>[{Parameters,[NumParameters]view(nr)}]</DisplayString>
    <Expand>
      <IndexListItems>
        <Size>NumParameters</Size>
        <ValueNode>Parameters[$i]</ValueNode>
      </IndexListItems>
    </Expand>
  </Type>

  <Type Name="PlainProps::FParameterIndexRange">
    <DisplayString Condition="NumParameters == 1" ExcludeView="nn;nr" Optional="1">{_ppResolveParameter(Idx).Name}</DisplayString>
    <DisplayString Condition="NumParameters == 2" ExcludeView="nn;nr" Optional="1">{_ppResolveParameter(Idx).Name},{_ppResolveParameter(Idx+1).Name}</DisplayString>
    <DisplayString Condition="NumParameters &gt; 2" ExcludeView="nn;nr" Optional="1">{_ppResolveParameter(Idx).Name},{_ppResolveParameter(Idx+1).Name},...,P{NumParameters-1}</DisplayString>
    
    <DisplayString Condition="NumParameters == 1" ExcludeView="nr" Optional="1">{_ppResolveParameter(Idx).Name,view(nn)}</DisplayString>
    <DisplayString Condition="NumParameters == 2" ExcludeView="nr" Optional="1">{_ppResolveParameter(Idx).Name,view(nn)},{_ppResolveParameter(Idx+1).Name,view(nn)}</DisplayString>
    <DisplayString Condition="NumParameters &gt; 2" ExcludeView="nr" Optional="1">{_ppResolveParameter(Idx).Name,view(nn)},{_ppResolveParameter(Idx+1).Name,view(nn)},...,P{Idx+NumParameters-1}</DisplayString>
    
    <DisplayString Condition="NumParameters == 1">Parameter{Idx}</DisplayString>
    <DisplayString Condition="NumParameters == 2">Parameter{Idx},Parameter{Idx+1}</DisplayString>
    <DisplayString Condition="NumParameters &gt; 2">Parameter{Idx},Parameter{Idx+1},...,Parameter{Idx+NumParameters-1}</DisplayString>
    
    <DisplayString>Invalid</DisplayString>
    <Expand>
      <IndexListItems ExcludeView="nn;nr">
        <Size>NumParameters</Size>
        <ValueNode>_ppResolveParameter(Idx + $i)</ValueNode>
      </IndexListItems>
      <IndexListItems IncludeView="nn">
        <Size>NumParameters</Size>
        <ValueNode>_ppResolveParameter(Idx + $i),view(nn)</ValueNode>
      </IndexListItems>
    </Expand>
  </Type>

  <Type Name="PlainProps::FScopeId">
    <Intrinsic Name="_IsUnscoped" Expression="Handle == Unscoped"/>
    <Intrinsic Name="_IsFlat" Expression="!(Handle &amp; NestedBit)"/>
    <Intrinsic Name="_IsNested" Expression="!_IsUnscoped() &amp;&amp; (Handle &amp; NestedBit)"/>
    <Intrinsic Name="_FlatIdx" Expression="Handle"/>
    <Intrinsic Name="_NestedIdx" Expression="Handle &amp; ~NestedBit"/>
    <Intrinsic Name="_AsFlat" Expression="*reinterpret_cast&lt;FFlatScopeId*&gt;(&amp;Handle)"/>

    <DisplayString Condition="_IsFlat()" ExcludeView="nn;nr">{_AsFlat().Name}</DisplayString>
    <DisplayString Condition="_IsFlat()" ExcludeView="nr">{_AsFlat().Name,view(nn)}</DisplayString>
    <DisplayString Condition="_IsFlat()">FlatScope{_FlatIdx()}</DisplayString>
    
    <DisplayString Condition="_IsNested()" ExcludeView="nn;nr" Optional="1">{_ppResolveNestedScope(_NestedIdx())}</DisplayString>
    <DisplayString Condition="_IsNested()" ExcludeView="nr" Optional="1">{_ppResolveNestedScope(_NestedIdx()),view(nn)}</DisplayString>
    <DisplayString Condition="_IsNested()">NestedScope{_NestedIdx()}</DisplayString>

    <DisplayString Condition="_IsUnscoped()">Unscoped</DisplayString>
    <DisplayString>Invalid</DisplayString>

    <Expand>
      <Item Name="[AsFlat]" Condition="_IsFlat()" ExcludeView="nn;nr">_AsFlat()</Item>
      <Item Name="[AsFlat]" Condition="_IsFlat()" IncludeView="nn">_AsFlat(),view(nn)</Item>
      <Item Name="[FlatIdx]" Condition="_IsFlat()">_FlatIdx()</Item>

      <Item Name="[NestedScope]" Condition="_IsNested()" ExcludeView="nn;nr">_ppResolveNestedScope(_NestedIdx())</Item>
      <Item Name="[NestedScope]" Condition="_IsNested()" IncludeView="nn">_ppResolveNestedScope(_NestedIdx()),view(nn)</Item>
      <Item Name="[NestedIdx]" Condition="_IsNested()">_NestedIdx()</Item>
    </Expand>
  </Type>

  <Type Name="PlainProps::FFlatScopeId">
    <DisplayString ExcludeView="nn;nr" Optional="1">{Name}</DisplayString>
    <DisplayString ExcludeView="nr" Optional="1">{Name,view(nn)}</DisplayString>
    <DisplayString>FlatScope{Name.Idx}</DisplayString>
  </Type>

  <Type Name="PlainProps::FNestedScopeId">
    <DisplayString ExcludeView="nn;nr" Optional="1">{_ppResolveNestedScope(Idx)}</DisplayString>
    <DisplayString ExcludeView="nr" Optional="1">{_ppResolveNestedScope(Idx),view(nn)}</DisplayString>
    <DisplayString>NestedScope{Idx}</DisplayString>
    <Expand HideRawView="0">
      <Item Name="[NestedScope]" ExcludeView="nn;nr">_ppResolveNestedScope(Idx)</Item>
      <Item Name="[NestedScope]" IncludeView="nn">_ppResolveNestedScope(Idx),view(nn)</Item>
    </Expand>
  </Type>

  <Type Name="PlainProps::FNestedScope">
    <!-- Explicitly unroll first level of outer scope information for perf reasons. -->
    <DisplayString Condition="Outer._IsNested()" ExcludeView="nn;nr" Optional="1">{_ppResolveNestedScope(Outer._NestedIdx()).Inner}.{Inner}</DisplayString>
    <DisplayString Condition="Outer._IsNested()" ExcludeView="nr" Optional="1">{_ppResolveNestedScope(Outer._NestedIdx()).Inner,view(nn)}.{Inner,view(nn)}</DisplayString>
    <DisplayString Condition="Outer._IsNested()">NestedScope{Outer._NestedIdx()}.{Inner,view(nr)}</DisplayString>
    <DisplayString ExcludeView="nn;nr" Optional="1">{Outer}.{Inner}</DisplayString>
    <DisplayString ExcludeView="nr" Optional="1">{Outer,view(nn)}.{Inner,view(nn)}</DisplayString>
    <DisplayString>{Outer,view(nr)}.{Inner,view(nr)}</DisplayString>
  </Type>
  
  <Type Name="PlainProps::FType" >
    <DisplayString Condition="!Scope._IsUnscoped()" ExcludeView="nn;nr">{Scope}.{Name}</DisplayString>
    <DisplayString Condition="!Scope._IsUnscoped()" ExcludeView="nr">{Scope,view(nn)}.{Name,view(nn)}</DisplayString>
    <DisplayString Condition="!Scope._IsUnscoped()">{Scope,view(nr)}.{Name,view(nr)}</DisplayString>
    <DisplayString ExcludeView="nn;nr">{Name}</DisplayString>
    <DisplayString ExcludeView="nr">{Name,view(nn)}</DisplayString>
    <DisplayString>{Name,view(nr)}</DisplayString>
  </Type>

  <Type Name="PlainProps::FInnerId">
    <Intrinsic Name="_AsStructId" Expression="*(FStructId*)this"/>
    <Intrinsic Name="_AsEnumId" Expression="*(FEnumId*)this"/>
    <DisplayString IncludeView="struct">{_AsStructId()}</DisplayString>
    <DisplayString IncludeView="enum">{_AsEnumId()}</DisplayString>
    <DisplayString>Inner{Idx}</DisplayString>
    <Expand>
      <Item Name="[AsStruct]" ExcludeView="nn;nr;enum">_AsStructId()</Item>
      <Item Name="[AsStruct]" IncludeView="nn">_AsStructId(),view(nn)</Item>
      <Item Name="[AsStruct]" IncludeView="nr">_AsStructId(),view(nr)</Item>
      <Item Name="[AsEnum]" ExcludeView="nn;nr;struct">_AsEnumId()</Item>
      <Item Name="[AsEnum]" IncludeView="nn">_AsEnumId(),view(nn)</Item>
      <Item Name="[AsEnum]" IncludeView="nr">_AsEnumId(),view(nr)</Item>
    </Expand>
  </Type>

  <Type Name="PlainProps::FEnumId">
    <DisplayString ExcludeView="nn;nr" Optional="1">{_ppResolveEnum(Idx).Name}</DisplayString>
    <DisplayString ExcludeView="nr" Optional="1">{_ppResolveEnum(Idx).Name,view(nn)}</DisplayString>
    <DisplayString>Enum{Idx}</DisplayString>
    <Expand>
      <Item Name="[Type]" ExcludeView="nn;nr">_ppResolveEnum(Idx)</Item>
      <Item Name="[Type]" IncludeView="nn">_ppResolveEnum(Idx),view(nn)</Item>
    </Expand>
  </Type>

  <Type Name="PlainProps::FStructId">
    <DisplayString ExcludeView="nn;nr" Optional="1">{_ppResolveStruct(Idx).Name}</DisplayString>
    <DisplayString ExcludeView="nr" Optional="1">{_ppResolveStruct(Idx).Name,view(nn)}</DisplayString>
    <DisplayString>Struct{Idx}</DisplayString>
    <Expand>
      <Item Name="[Type]" ExcludeView="nn;nr">_ppResolveStruct(Idx)</Item>
      <Item Name="[Type]" IncludeView="nn">_ppResolveStruct(Idx),view(nn)</Item>
    </Expand>
  </Type>

  <Type Name="PlainProps::FSchemaId">
    <Intrinsic Name="_AsStructId" Expression="*(FStructSchemaId*)this"/>
    <Intrinsic Name="_AsEnumId" Expression="*(FEnumSchemaId*)this"/>
    <DisplayString IncludeView="struct">{_AsStructId()}</DisplayString>
    <DisplayString IncludeView="enum">{_AsEnumId()}</DisplayString>
    <DisplayString>Schema{Idx}</DisplayString>
  </Type>

  <Type Name="PlainProps::FEnumSchemaId">
    <!-- <DisplayString Optional="1">EnumSchema{Idx}:{_ppIndexer()->PrintEnumSchema(*this),sub}</DisplayString> -->
    <DisplayString>EnumSchema{Idx}</DisplayString>
  </Type>

  <Type Name="PlainProps::FStructSchemaId">
    <!-- <DisplayString Optional="1">StructSchema{Idx}:{_ppIndexer()->PrintStructSchema(*this),sub}</DisplayString> -->
    <DisplayString>StructSchema{Idx}</DisplayString>
  </Type>


  <!-- PlainPropsBind.h -->

  <Type Name="PlainProps::FUnpackedLeafBindType">
    <DisplayString Condition="Type == ELeafBindType::Bool &amp;&amp;    Width == ELeafWidth::B8"  >bool</DisplayString>
    <DisplayString Condition="Type == ELeafBindType::IntS &amp;&amp;    Width == ELeafWidth::B8"  >i8</DisplayString>
    <DisplayString Condition="Type == ELeafBindType::IntS &amp;&amp;    Width == ELeafWidth::B16" >i16</DisplayString>
    <DisplayString Condition="Type == ELeafBindType::IntS &amp;&amp;    Width == ELeafWidth::B32" >i32</DisplayString>
    <DisplayString Condition="Type == ELeafBindType::IntS &amp;&amp;    Width == ELeafWidth::B64" >i64</DisplayString>
    <DisplayString Condition="Type == ELeafBindType::IntU &amp;&amp;    Width == ELeafWidth::B8"  >u8</DisplayString>
    <DisplayString Condition="Type == ELeafBindType::IntU &amp;&amp;    Width == ELeafWidth::B16" >u16</DisplayString>
    <DisplayString Condition="Type == ELeafBindType::IntU &amp;&amp;    Width == ELeafWidth::B32" >u32</DisplayString>
    <DisplayString Condition="Type == ELeafBindType::IntU &amp;&amp;    Width == ELeafWidth::B64" >u64</DisplayString>
    <DisplayString Condition="Type == ELeafBindType::Float &amp;&amp;   Width == ELeafWidth::B32" >float</DisplayString>
    <DisplayString Condition="Type == ELeafBindType::Float &amp;&amp;   Width == ELeafWidth::B64" >double</DisplayString>
    <DisplayString Condition="Type == ELeafBindType::Hex &amp;&amp;     Width == ELeafWidth::B8"  >Hex8</DisplayString>
    <DisplayString Condition="Type == ELeafBindType::Hex &amp;&amp;     Width == ELeafWidth::B16" >Hex16</DisplayString>
    <DisplayString Condition="Type == ELeafBindType::Hex &amp;&amp;     Width == ELeafWidth::B32" >Hex32</DisplayString>
    <DisplayString Condition="Type == ELeafBindType::Hex &amp;&amp;     Width == ELeafWidth::B64" >Hex64</DisplayString>
    <DisplayString Condition="Type == ELeafBindType::Enum &amp;&amp;    Width == ELeafWidth::B8"  >Enum8</DisplayString>
    <DisplayString Condition="Type == ELeafBindType::Enum &amp;&amp;    Width == ELeafWidth::B16" >Enum16</DisplayString>
    <DisplayString Condition="Type == ELeafBindType::Enum &amp;&amp;    Width == ELeafWidth::B32" >Enum32</DisplayString>
    <DisplayString Condition="Type == ELeafBindType::Enum &amp;&amp;    Width == ELeafWidth::B64" >Enum64</DisplayString>
    <DisplayString Condition="Type == ELeafBindType::Unicode &amp;&amp; Width == ELeafWidth::B8"  >Utf8</DisplayString>
    <DisplayString Condition="Type == ELeafBindType::Unicode &amp;&amp; Width == ELeafWidth::B16" >Utf16</DisplayString>
    <DisplayString Condition="Type == ELeafBindType::Unicode &amp;&amp; Width == ELeafWidth::B32" >Utf32</DisplayString>
    <DisplayString Condition="Type == ELeafBindType::BitfieldBool">Bitfield{BitfieldIdx,nvo}</DisplayString>
    <DisplayString>Fallback{Type,en}</DisplayString>
  </Type>
  
  <Type Name="PlainProps::FLeafBindType">
    <DisplayString Condition="Bind._ != EMemberKind::Leaf" Optional="1">Invalid</DisplayString>
    <DisplayString Condition="Bind.Type == ELeafBindType::Bool &amp;&amp;    Basic.Width == ELeafWidth::B8"  >bool</DisplayString>
    <DisplayString Condition="Bind.Type == ELeafBindType::IntS &amp;&amp;    Basic.Width == ELeafWidth::B8"  >i8</DisplayString>
    <DisplayString Condition="Bind.Type == ELeafBindType::IntS &amp;&amp;    Basic.Width == ELeafWidth::B16" >i16</DisplayString>
    <DisplayString Condition="Bind.Type == ELeafBindType::IntS &amp;&amp;    Basic.Width == ELeafWidth::B32" >i32</DisplayString>
    <DisplayString Condition="Bind.Type == ELeafBindType::IntS &amp;&amp;    Basic.Width == ELeafWidth::B64" >i64</DisplayString>
    <DisplayString Condition="Bind.Type == ELeafBindType::IntU &amp;&amp;    Basic.Width == ELeafWidth::B8"  >u8</DisplayString>
    <DisplayString Condition="Bind.Type == ELeafBindType::IntU &amp;&amp;    Basic.Width == ELeafWidth::B16" >u16</DisplayString>
    <DisplayString Condition="Bind.Type == ELeafBindType::IntU &amp;&amp;    Basic.Width == ELeafWidth::B32" >u32</DisplayString>
    <DisplayString Condition="Bind.Type == ELeafBindType::IntU &amp;&amp;    Basic.Width == ELeafWidth::B64" >u64</DisplayString>
    <DisplayString Condition="Bind.Type == ELeafBindType::Float &amp;&amp;   Basic.Width == ELeafWidth::B32" >float</DisplayString>
    <DisplayString Condition="Bind.Type == ELeafBindType::Float &amp;&amp;   Basic.Width == ELeafWidth::B64" >double</DisplayString>
    <DisplayString Condition="Bind.Type == ELeafBindType::Hex &amp;&amp;     Basic.Width == ELeafWidth::B8"  >Hex8</DisplayString>
    <DisplayString Condition="Bind.Type == ELeafBindType::Hex &amp;&amp;     Basic.Width == ELeafWidth::B16" >Hex16</DisplayString>
    <DisplayString Condition="Bind.Type == ELeafBindType::Hex &amp;&amp;     Basic.Width == ELeafWidth::B32" >Hex32</DisplayString>
    <DisplayString Condition="Bind.Type == ELeafBindType::Hex &amp;&amp;     Basic.Width == ELeafWidth::B64" >Hex64</DisplayString>
    <DisplayString Condition="Bind.Type == ELeafBindType::Enum &amp;&amp;    Basic.Width == ELeafWidth::B8"  >Enum8</DisplayString>
    <DisplayString Condition="Bind.Type == ELeafBindType::Enum &amp;&amp;    Basic.Width == ELeafWidth::B16" >Enum16</DisplayString>
    <DisplayString Condition="Bind.Type == ELeafBindType::Enum &amp;&amp;    Basic.Width == ELeafWidth::B32" >Enum32</DisplayString>
    <DisplayString Condition="Bind.Type == ELeafBindType::Enum &amp;&amp;    Basic.Width == ELeafWidth::B64" >Enum64</DisplayString>
    <DisplayString Condition="Bind.Type == ELeafBindType::Unicode &amp;&amp; Basic.Width == ELeafWidth::B8"  >Utf8</DisplayString>
    <DisplayString Condition="Bind.Type == ELeafBindType::Unicode &amp;&amp; Basic.Width == ELeafWidth::B16" >Utf16</DisplayString>
    <DisplayString Condition="Bind.Type == ELeafBindType::Unicode &amp;&amp; Basic.Width == ELeafWidth::B32" >Utf32</DisplayString>
    <DisplayString Condition="Bind.Type == ELeafBindType::BitfieldBool">Bitfield{Bitfield.Idx,nvo}</DisplayString>
    <DisplayString Condition="Bind._ == EMemberKind::Leaf" Optional="1">Fallback{Bind._,en}{Bind.Type,en}</DisplayString>
    <DisplayString>Fallback{Bind.Type,en}</DisplayString>
  </Type>
  
  <Type Name="PlainProps::FMemberBindType">
    <Intrinsic Name="_IsLeaf" Expression="Kind == EMemberKind::Leaf"/>
    <Intrinsic Name="_IsRange" Expression="Kind == EMemberKind::Range"/>
    <Intrinsic Name="_IsStruct" Expression="Kind == EMemberKind::Struct"/>
    <Intrinsic Name="_IsEnum" Expression="_IsLeaf() &amp;&amp; Leaf.Bind.Type == ELeafBindType::Enum"/>
    <DisplayString Condition="_IsLeaf()">{Leaf}</DisplayString>
    <DisplayString Condition="_IsRange()">{Range}</DisplayString>
    <DisplayString Condition="_IsStruct()">{Struct}</DisplayString>
    <DisplayString>Invalid</DisplayString>
    <Expand>
      <Item Name="Kind">Kind</Item>
      <Item Name="[Leaf]" Condition="_IsLeaf()">Leaf</Item>
      <Item Name="[Range]" Condition="_IsRange()">Range</Item>
      <Item Name="[Struct]" Condition="_IsStruct()">Struct</Item>
    </Expand>
  </Type>

  <Type Name="PlainProps::FMemberBinding">
    <Intrinsic Name="_InnerId" Expression="InnermostId._Ptr()"/>
    <Intrinsic Name="_NumInnerRanges" Expression="RangeBindings.ArrayNum"/>
    <Intrinsic Name="_GetSizeType" Expression="RangeBindings[Idx]._GetSizeType()">
      <Parameter Name="Idx" Type="int32"/>
    </Intrinsic>

    <DisplayString Condition="_NumInnerRanges() == 1 &amp;&amp; InnermostType._IsStruct()" ExcludeView="nn">{_InnerId(),view(struct)}({InnermostType})[{_GetSizeType(0),en}], Offset={Offset}</DisplayString>
    <DisplayString Condition="_NumInnerRanges() == 1 &amp;&amp; InnermostType._IsEnum()" ExcludeView="nn">{_InnerId(),view(enum)}({InnermostType})[{_GetSizeType(0),en}], Offset={Offset}</DisplayString>
    <DisplayString Condition="_NumInnerRanges() == 1">{InnermostType}[{_GetSizeType(0),en}], Offset={Offset}</DisplayString>
    <DisplayString Condition="_NumInnerRanges() == 2 &amp;&amp; InnermostType._IsStruct()" ExcludeView="nn">{_InnerId(),view(struct)}({InnermostType})[{_GetSizeType(0),en}][{_GetSizeType(1),en}], Offset={Offset}</DisplayString>
    <DisplayString Condition="_NumInnerRanges() == 2 &amp;&amp; InnermostType._IsEnum()" ExcludeView="nn">{_InnerId(),view(enum)}({InnermostType})[{_GetSizeType(0),en}][{_GetSizeType(1),en}], Offset={Offset}</DisplayString>
    <DisplayString Condition="_NumInnerRanges() == 2">{InnermostType}[{_GetSizeType(0),en}][{_GetSizeType(1),en}], Offset={Offset}</DisplayString>
    <DisplayString Condition="_NumInnerRanges() &gt; 2 &amp;&amp; InnermostType._IsStruct()" ExcludeView="nn">{_InnerId(),view(struct)}({InnermostType})[{_GetSizeType(0),en}][{_GetSizeType(1),en}][...], Offset={Offset}</DisplayString>
    <DisplayString Condition="_NumInnerRanges() &gt; 2 &amp;&amp; InnermostType._IsEnum()" ExcludeView="nn">{_InnerId(),view(enum)}({InnermostType})[{_GetSizeType(0),en}][{_GetSizeType(1),en}][...], Offset={Offset}</DisplayString>
    <DisplayString Condition="_NumInnerRanges() &gt; 2">{InnermostType}[{_GetSizeType(0),en}][{_GetSizeType(1),en}][...], Offset={Offset}</DisplayString>

    <DisplayString Condition="InnermostType._IsStruct()" ExcludeView="nn">{_InnerId(),view(struct)}, Offset={Offset}</DisplayString>
    <DisplayString Condition="InnermostType._IsStruct()">{_InnerId(),view(struct)}, Offset={Offset}</DisplayString>
    
    <DisplayString Condition="InnermostType._IsEnum()" ExcludeView="nn">{_InnerId(),view(enum)}, Offset={Offset}</DisplayString>
    <DisplayString Condition="InnermostType._IsEnum()">{_InnerId(),view(enum)}, Offset={Offset}</DisplayString>
    
    <DisplayString>{InnermostType}, Offset={Offset}</DisplayString>
    <Expand>
      <Item Name="Offset">Offset</Item>
      <Item Name="InnermostType">InnermostType</Item>
      <Item Name="InnermostId" Condition="InnermostId._IsSet()">InnermostId</Item>
      <Item Name="[StructSchema]" Condition="InnermostType._IsStruct()">*_InnerId(),view(struct)</Item>
      <Item Name="[EnumSchema]" Condition="InnermostType._IsEnum()">*_InnerId(),view(enum)</Item>
      <Item Name="RangeBindings" Condition="_NumInnerRanges() > 0">RangeBindings</Item>
    </Expand>
  </Type>
  
  <Type Name="PlainProps::FRangeBinding">
    <Intrinsic Name="_IsLeafBinding" Expression="LeafMask &amp; Handle"/>
    <Intrinsic Name="_IsItemBinding" Expression="!(LeafMask &amp; Handle)"/>
    <Intrinsic Name="_Ptr" Expression="(Handle &amp; BindMask)"/>
    <Intrinsic Name="_AsLeafBinding" Expression="(ILeafRangeBinding*)_Ptr()"/>
    <Intrinsic Name="_AsItemBinding" Expression="(IItemRangeBinding*)_Ptr()"/>
    <Intrinsic Name="_GetSizeType" Expression="static_cast&lt;ESizeType&gt;(Handle &amp; SizeMask)"/>
    
    <DisplayString Condition="_IsLeafBinding()" ExcludeView="nn">{(*_AsLeafBinding()).BindName}[{_GetSizeType(),en}], BindPtr={_Ptr(),x}</DisplayString>
    <DisplayString Condition="_IsLeafBinding()">{(*_AsLeafBinding()).BindName,view(nn)}[{_GetSizeType(),en}], BindPtr={_Ptr(),x}</DisplayString>
    
    <DisplayString Condition="_IsItemBinding()" ExcludeView="nn">{(*_AsItemBinding()).BindName}[{_GetSizeType(),en}], BindPtr={_Ptr(),x}</DisplayString>
    <DisplayString Condition="_IsItemBinding()">{(*_AsItemBinding()).BindName,view(nn)}[{_GetSizeType(),en}], BindPtr={_Ptr(),x}</DisplayString>

    <DisplayString>Invalid</DisplayString>
    
    <Expand>
      <Item Name="[LeafBinding]" Condition="_IsLeafBinding()" ExcludeView="nn">*_AsLeafBinding()</Item>
      <Item Name="[LeafBinding]" Condition="_IsLeafBinding()" IncludeView="nn">*_AsLeafBinding(),view(nn)</Item>
      
      <Item Name="[ItemBinding]" Condition="_IsItemBinding()" ExcludeView="nn">*_AsItemBinding()</Item>
      <Item Name="[ItemBinding]" Condition="_IsItemBinding()" IncludeView="nn">*_AsItemBinding(),view(nn)</Item>

      <Item Name="[BindPtr]">_Ptr(),x</Item>
      <Item Name="[SizeType]">_GetSizeType(),en</Item>
    </Expand>
  </Type>

  <Type Name="PlainProps::IItemRangeBinding" Inheritable="1">
    <DisplayString ExcludeView="nn">{BindName}, {this,na!}</DisplayString>
    <DisplayString>{BindName,view(nn)}, {this,na!}</DisplayString>
  </Type>

  <Type Name="PlainProps::ILeafRangeBinding" Inheritable="1">
    <DisplayString ExcludeView="nn">{BindName}, {this,na!}</DisplayString>
    <DisplayString>{BindName,view(nn)}, {this,na!}</DisplayString>
  </Type>
  
  <Type Name="PlainProps::FSchemaBinding">
    <Intrinsic Name="_InnerRangeTypes" Expression="Members + NumMembers"/>
    <Intrinsic Name="_Offsets" Expression="(uint32*)_AlignInt((uintptr_t)(_InnerRangeTypes() + NumInnerRanges), sizeof(uint32))"/>
    <Intrinsic Name="_InnerIds" Expression="(FInnerId*)_AlignInt((uintptr_t)(_Offsets() + NumMembers), sizeof(FInnerId))"/>
    <Intrinsic Name="_RangeBindings" Expression="(FRangeBinding*)_AlignInt((uintptr_t)(_InnerIds() + NumInnerIds), sizeof(FRangeBinding))"/>
    <Intrinsic Name="_HasSuper" Expression="NumInnerIds &gt; 0 &amp;&amp; Members[0]._IsStruct() &amp;&amp; Members[0].Struct.IsSuper"/>
    <Intrinsic Name="_Super" Expression="_InnerIds()[0]._AsStructId()"/>

    <DisplayString Condition="_HasSuper()" ExcludeView="nn">{DeclId}, Super={_Super()}, NumMembers={NumMembers}</DisplayString>
    <DisplayString Condition="_HasSuper()">{DeclId,view(nn)}, Super={_Super(),view(nn)}, NumMembers={NumMembers}</DisplayString>
    <DisplayString ExcludeView="nn">{DeclId}, NumMembers={NumMembers}</DisplayString>
    <DisplayString>{DeclId,view(nn)}, NumMembers={NumMembers}</DisplayString>
    
    <Expand>
      <Item Name="DeclId">DeclId</Item>
      <Item Name="[Super]" Condition="_HasSuper()">_Super()</Item>
      <Synthetic Name="[Members]">
        <DisplayString>{(void*)Members}, Num={NumMembers}</DisplayString>
        <Expand>
          <IndexListItems>
            <Size>NumMembers</Size>
            <ValueNode>Members[$i]</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>
      <Synthetic Name="[Offsets]">
        <DisplayString>{(void*)_Offsets()}, Num={NumMembers}</DisplayString>
        <Expand>
          <IndexListItems>
            <Size>NumMembers</Size>
            <ValueNode>_Offsets()[$i]</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>
      <Synthetic Name="[InnerRangeTypes]" Condition="NumInnerRanges &gt; 0">
        <DisplayString>{(void*)_InnerRangeTypes()}, Num={NumInnerRanges}</DisplayString>
        <Expand>
          <IndexListItems>
            <Size>NumInnerRanges</Size>
            <ValueNode>_InnerRangeTypes()[$i]</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>
      <Synthetic Name="[InnerIds]" Condition="NumInnerIds &gt; 0">
        <DisplayString>{(void*)_InnerIds()}, Num={NumInnerIds}</DisplayString>
        <Expand>
          <IndexListItems>
            <Size>NumInnerIds</Size>
            <ValueNode>_InnerIds()[$i]</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>
      <Synthetic Name="[RangeBindings]" Condition="NumInnerRanges &gt; 0">
        <DisplayString>{(void*)_RangeBindings()}, Num={NumInnerRanges}</DisplayString>
        <Expand>
          <IndexListItems>
            <Size>NumInnerRanges</Size>
            <ValueNode>_RangeBindings()[$i]</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>
    </Expand>
  </Type>


  <!-- PlainPropsBuild.h -->

  <Type Name="PlainProps::FMemberSchema">
    <Intrinsic Name="_InnerId" Expression="InnerSchema._Ptr()"/>
    <Intrinsic Name="_InnerRangeTypes" Expression="NestedRangeTypes ? NestedRangeTypes : &amp;InnerRangeType"/>
    <Intrinsic Name="_InnermostType" Expression="NumInnerRanges ? (_InnerRangeTypes() + NumInnerRanges - 1) : &amp;Type"/>

    <DisplayString Condition="NumInnerRanges == 1 &amp;&amp; InnerRangeType._IsStruct()">{_InnerId(),view(struct)}({InnerRangeType})[{Type.Range.MaxSize,en}]</DisplayString>
    <DisplayString Condition="NumInnerRanges == 1 &amp;&amp; InnerRangeType._IsEnum()">{_InnerId(),view(enum)}({InnerRangeType})[{Type.Range.MaxSize,en}]</DisplayString>
    <DisplayString Condition="NumInnerRanges == 1">{InnerRangeType}[{Type.Range.MaxSize,en}]</DisplayString>
    <DisplayString Condition="NumInnerRanges == 2 &amp;&amp; _InnermostType()->_IsStruct()">{_InnerId(),view(struct)}({_InnermostType(),na})[{Type.Range.MaxSize,en}][{NestedRangeTypes[0].Range.MaxSize,en}]</DisplayString>
    <DisplayString Condition="NumInnerRanges == 2 &amp;&amp; _InnermostType()->_IsEnum()">{_InnerId(),view(enum)}({_InnermostType(),na})[{Type.Range.MaxSize,en}][{NestedRangeTypes[0].Range.MaxSize,en}]</DisplayString>
    <DisplayString Condition="NumInnerRanges == 2">{_InnermostType(),na}[{Type.Range.MaxSize,en}][{NestedRangeTypes[0].Range.MaxSize,en}]</DisplayString>
    <DisplayString Condition="NumInnerRanges > 2 &amp;&amp; _InnermostType()->_IsStruct()">{_InnerId(),view(struct)}({_InnermostType(),na})[{Type.Range.MaxSize,en}][{NestedRangeTypes[0].Range.MaxSize,en}][...]</DisplayString>
    <DisplayString Condition="NumInnerRanges > 2 &amp;&amp; _InnermostType()->_IsEnum()">{_InnerId(),view(enum)}({_InnermostType(),na})[{Type.Range.MaxSize,en}][{NestedRangeTypes[0].Range.MaxSize,en}][...]</DisplayString>
    <DisplayString Condition="NumInnerRanges > 2">{_InnermostType(),na}[{Type.Range.MaxSize,en}][{NestedRangeTypes[0].Range.MaxSize,en}][...]</DisplayString>
    <DisplayString Condition="Type._IsStruct()">{_InnerId(),view(struct)}({Type})</DisplayString>
    <DisplayString Condition="Type._IsEnum()">{_InnerId(),view(enum)}({Type})</DisplayString>
    <DisplayString>{Type}</DisplayString>
    <Expand>
      <Item Name="Type">Type</Item>
      <Item Name="InnerRangeType" Condition="NumInnerRanges == 1">InnerRangeType</Item>
      <Item Name="InnerSchema" Condition="InnerSchema._IsSet()">InnerSchema</Item>
      <Item Name="[InnermostType]" Condition="NumInnerRanges &gt; 1">_InnermostType(),na</Item>
      <Item Name="[StructSchema]" Condition="_InnermostType()->_IsStruct() &amp;&amp; _InnermostType()->Struct.IsDynamic">"__dynamic__",sb</Item>
      <Item Name="[StructSchema]" Condition="_InnermostType()->_IsStruct() &amp;&amp; !_InnermostType()->Struct.IsDynamic">*_InnerId(),view(struct)</Item>
      <Item Name="[EnumSchema]" Condition="_InnermostType()->_IsEnum()">*_InnerId(),view(enum)</Item>
      <Synthetic Name="[InnerRangeTypes]" Condition="NumInnerRanges &gt; 1">
        <DisplayString>{(void*)_InnerRangeTypes()}, Num={NumInnerRanges}</DisplayString>
        <Expand>
          <IndexListItems>
            <Size>NumInnerRanges</Size>
            <ValueNode>_InnerRangeTypes()[$i]</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>
    </Expand>
  </Type>
  
  <!-- TODO: Add FTypedValue as AlternativeType -->
  <Type Name="PlainProps::FBuiltMember">
    <Intrinsic Name="_IsLeaf" Expression="Schema.Type._IsLeaf()"/>
    <Intrinsic Name="_IsStruct" Expression="Schema.Type._IsStruct()"/>
    <Intrinsic Name="_IsEmptyStruct" Expression="_IsStruct() &amp;&amp; !Value.Struct"/>
    <Intrinsic Name="_IsRange" Expression="Schema.Type._IsRange()"/>
    <Intrinsic Name="_IsEmptyRange" Expression="Schema.Type._IsRange() &amp;&amp; !Value.Range"/>
    <Intrinsic Name="_IsLeafRange" Expression="Schema.Type._IsRange() &amp;&amp; Schema.InnerRangeType._IsLeaf()"/>

    <DisplayString Condition="_IsEmptyStruct()">{Name}, {Schema,na}, Empty</DisplayString>
    <DisplayString Condition="_IsStruct()">{Name}, {Schema,na}, {Value.Struct,na}</DisplayString>
    <DisplayString Condition="_IsEmptyRange()">{Name}, {Schema,na}, Empty</DisplayString>
    <DisplayString Condition="_IsLeafRange() &amp;&amp; Schema.InnerRangeType.Leaf._IsUtf8()">{Name}, {Schema,na}, {(PlainProps::DbgVis::FBuiltRange*)Value.Range,view(utf8)na}</DisplayString>
    <DisplayString Condition="_IsLeafRange() &amp;&amp; Schema.InnerRangeType.Leaf._IsUtf16()">{Name}, {Schema,na}, {(PlainProps::DbgVis::FBuiltRange*)Value.Range,view(utf16)na}</DisplayString>
    <DisplayString Condition="_IsLeafRange() &amp;&amp; Schema.InnerRangeType.Leaf._IsUtf32()">{Name}, {Schema,na}, {(PlainProps::DbgVis::FBuiltRange*)Value.Range,view(utf32)na}</DisplayString>
    <DisplayString Condition="_IsRange()">{Name}, {Schema,na}, {(PlainProps::DbgVis::FBuiltRange*)Value.Range,na}</DisplayString>
    <DisplayString Condition="_IsLeaf() &amp;&amp; Schema.Type.Leaf._IsBool()"  >{Name}, {Schema}, Value={Value,view(bool)}</DisplayString>
    <DisplayString Condition="_IsLeaf() &amp;&amp; Schema.Type.Leaf._IsU8()"    >{Name}, {Schema}, Value={Value,view(u8)}</DisplayString>
    <DisplayString Condition="_IsLeaf() &amp;&amp; Schema.Type.Leaf._IsU16()"   >{Name}, {Schema}, Value={Value,view(u16)}</DisplayString>
    <DisplayString Condition="_IsLeaf() &amp;&amp; Schema.Type.Leaf._IsU32()"   >{Name}, {Schema}, Value={Value,view(u32)}</DisplayString>
    <DisplayString Condition="_IsLeaf() &amp;&amp; Schema.Type.Leaf._IsU64()"   >{Name}, {Schema}, Value={Value,view(u64)}</DisplayString>
    <DisplayString Condition="_IsLeaf() &amp;&amp; Schema.Type.Leaf._IsS8()"    >{Name}, {Schema}, Value={Value,view(s8)}</DisplayString>
    <DisplayString Condition="_IsLeaf() &amp;&amp; Schema.Type.Leaf._IsS16()"   >{Name}, {Schema}, Value={Value,view(s16)}</DisplayString>
    <DisplayString Condition="_IsLeaf() &amp;&amp; Schema.Type.Leaf._IsS32()"   >{Name}, {Schema}, Value={Value,view(s32)}</DisplayString>
    <DisplayString Condition="_IsLeaf() &amp;&amp; Schema.Type.Leaf._IsS64()"   >{Name}, {Schema}, Value={Value,view(s64)}</DisplayString>
    <DisplayString Condition="_IsLeaf() &amp;&amp; Schema.Type.Leaf._IsFloat()" >{Name}, {Schema}, Value={Value,view(float)}</DisplayString>
    <DisplayString Condition="_IsLeaf() &amp;&amp; Schema.Type.Leaf._IsDouble()">{Name}, {Schema}, Value={Value,view(double)}</DisplayString>
    <DisplayString Condition="_IsLeaf() &amp;&amp; Schema.Type.Leaf._IsHex8()"  >{Name}, {Schema}, Value={Value,view(hex8)}</DisplayString>
    <DisplayString Condition="_IsLeaf() &amp;&amp; Schema.Type.Leaf._IsHex16()" >{Name}, {Schema}, Value={Value,view(hex16)}</DisplayString>
    <DisplayString Condition="_IsLeaf() &amp;&amp; Schema.Type.Leaf._IsHex32()" >{Name}, {Schema}, Value={Value,view(hex32)}</DisplayString>
    <DisplayString Condition="_IsLeaf() &amp;&amp; Schema.Type.Leaf._IsHex64()" >{Name}, {Schema}, Value={Value,view(hex64)}</DisplayString>
    <DisplayString Condition="_IsLeaf() &amp;&amp; Schema.Type.Leaf._IsEnum8()" >{Name}, {Schema}, Value={Value,view(u8)}</DisplayString>
    <DisplayString Condition="_IsLeaf() &amp;&amp; Schema.Type.Leaf._IsEnum16()">{Name}, {Schema}, Value={Value,view(u16)}</DisplayString>
    <DisplayString Condition="_IsLeaf() &amp;&amp; Schema.Type.Leaf._IsEnum32()">{Name}, {Schema}, Value={Value,view(u32)}</DisplayString>
    <DisplayString Condition="_IsLeaf() &amp;&amp; Schema.Type.Leaf._IsEnum64()">{Name}, {Schema}, Value={Value,view(u64)}</DisplayString>
    <DisplayString Condition="_IsLeaf() &amp;&amp; Schema.Type.Leaf._IsUtf8()"  >{Name}, {Schema}, Value={Value,view(utf8)}</DisplayString>
    <DisplayString Condition="_IsLeaf() &amp;&amp; Schema.Type.Leaf._IsUtf16()" >{Name}, {Schema}, Value={Value,view(utf16)}</DisplayString>
    <DisplayString Condition="_IsLeaf() &amp;&amp; Schema.Type.Leaf._IsUtf32()" >{Name}, {Schema}, Value={Value,view(utf32)}</DisplayString>
    <DisplayString>Fallback {Name}, {Schema}, Value={Value}</DisplayString>
    
    <Expand>
      <Item Name="Name">Name</Item>
      <Item Name="Schema">Schema</Item>
      <Item Name="Struct" Condition="_IsStruct()">Value.Struct</Item>
      <Item Name="Range" Condition="_IsRange() &amp;&amp; !Value.Range">Value.Range,!</Item>

      <CustomListItems Condition="_IsLeaf()">
        <Variable Name="Leaf" InitialValue="&amp;Schema.Type.Leaf"/>
        <Item Name="Value" Condition="(*Leaf)._IsBool()"  >Value,view(bool)</Item>
        <Item Name="Value" Condition="(*Leaf)._IsU8()"    >Value,view(u8)</Item>
        <Item Name="Value" Condition="(*Leaf)._IsU16()"   >Value,view(u16)</Item>
        <Item Name="Value" Condition="(*Leaf)._IsU32()"   >Value,view(u32)</Item>
        <Item Name="Value" Condition="(*Leaf)._IsU64()"   >Value,view(u64)</Item>
        <Item Name="Value" Condition="(*Leaf)._IsS16()"   >Value,view(s16)</Item>
        <Item Name="Value" Condition="(*Leaf)._IsS32()"   >Value,view(s32)</Item>
        <Item Name="Value" Condition="(*Leaf)._IsS64()"   >Value,view(s64)</Item>
        <Item Name="Value" Condition="(*Leaf)._IsS8()"    >Value,view(s8)</Item>
        <Item Name="Value" Condition="(*Leaf)._IsFloat()" >Value,view(float)</Item>
        <Item Name="Value" Condition="(*Leaf)._IsDouble()">Value,view(double)</Item>
        <Item Name="Value" Condition="(*Leaf)._IsHex8()"  >Value,view(hex8)</Item>
        <Item Name="Value" Condition="(*Leaf)._IsHex16()" >Value,view(hex16)</Item>
        <Item Name="Value" Condition="(*Leaf)._IsHex32()" >Value,view(hex32)</Item>
        <Item Name="Value" Condition="(*Leaf)._IsHex64()" >Value,view(hex64)</Item>
        <Item Name="Value" Condition="(*Leaf)._IsEnum8()" >Value,view(u8)</Item>
        <Item Name="Value" Condition="(*Leaf)._IsEnum16()">Value,view(u16)</Item>
        <Item Name="Value" Condition="(*Leaf)._IsEnum32()">Value,view(u32)</Item>
        <Item Name="Value" Condition="(*Leaf)._IsEnum64()">Value,view(u64)</Item>
        <Item Name="Value" Condition="(*Leaf)._IsUtf8()"  >Value,view(utf8)</Item>
        <Item Name="Value" Condition="(*Leaf)._IsUtf16()" >Value,view(utf16)</Item>
        <Item Name="Value" Condition="(*Leaf)._IsUtf32()" >Value,view(utf32)</Item>
      </CustomListItems>
      
      <!-- Specialized expansion for one-dimensional ranges -->
      <Item Name="Range" Condition="_IsRange() &amp;&amp; !!Value.Range &amp;&amp; Schema.InnerRangeType._IsStruct()">*(PlainProps::DbgVis::FBuiltRange*)Value.Range,view(struct)</Item>
      <CustomListItems Condition="_IsRange() &amp;&amp; !!Value.Range &amp;&amp; Schema.InnerRangeType._IsLeaf()">
        <Variable Name="Range" InitialValue="(PlainProps::DbgVis::FBuiltRange*)Value.Range"/>
        <Variable Name="Leaf" InitialValue="&amp;Schema.InnerRangeType.Leaf"/>
        <Item Name="Range" Condition="Leaf->_IsBool()"  >Range,view(bool)</Item>
        <Item Name="Range" Condition="Leaf->_IsU8()"    >Range,view(u8)</Item>
        <Item Name="Range" Condition="Leaf->_IsU16()"   >Range,view(u16)</Item>
        <Item Name="Range" Condition="Leaf->_IsU32()"   >Range,view(u32)</Item>
        <Item Name="Range" Condition="Leaf->_IsU64()"   >Range,view(u64)</Item>
        <Item Name="Range" Condition="Leaf->_IsS8()"    >Range,view(s8)</Item>
        <Item Name="Range" Condition="Leaf->_IsS16()"   >Range,view(s16)</Item>
        <Item Name="Range" Condition="Leaf->_IsS32()"   >Range,view(s32)</Item>
        <Item Name="Range" Condition="Leaf->_IsS64()"   >Range,view(s64)</Item>
        <Item Name="Range" Condition="Leaf->_IsFloat()" >Range,view(float)</Item>
        <Item Name="Range" Condition="Leaf->_IsDouble()">Range,view(double)</Item>
        <Item Name="Range" Condition="Leaf->_IsHex8()"  >Range,view(hex8)</Item>
        <Item Name="Range" Condition="Leaf->_IsHex16()" >Range,view(hex16)</Item>
        <Item Name="Range" Condition="Leaf->_IsHex32()" >Range,view(hex32)</Item>
        <Item Name="Range" Condition="Leaf->_IsHex64()" >Range,view(hex64)</Item>
        <Item Name="Range" Condition="Leaf->_IsEnum8()" >Range,view(u8)</Item>
        <Item Name="Range" Condition="Leaf->_IsEnum16()">Range,view(u16)</Item>
        <Item Name="Range" Condition="Leaf->_IsEnum32()">Range,view(u32)</Item>
        <Item Name="Range" Condition="Leaf->_IsEnum64()">Range,view(u64)</Item>
        <Item Name="Range" Condition="Leaf->_IsUtf8()"  >Range,view(utf8)</Item>
        <Item Name="Range" Condition="Leaf->_IsUtf16()" >Range,view(utf16)</Item>
        <Item Name="Range" Condition="Leaf->_IsUtf32()" >Range,view(utf32)</Item>
      </CustomListItems>

      <!-- Specialized expansion for two-dimensional ranges -->
      <Synthetic Name="Range" Condition="_IsRange() &amp;&amp; !!Value.Range &amp;&amp; Schema.NumInnerRanges == 2 &amp;&amp; (*Schema._InnermostType())._IsStruct()">
        <DisplayString>{(PlainProps::DbgVis::FBuiltRange*)Value.Range}</DisplayString>
        <Expand>
          <IndexListItems>
            <Size>(*(PlainProps::DbgVis::FBuiltRange*)Value.Range).Num</Size>
            <ValueNode>(*(PlainProps::DbgVis::FBuiltRange*)Value.Range)._AsRanges()[$i],view(struct)</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>
      <Synthetic Name="Ranges" Condition="_IsRange() &amp;&amp; !!Value.Range &amp;&amp; Schema.NumInnerRanges == 2 &amp;&amp; (*Schema._InnermostType())._IsLeaf()">
        <DisplayString>{(PlainProps::DbgVis::FBuiltRange*)Value.Range}</DisplayString>
        <Expand>
          <CustomListItems>
            <Variable Name="Range" InitialValue="(PlainProps::DbgVis::FBuiltRange*)Value.Range"/>
            <Variable Name="Item" InitialValue="(PlainProps::DbgVis::FBuiltRange*)nullptr"/>
            <Variable Name="Leaf" InitialValue="&amp;(*Schema._InnermostType()).Leaf"/>
            <Variable Name="Num" InitialValue="Range->Num"/>
            <Variable Name="Index" InitialValue="0"/>
            <Loop Condition="Index &lt; Num">
              <Exec>Item = ((PlainProps::DbgVis::FBuiltRange**)((*Range).Data))[Index]</Exec>
              <!-- <Item Name="[{Index}]">Item</Item> -->
              <Item Name="[{Index}]" Condition="Item == nullptr">Item,!</Item>
              <If Condition="Item != nullptr">
                <Item Name="[{Index}]" Condition="Leaf->_IsBool()"  >Item,view(bool)</Item>
                <Item Name="[{Index}]" Condition="Leaf->_IsU8()"    >Item,view(u8)</Item>
                <Item Name="[{Index}]" Condition="Leaf->_IsU16()"   >Item,view(u16)</Item>
                <Item Name="[{Index}]" Condition="Leaf->_IsU32()"   >Item,view(u32)</Item>
                <Item Name="[{Index}]" Condition="Leaf->_IsU64()"   >Item,view(u64)</Item>
                <Item Name="[{Index}]" Condition="Leaf->_IsS8()"    >Item,view(s8)</Item>
                <Item Name="[{Index}]" Condition="Leaf->_IsS16()"   >Item,view(s16)</Item>
                <Item Name="[{Index}]" Condition="Leaf->_IsS32()"   >Item,view(s32)</Item>
                <Item Name="[{Index}]" Condition="Leaf->_IsS64()"   >Item,view(s64)</Item>
                <Item Name="[{Index}]" Condition="Leaf->_IsHex8()"  >Item,view(hex8)</Item>
                <Item Name="[{Index}]" Condition="Leaf->_IsHex16()" >Item,view(hex16)</Item>
                <Item Name="[{Index}]" Condition="Leaf->_IsHex32()" >Item,view(hex32)</Item>
                <Item Name="[{Index}]" Condition="Leaf->_IsHex64()" >Item,view(hex64)</Item>
                <Item Name="[{Index}]" Condition="Leaf->_IsEnum8()" >Item,view(u8)</Item>
                <Item Name="[{Index}]" Condition="Leaf->_IsEnum16()">Item,view(u16)</Item>
                <Item Name="[{Index}]" Condition="Leaf->_IsEnum32()">Item,view(u32)</Item>
                <Item Name="[{Index}]" Condition="Leaf->_IsEnum64()">Item,view(u64)</Item>
                <Item Name="[{Index}]" Condition="Leaf->_IsFloat()" >Item,view(float)</Item>
                <Item Name="[{Index}]" Condition="Leaf->_IsDouble()">Item,view(double)</Item>
                <Item Name="[{Index}]" Condition="Leaf->_IsUtf8()"  >Item,view(utf8)</Item>
                <Item Name="[{Index}]" Condition="Leaf->_IsUtf16()" >Item,view(utf16)</Item>
                <Item Name="[{Index}]" Condition="Leaf->_IsUtf32()" >Item,view(utf32)</Item>
              </If>
              <Exec>++Index</Exec>
            </Loop>
          </CustomListItems>
          <Item Name="[Raw View]">(PlainProps::DbgVis::FBuiltRange*)Value.Range</Item>
        </Expand>
      </Synthetic>

      <!-- Generic range expansion for any type and any rank -->
      <Item Name="Range" Condition="_IsRange() &amp;&amp; !!Value.Range &amp;&amp; Schema.NumInnerRanges &gt; 2">(PlainProps::DbgVis::FBuiltRange*)Value.Range,view(range)</Item>
    </Expand>
  </Type>

  <Type Name="PlainProps::FBuiltValue">
    <Intrinsic Name="_AsBool"   Expression="*(bool*)&amp;Leaf"/>
    <Intrinsic Name="_AsU8"     Expression="*(uint8*)&amp;Leaf"/>
    <Intrinsic Name="_AsU16"    Expression="*(uint16*)&amp;Leaf"/>
    <Intrinsic Name="_AsU32"    Expression="*(uint32*)&amp;Leaf"/>
    <Intrinsic Name="_AsU64"    Expression="*(uint64*)&amp;Leaf"/>
    <Intrinsic Name="_AsS8"     Expression="*(int8*)&amp;Leaf"/>
    <Intrinsic Name="_AsS16"    Expression="*(int16*)&amp;Leaf"/>
    <Intrinsic Name="_AsS32"    Expression="*(int32*)&amp;Leaf"/>
    <Intrinsic Name="_AsS64"    Expression="*(int64*)&amp;Leaf"/>
    <Intrinsic Name="_AsFloat"  Expression="*(float*)&amp;Leaf"/>
    <Intrinsic Name="_AsDouble" Expression="*(double*)&amp;Leaf"/>
    <Intrinsic Name="_AsChar8"  Expression="*(char8_t*)&amp;Leaf" Optional="1"/>
    <Intrinsic Name="_AsChar8"  Expression="*(char*)&amp;Leaf"/>
    <Intrinsic Name="_AsChar16" Expression="*(char16_t*)&amp;Leaf" Optional="1"/>
    <Intrinsic Name="_AsChar16" Expression="*(UTF16CHAR*)&amp;Leaf"/>
    <Intrinsic Name="_AsChar32" Expression="*(char32_t*)&amp;Leaf" Optional="1"/>
    <Intrinsic Name="_AsChar32" Expression="*(UTF32CHAR*)&amp;Leaf"/>

    <DisplayString IncludeView="struct">{Struct}</DisplayString>
    <DisplayString IncludeView="range" >{Range}</DisplayString>
    <DisplayString IncludeView="bool"  >{_AsBool()}</DisplayString>
    <DisplayString IncludeView="u8"    >{_AsU8(),nvo}</DisplayString>
    <DisplayString IncludeView="u16"   >{_AsU16()}</DisplayString>
    <DisplayString IncludeView="u32"   >{_AsU32()}</DisplayString>
    <DisplayString IncludeView="u64"   >{_AsU64()}</DisplayString>
    <DisplayString IncludeView="s8"    >{_AsS8(),nvo}</DisplayString>
    <DisplayString IncludeView="s16"   >{_AsS16()}</DisplayString>
    <DisplayString IncludeView="s32"   >{_AsS32()}</DisplayString>
    <DisplayString IncludeView="s64"   >{_AsS64()}</DisplayString>
    <DisplayString IncludeView="hex8"  >{_AsU8(),nvox}</DisplayString>
    <DisplayString IncludeView="hex16" >{_AsU16(),x}</DisplayString>
    <DisplayString IncludeView="hex32" >{_AsU32(),x}</DisplayString>
    <DisplayString IncludeView="hex64" >{_AsU64(),x}</DisplayString>
    <DisplayString IncludeView="float" >{_AsFloat()}</DisplayString>
    <DisplayString IncludeView="double">{_AsDouble()}</DisplayString>
    <DisplayString IncludeView="utf8"  >{_AsChar8()}</DisplayString>
    <DisplayString IncludeView="utf16" >{_AsChar16()}</DisplayString>
    <DisplayString IncludeView="utf32" >{_AsChar32()}</DisplayString>
    <DisplayString>{Leaf,x}</DisplayString>
    <Expand>
      <Item ExcludeView="Leaf" Name="Leaf">Leaf,x</Item>
      <Item ExcludeView="XXXXXX;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="Struct"  >Struct</Item>
      <Item ExcludeView="struct;XXXXX;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="Range"   >Range</Item>
      <Item ExcludeView="struct;range;XXXX;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[Bool]"  >_AsBool()</Item>
      <Item ExcludeView="struct;range;bool;XX;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[U8]"    >_AsU8(),nvo</Item>
      <Item ExcludeView="struct;range;bool;u8;XXX;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[U16]"   >_AsU16()</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;XXX;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[U32]"   >_AsU32()</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;XXX;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[U64]"   >_AsU64()</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;XX;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[S8]"    >_AsS8(),nvo</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;XXX;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[S16]"   >_AsS16()</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;XXX;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[S32]"   >_AsS32()</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;XXX;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[S64]"   >_AsS64()</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;XXXX;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[Hex8]"  >_AsU8(),nvox</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;XXXXX;hex32;hex64;float;double;utf8;utf16;utf32" Name="[Hex16]" >_AsU16(),x</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;XXXXX;hex64;float;double;utf8;utf16;utf32" Name="[Hex32]" >_AsU32(),x</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;XXXXX;float;double;utf8;utf16;utf32" Name="[Hex64]" >_AsU64(),x</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;XXXXX;double;utf8;utf16;utf32" Name="[Float]" >_AsFloat()</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;XXXXXX;utf8;utf16;utf32" Name="[Double]">_AsDouble()</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;XXXX;utf16;utf32" Name="[Utf8]"  >_AsChar8()</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;XXXXX;utf32" Name="[Utf16]" >_AsChar16()</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;XXXXX" Name="[Utf32]" >_AsChar32()</Item>
    </Expand>
  </Type>


  <!-- PlainPropsBuildSchema.h -->

  <Type Name="PlainProps::FBuiltEnumSchema">
    <DisplayString>{Id}, Mode={Mode,en}, Width={Width,en}, NumEnumerators={Names.ArrayNum}</DisplayString>
  </Type>

  <Type Name="PlainProps::FBuiltStructSchema">
    <DisplayString Condition="Super._IsSet() &amp;&amp; bDense" ExcludeView="nn">{Id}, Super={Super}, NumMembers={MemberNames.ArrayNum} (Dense)</DisplayString>
    <DisplayString Condition="Super._IsSet() &amp;&amp; bDense">{Id,view(nn)}, Super={Super,view(nn)}, NumMembers={MemberNames.ArrayNum} (Dense)</DisplayString>

    <DisplayString Condition="Super._IsSet()" ExcludeView="nn">{Id}, Super={Super}, NumMembers={MemberNames.ArrayNum} (Sparse)</DisplayString>
    <DisplayString Condition="Super._IsSet()">{Id,view(nn)}, Super={Super,view(nn)}, NumMembers={MemberNames.ArrayNum} (Sparse)</DisplayString>

    <DisplayString Condition="bDense" ExcludeView="nn">{Id}, NumMembers={MemberNames.ArrayNum} (Dense)</DisplayString>
    <DisplayString Condition="bDense">{Id,view(nn)}, NumMembers={MemberNames.ArrayNum} (Dense)</DisplayString>

    <DisplayString ExcludeView="nn">{Id}, NumMembers={MemberNames.ArrayNum} (Sparse)</DisplayString>
    <DisplayString>{Id,view(nn)}, NumMembers={MemberNames.ArrayNum} (Sparse)</DisplayString>
  </Type>


  <!-- PlainPropsBuildSchema.cpp -->

  <Type Name="PlainProps::FStructSchemaBuilder">
    <Intrinsic Name="_NumMembers" Expression="Declaration.NumMembers+!!Declaration.Super.Idx"/>
    <DisplayString>{Declaration}</DisplayString>
    <Expand>
      <ExpandedItem>this,!</ExpandedItem>
      <Synthetic Name="[MemberOrder]">
        <DisplayString>Num={_NumMembers()}</DisplayString>
        <Expand>
          <IndexListItems>
            <Size>_NumMembers()</Size>
            <ValueNode>MemberOrder[$i]</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>
      <Synthetic Name="[NotedSchemas]">
        <DisplayString>Num={_NumMembers()}</DisplayString>
        <Expand>
          <IndexListItems>
            <Size>_NumMembers()</Size>
            <ValueNode>NotedSchemas[$i]</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>
    </Expand>
  </Type>


  <!-- PlainPropsCtti.h -->
  <Type Name="PlainProps::TScopedStructBinding&lt;*,*,*&gt;">
    <DisplayString Condition="BindId.Idx != DeclId.Idx">{BindId}, {"$T3",sb} ({DeclId}) </DisplayString>
    <DisplayString>{BindId}, {"$T3",sb}</DisplayString>
  </Type>


  <!-- PlainPropsDeclare.h -->

  <Type Name="PlainProps::FOptionalLeafWidth">
    <DisplayString Condition="Value == 0xFF">NoWidth</DisplayString>
    <DisplayString>{(ELeafWidth)Value,nvo}</DisplayString>
  </Type>

  <Type Name="PlainProps::FEnumDeclaration">
    <DisplayString ExcludeView="nn" Optional="1">{Type.Name}, NumEnumerators={NumEnumerators} ({Mode,en})</DisplayString>
    <DisplayString>{Type.Name,view(nn)}, NumEnumerators={NumEnumerators} ({Mode,en})</DisplayString>
    <Expand>
      <Item Name="Type">Type</Item>
      <Item Name="Mode">Mode,nvo</Item>
      <Item Name="Width">Width</Item>
      <Synthetic Name="[Enumerators]" Condition="NumEnumerators &gt; 0">
        <DisplayString>Num={NumEnumerators}</DisplayString>
        <Expand>
          <IndexListItems>
            <Size>NumEnumerators</Size>
            <ValueNode>Enumerators[$i]</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>
    </Expand>
  </Type>

  <Type Name="PlainProps::FStructDeclaration">
    <Intrinsic Name="_Struct" Expression="_ppResolveStruct(Id.Idx)"/>
    <Intrinsic Name="_Super" Expression="_ppResolveStruct(Super.Idx)"/>
    
    <DisplayString Condition="Super._IsSet()" ExcludeView="nn" Optional="1">{_Struct().Name}, Super={_Super().Name}, NumMembers={NumMembers} ({Occupancy,en})</DisplayString>
    <DisplayString Condition="Super._IsSet()">{_Struct().Name,view(nn)}, Super={_Super().Name,view(nn)}, NumMembers={NumMembers} ({Occupancy,en}))</DisplayString>
    
    <DisplayString ExcludeView="nn" Optional="1">{_Struct().Name}, NumMembers={NumMembers} ({Occupancy,en})</DisplayString>
    <DisplayString>{Id,view(nn)}, NumMembers={NumMembers} ({Occupancy,en})</DisplayString>
    
    <Expand>
      <Item Name="Id">Id</Item>
      <Item Name="Super" Condition="Super._IsSet()">Super</Item>
      <Item Name="Occupancy">Occupancy</Item>
      <Item Name="RefCount">RefCount</Item>
      <Synthetic Name="[MemberNames]">
        <DisplayString>Num={NumMembers}</DisplayString>
        <Expand>
          <IndexListItems>
            <Size>NumMembers</Size>
            <ValueNode>MemberNames[$i]</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>
      <Synthetic Name="[InnerIds]">
        <DisplayString>Num={NumInnerIds}</DisplayString>
        <Expand>
          <CustomListItems>
            <Variable Name="InnerIds" InitialValue="(FInnerId*)_AlignInt((uintptr_t)(MemberNames + NumMembers), sizeof(FInnerId))"/>
            <Variable Name="Index" InitialValue="0"/>
            <Loop Condition="Index &lt; NumInnerIds">
              <Item>InnerIds[Index],view(nr)</Item>
              <Exec>++Index</Exec>
            </Loop>
          </CustomListItems>
        </Expand>
      </Synthetic>
      <Synthetic Name="[Types]">
        <DisplayString>Num={NumMembers}</DisplayString>
        <Expand>
          <CustomListItems>
            <Variable Name="InnerIds" InitialValue="(FInnerId*)_AlignInt((uintptr_t)(MemberNames + NumMembers), sizeof(FInnerId))"/>
            <Variable Name="Types" InitialValue="(FMemberType*)_AlignInt((uintptr_t)(InnerIds + NumInnerIds), sizeof(FInnerId))"/>
            <Variable Name="Index" InitialValue="0"/>
            <Loop Condition="Index &lt; NumMembers">
              <Item>Types[Index],view(nr)</Item>
              <Exec>++Index</Exec>
            </Loop>
          </CustomListItems>
        </Expand>
      </Synthetic>
      <Synthetic Name="[InnerRangeTypes]">
        <DisplayString>Num={NumInnerRanges}</DisplayString>
        <Expand>
          <CustomListItems>
            <Variable Name="InnerIds" InitialValue="(FInnerId*)_AlignInt((uintptr_t)(MemberNames + NumMembers), sizeof(FInnerId))"/>
            <Variable Name="Types" InitialValue="(FMemberType*)_AlignInt((uintptr_t)(InnerIds + NumInnerIds), sizeof(FInnerId))"/>
            <Variable Name="InnerRangeTypes" InitialValue="(FMemberType*)(Types + NumMembers)"/>
            <Variable Name="Index" InitialValue="0"/>
            <Loop Condition="Index &lt; NumInnerRanges">
              <Item>InnerRangeTypes[Index],view(nr)</Item>
              <Exec>++Index</Exec>
            </Loop>
          </CustomListItems>
        </Expand>
      </Synthetic>
    </Expand>
  </Type>

  <!-- PlainPropsLoad.cpp -->
  
  <Type Name="PlainProps::FLoadBatch" Priority="MediumHigh">
    <DisplayString>{(PlainProps::DbgVis::FLoadBatch*)this,na}</DisplayString>
    <Expand HideRawView="1">
      <ExpandedItem>(PlainProps::DbgVis::FLoadBatch*)this</ExpandedItem>
      <Item Name="[Raw View]">(PlainProps::DbgVis::FLoadBatch*)this,!</Item>
    </Expand>
  </Type>

  <Type Name="PlainProps::FLoadStructPlan">
    <Intrinsic Name="_IsSchema" Expression="(Handle &amp; SchemaBit) == SchemaBit"/>
    <Intrinsic Name="_IsSparseSchema" Expression="(Handle &amp; SparseSchemaMask) == SparseSchemaMask"/>
    <Intrinsic Name="_IsMemcpy" Expression="(Handle &amp; LoMask) == MemcpyMask"/>
    <Intrinsic Name="_IsCustom" Expression="(Handle &amp; LoMask) == CustomMask"/>
    <Intrinsic Name="_Ptr" Expression="(Handle &amp; PtrMask)"/>
    <Intrinsic Name="_AsSchema" Expression="(FSchemaBinding*)_Ptr()"/>
    <Intrinsic Name="_AsCustom" Expression="(ICustomBinding*)_Ptr()"/>
    <Intrinsic Name="_MemcpySize" Expression="(uint32)(Handle >> 32)"/>
    <Intrinsic Name="_MemcpyOffset" Expression="(uint32)(Handle) >> 2"/>
    <DisplayString Condition="_IsSchema() &amp;&amp; _IsSparseSchema()">Schema: {_AsSchema()} (Sparse)</DisplayString>
    <DisplayString Condition="_IsSchema()">Schema: {_AsSchema()} (Dense)</DisplayString>
    <DisplayString Condition="_IsCustom()">Custom: {_AsCustom()}</DisplayString>
    <DisplayString Condition="_IsMemcpy()">Memcpy: Offset={_MemcpyOffset()}, Size={_MemcpySize()}</DisplayString>
    <Expand>
      <Item Name="[SchemaBinding]" Condition="_IsSchema()">_AsSchema()</Item>
      <Item Name="[CustomBinding]" Condition="_IsCustom()">_AsCustom()</Item>
      <Item Name="[Memcpy.Offset]" Condition="_IsMemcpy()">_MemcpyOffset()</Item>
      <Item Name="[Memcpy.Size]" Condition="_IsMemcpy()">_MemcpySize()</Item>
      <Item Name="[IsSchema]">_IsSchema()</Item>
      <Item Name="[IsSparseSchema]">_IsSparseSchema()</Item>
      <Item Name="[IsMemcpy]">_IsMemcpy()</Item>
      <Item Name="[IsCustom]">_IsCustom()</Item>
      <Item Name="Handle">Handle,x</Item>
    </Expand>
  </Type>

  <Type Name="PlainProps::FLateLoadPlans">
    <AlternativeType Name="PlainProps::DbgVis::FLateLoadPlans"/>
    <DisplayString>{{Data={(void*)Data}, Next={(PlainProps::DbgVis::FLoadBatch*)Next}}}</DisplayString>
  </Type>

  <Type Name="PlainProps::FLoadBatch">
    <AlternativeType Name="PlainProps::DbgVis::FLoadBatch"/>
    <Intrinsic Name="_ResolveStruct" Expression="LoadIdx &lt; NumReadSchemas ? _ppResolveBatch(BatchId.Idx)->_ResolveStruct(LoadIdx) : _ppResolveBatch(BatchId.Idx)->_ResolveStruct(LoadIdx - NumReadSchemas)">
      <Parameter Name="LoadIdx" Type="uint32"/>
    </Intrinsic>
    <DisplayString>{BatchId}, NumPlans={NumPlans}, NumReadSchemas={NumReadSchemas}, NumMappings={NumPlans-NumReadSchemas}</DisplayString>
    <Expand>
      <Item Name="BatchId">BatchId</Item>
      <!-- Full visualization of FLoadStructPlan -->
      <Synthetic Name="[Plans]" Condition="NumPlans &gt; 0">
        <DisplayString>Num={NumPlans}</DisplayString>
        <Expand>
          <IndexListItems Optional="1">
            <Size>NumPlans</Size>
            <ValueNode>((PlainProps::FLoadStructPlan*)Plans)[$i]</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>
      <!-- Simplified raw visualization of FLoadStructPlan that works in all modules -->
      <Synthetic Name="[Plans.Handles]" Condition="NumPlans &gt; 0">
        <DisplayString>Num={NumPlans}</DisplayString>
        <Expand>
          <IndexListItems>
            <Size>NumPlans</Size>
            <ValueNode>((PlainProps::DbgVis::FLoadStructPlan*)Plans)[$i].Handle,x</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>
      <Synthetic Name="[Mappings]" Condition="NumReadSchemas &gt; 0">
        <DisplayString>Num={NumPlans - NumReadSchemas}</DisplayString>
        <Expand>
          <CustomListItems>
            <Variable Name="SaveIds" InitialValue="(PlainProps::FStructSchemaId*)(Plans+MaxPlans)"/>
            <Variable Name="Index" InitialValue="0"/>
            <Variable Name="Num" InitialValue="NumPlans - NumReadSchemas"/>
            <Loop Condition="Index &lt; Num">
              <Item>SaveIds + Index,view(nr)</Item>
              <Exec>++Index</Exec>
            </Loop>
          </CustomListItems>
        </Expand>
      </Synthetic>
      <Synthetic Name="[LatePlans]" Condition="LatePlans">
        <DisplayString Condition="LatePlans &amp;&amp; LatePlans->Next">Num>=1</DisplayString>
        <DisplayString Condition="LatePlans">Num=1</DisplayString>
        <DisplayString>Num=0</DisplayString>
        <Expand>
          <LinkedListItems>
            <HeadPointer>LatePlans</HeadPointer>
            <NextPointer>Next</NextPointer>
            <ValueNode>(void*)Data</ValueNode>
          </LinkedListItems>
        </Expand>
      </Synthetic>
    </Expand>
  </Type>


  <!-- PlainPropsLoadMember.h -->

  <Type Name="PlainProps::FSchemaLoadHandle">
    <Intrinsic Name="_Batch" Expression="(PlainProps::DbgVis::FLoadBatch*)&amp;Batch"/>
    <DisplayString ExcludeView="nr" Optional="0">{LoadId}, {_Batch()->_ResolveStruct(LoadId.Idx)}</DisplayString>
    <DisplayString>{LoadId}, {Batch}</DisplayString>
    <Expand>
      <Item Name="LoadId">LoadId</Item>
      <Item Name="[StructSchema]">_Batch()->_ResolveStruct(LoadId.Idx)</Item>
      <Item Name="Batch">Batch</Item>
    </Expand>
  </Type>

  <Type Name="PlainProps::FRangeLoadSchema">
    <Intrinsic Name="_Batch" Expression="(PlainProps::DbgVis::FLoadBatch*)&amp;Batch"/>
    <DisplayString Condition="ItemType._IsStruct()">{InnermostId._Ptr(),view(struct)}({ItemType})[]</DisplayString>
    <DisplayString Condition="ItemType._IsEnum()">{InnermostId._Ptr(),view(enum)}({ItemType})[]</DisplayString>
    <DisplayString>{ItemType}[]</DisplayString>
    <Expand>
      <Item Name="ItemType">ItemType</Item>
      <Item Name="InnermostId" Condition="InnermostId._IsSet()">InnermostId</Item>
      <Item Name="[StructSchema]" Condition="ItemType._IsStruct()">_Batch()->_ResolveStruct(InnermostId.Idx)</Item>
      <Item Name="Batch">Batch</Item>
    </Expand>
  </Type>

  <Type Name="PlainProps::FStructLoadView">
    <DisplayString>{Schema}, {Values}</DisplayString>
  </Type>

  <Type Name="PlainProps::FRangeLoadView">
    <DisplayString>{Schema}, NumItems={NumItems}, {Values}</DisplayString>
  </Type>
  

  <!-- PlainPropsRead.h -->

  <Type Name="PlainProps::FSchemaBatchId">
    <DisplayString ExcludeView="nr" Optional="1">Batch{Idx}, {_ppResolveBatch(Idx),na}</DisplayString>
    <DisplayString>Batch{Idx}</DisplayString>
    <Expand>
      <Item Name="[Batch]" ExcludeView="nr">_ppResolveBatch(Idx)</Item>
    </Expand>
  </Type>

  <Type Name="PlainProps::FByteReader">
    <Intrinsic Name="_NumBytes" Expression="End-It"/>
    <DisplayString>It={(void*)It}, NumBytes={_NumBytes()}</DisplayString>
    <Expand>
      <ExpandedItem>this,!</ExpandedItem>
      <Synthetic Name="[Bytes]">
        <DisplayString>NumBytes={_NumBytes()}</DisplayString>
        <Expand>
          <IndexListItems>
            <Size>_NumBytes()</Size>
            <ValueNode>It[$i],nvox</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>
    </Expand>
  </Type>

  <Type Name="PlainProps::FRangeSchema">
    <DisplayString Condition="ItemType._IsStruct()">{InnermostSchema._Ptr(),view(struct)}({ItemType})[]</DisplayString>
    <DisplayString Condition="ItemType._IsEnum()">{InnermostSchema._Ptr(),view(enum)}({ItemType})[]</DisplayString>
    <DisplayString>{ItemType}[]</DisplayString>
    <Expand>
      <Item Name="ItemType">ItemType</Item>
      <Item Name="InnermostSchema" Condition="InnermostSchema._IsSet()">InnermostSchema</Item>
      <Item Name="[StructSchema]" Condition="ItemType._IsStruct()">_ppResolveStructSchema(Batch.Idx, InnermostSchema.Idx)</Item>
      <Item Name="[EnumSchema]" Condition="ItemType._IsEnum()">_ppResolveEnumSchema(Batch.Idx, InnermostSchema.Idx)</Item>
      <Item Name="Batch">Batch</Item>
      <Item Name="NestedItemTypes">NestedItemTypes</Item>
    </Expand>
  </Type>


  <!-- PlainPropsSave.h -->

  <Type Name="PlainProps::FDefaultLink">
    <DisplayString>{(void*)Ptr}</DisplayString>
  </Type>

  <Type Name="PlainProps::FBaseline">
    <Intrinsic Name="_IsChain"  Expression="(Ptr &amp; TagBit) != 0"/>
    <Intrinsic Name="_AsLink"   Expression="(PlainProps::FDefaultLink*)(Ptr &amp; ~TagBit)"/>
    <Intrinsic Name="_Get"       Expression="_IsChain() ? (void*)_AsLink()->Ptr : (void*)Ptr"/>
    <Intrinsic Name="_SuperLink" Expression="_AsLink()[1]"/>
    <Intrinsic Name="_Super"     Expression="(void*)(_SuperLink().Ptr &amp; ~TagBit)"/>
    <DisplayString Condition="_IsChain()">Default={_Get()}, Super={_Super()}</DisplayString>
    <DisplayString >{_Get()}</DisplayString>
    <Expand>
      <Item Name="Default" Condition="_IsChain()">_Get()</Item>
      <Item Name="Super" Condition="_IsChain()">_Super()</Item>
      <Synthetic Name="Chain" Condition="_IsChain()">
        <DisplayString>Num>=2</DisplayString>
        <Expand>
          <CustomListItems Condition="_IsChain()">
            <Variable Name="Link" InitialValue="_AsLink()"/>
            <Variable Name="Index" InitialValue="0"/>
            <Loop>
              <Break Condition="(Link->Ptr &amp; TagBit) != 0"/>
              <Item Name="[{Index}]">(void*)Link->Ptr</Item>
              <Exec>Link++</Exec>
              <Exec>Index++</Exec>
            </Loop>
            <Item Name="[{Index}]">(void*)(Link->Ptr &amp; ~TagBit)</Item>
          </CustomListItems>
        </Expand>
      </Synthetic>
    </Expand>
  </Type>


  <!-- PlainPropsInternalFormat.h -->

  <Type Name="PlainProps::FSchemaBatch" Priority="MediumHigh">
    <DisplayString>{(PlainProps::DbgVis::FSchemaBatch*)this,na}</DisplayString>
    <Expand HideRawView="1">
      <ExpandedItem>(PlainProps::DbgVis::FSchemaBatch*)this</ExpandedItem>
      <Item Name="[Raw View]">(PlainProps::DbgVis::FSchemaBatch*)this,!</Item>
    </Expand>
  </Type>

  <Type Name="PlainProps::FStructSchema" Priority="MediumHigh">
    <DisplayString>{(PlainProps::DbgVis::FStructSchema*)this,na}</DisplayString>
    <Expand HideRawView="1">
      <ExpandedItem>(PlainProps::DbgVis::FStructSchema*)this</ExpandedItem>
      <Item Name="[Raw View]">(PlainProps::DbgVis::FStructSchema*)this,!</Item>
    </Expand>
  </Type>

  <Type Name="PlainProps::FEnumSchema" Priority="MediumHigh">
    <DisplayString>{(PlainProps::DbgVis::FEnumSchema*)this,na}</DisplayString>
    <Expand HideRawView="1">
      <ExpandedItem>(PlainProps::DbgVis::FEnumSchema*)this</ExpandedItem>
      <Item Name="[Raw View]">(PlainProps::DbgVis::FEnumSchema*)this,!</Item>
    </Expand>
  </Type>

  <Type Name="PlainProps::FSchemaBatch">
    <AlternativeType Name="PlainProps::DbgVis::FSchemaBatch"/>
    <Intrinsic Name="_ResolveStruct" Expression="(PlainProps::DbgVis::FStructSchema*)(((uint8*)this) + SchemaOffsets[Idx])">
      <Parameter Name="Idx" Type="uint32"/>
    </Intrinsic>
    <Intrinsic Name="_ResolveEnum" Expression="(PlainProps::DbgVis::FEnumSchema*)(((uint8*)this) + SchemaOffsets[NumStructSchemas + Idx])">
      <Parameter Name="Idx" Type="uint32"/>
    </Intrinsic>
    <DisplayString>NumSchemas={NumSchemas}, NumStructSchemas={NumStructSchemas}, NumEnumSchemas={NumSchemas - NumStructSchemas}</DisplayString>
    <Expand>
      <Synthetic Name="[StructOffsets]" Condition="NumStructSchemas &gt; 0" IncludeView="debug">
        <DisplayString>Num={NumStructSchemas}</DisplayString>
        <Expand>
          <IndexListItems>
            <Size>NumStructSchemas</Size>
            <ValueNode>SchemaOffsets[$i]</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>
      <Synthetic Name="[EnumOffsets]" Condition="(NumSchemas - NumStructSchemas) &gt; 0" IncludeView="debug">
        <DisplayString>Num={NumSchemas - NumStructSchemas}</DisplayString>
        <Expand>
          <IndexListItems>
            <Size>NumSchemas - NumStructSchemas</Size>
            <ValueNode>(SchemaOffsets + NumStructSchemas)[$i]</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>
      <Synthetic Name="[StructSchemas]" Condition="NumStructSchemas &gt; 0"> 
        <DisplayString>Num={NumStructSchemas}</DisplayString>
        <Expand> 
          <IndexListItems> 
            <Size>NumStructSchemas</Size> 
            <ValueNode>(PlainProps::DbgVis::FStructSchema*)(((uint8*)this) + SchemaOffsets[$i]),view(nr)</ValueNode> 
          </IndexListItems> 
        </Expand> 
      </Synthetic> 
      <Synthetic Name="[EnumSchemas]" Condition="(NumSchemas - NumStructSchemas) &gt; 0"> 
        <DisplayString>Num={NumSchemas - NumStructSchemas}</DisplayString> 
        <Expand> 
          <IndexListItems> 
            <Size>NumSchemas - NumStructSchemas</Size> 
            <ValueNode>(PlainProps::DbgVis::FEnumSchema*)(((uint8*)this) + SchemaOffsets[NumStructSchemas + $i]),view(nr)</ValueNode> 
          </IndexListItems> 
        </Expand> 
      </Synthetic> 
      <Synthetic Name="[NestedScopes]" Condition="NumNestedScopes &gt; 0">
        <DisplayString>Num={NumNestedScopes}</DisplayString>
        <Expand>
          <IndexListItems>
            <Size>NumNestedScopes</Size>
            <ValueNode>((PlainProps::FNestedScope*)(((uint8*)this) + NestedScopesOffset))+$i,view(nr)</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>
      <Synthetic Name="[ParametricTypes]" Condition="NumParametricTypes &gt; 0">
        <DisplayString>Num={NumParametricTypes}</DisplayString>
        <Expand>
          <CustomListItems>
            <Variable Name="NestedScopes" InitialValue="(PlainProps::FNestedScope*)(((uint8*)this) + NestedScopesOffset)"/>
            <Variable Name="ParametricTypes" InitialValue="(FParametricType*)(NestedScopes + NumNestedScopes)"/>
            <Variable Name="Index" InitialValue="0"/>
            <Loop Condition="Index &lt; NumParametricTypes">
              <Item>ParametricTypes + Index,view(nr)</Item>
              <Exec>++Index</Exec>
            </Loop>
          </CustomListItems>
        </Expand>
      </Synthetic>
      <Synthetic Name="[Parameters]" Condition="NumParametricTypes &gt; 0">
        <DisplayString>Num>={NumParametricTypes}</DisplayString>
        <Expand>
          <CustomListItems>
            <Variable Name="NestedScopes" InitialValue="(PlainProps::FNestedScope*)(((uint8*)this) + NestedScopesOffset)"/>
            <Variable Name="ParametricTypes" InitialValue="(FParametricType*)(NestedScopes + NumNestedScopes)"/>
            <Variable Name="LastParametricType" InitialValue="ParametricTypes + NumParametricTypes - 1"/>
            <Variable Name="Parameters" InitialValue="(FType*)(ParametricTypes + NumParametricTypes)"/>
            <Variable Name="NumParameters" InitialValue="LastParametricType->Parameters.Idx + LastParametricType->Parameters.NumParameters"/>
            <Variable Name="Index" InitialValue="0"/>
            <Item Name="Num">NumParameters</Item>
            <Loop Condition="Index &lt; NumParameters">
              <Item>Parameters + Index,view(nr)</Item>
              <Exec>++Index</Exec>
            </Loop>
          </CustomListItems>
        </Expand>
      </Synthetic>
    </Expand>
  </Type>

  <Type Name="PlainProps::FStructSchema">
    <AlternativeType Name="PlainProps::DbgVis::FStructSchema"/>

    <Intrinsic Name="_NumNames" Expression="NumMembers - _ppUsesSuper(Inheritance)"/>
    <DisplayString Condition="IsDense">{Type,view(nr)}, Inheritance={Inheritance,en}, NumMembers={NumMembers} (Dense)</DisplayString>
    <DisplayString>{Type,view(nr)}, Inheritance={Inheritance,en}, NumMembers={NumMembers} (Sparse)</DisplayString>
    <Expand>
      <Item Name="Type">Type,view(nr)</Item>
      <Item Name="Inheritance">Inheritance</Item>
      <Item Name="IsDense">IsDense</Item>
      <Synthetic Name="[MemberTypes]" Condition="NumMembers &gt; 0">
        <DisplayString>Num={NumMembers}</DisplayString>
        <Expand>
          <IndexListItems>
            <Size>NumMembers</Size>
            <ValueNode>((FMemberType*)(Footer))[$i]</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>
      <Synthetic Name="[RangeTypes]" Condition="NumRangeTypes &gt; 0">
        <DisplayString>Num={NumRangeTypes}</DisplayString>
        <Expand>
          <IndexListItems>
            <Size>NumRangeTypes</Size>
            <ValueNode>((FMemberType*)(Footer + NumMembers))[$i]</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>
      <Synthetic Name="[MemberNames]" Condition="this-&gt;_NumNames() &gt; 0">
        <DisplayString>Num={this-&gt;_NumNames()}</DisplayString>
        <Expand>
          <CustomListItems>
            <Variable Name="MemberNames" InitialValue="(FMemberId*)_AlignInt((uintptr_t)(Footer + NumMembers + NumRangeTypes), sizeof(FMemberId))"/>
            <Variable Name="Index" InitialValue="0"/>
            <Variable Name="NumNames" InitialValue="this-&gt;_NumNames()"/>
            <Loop Condition="Index &lt; NumNames">
              <Item>MemberNames[Index],view(nr)</Item>
              <Exec>++Index</Exec>
            </Loop>
          </CustomListItems>
        </Expand>
      </Synthetic>
      <Synthetic Name="[InnerSchemas]" Condition="NumInnerSchemas &gt; 0">
        <DisplayString>Num={NumInnerSchemas}</DisplayString>
        <Expand>
          <CustomListItems>
            <Variable Name="MemberNames" InitialValue="(FMemberId*)_AlignInt((uintptr_t)(Footer + NumMembers + NumRangeTypes), sizeof(FMemberId))"/>
            <Variable Name="InnerSchemas" InitialValue="(FSchemaId*)(MemberNames + NumMembers)"/>
            <Variable Name="Index" InitialValue="0"/>
            <Loop Condition="Index &lt; NumInnerSchemas">
              <Item>InnerSchemas[Index],view(nr)</Item>
              <Exec>++Index</Exec>
            </Loop>
          </CustomListItems>
        </Expand>
      </Synthetic>
    </Expand>
  </Type>
  
  <Type Name="PlainProps::FEnumSchema">
    <AlternativeType Name="PlainProps::DbgVis::FEnumSchema"/>
    <DisplayString Condition="ExplicitConstants">{Type,view(nr)}, Mode={(EEnumMode)FlagMode,en}, Width={Width,en}, NumEnumerators={Num} (Explicit)</DisplayString>
    <DisplayString>{Type,view(nr)}, Mode={(EEnumMode)FlagMode,en}, Width={Width,en}, NumEnumerators={Num} (Sequential)</DisplayString>
    <Expand>
      <Item Name="Type">Type,view(nr)</Item>
      <Item Name="FlagMode">FlagMode</Item>
      <Item Name="ExplicitConstants">ExplicitConstants</Item>
      <Item Name="Width">Width</Item>
      <Synthetic Name="[Names]" Condition="Num &gt; 0">
        <DisplayString>Num={Num}</DisplayString>
        <Expand>
          <IndexListItems>
            <Size>Num</Size>
            <ValueNode>Footer[$i],view(nr)</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>
      <Synthetic Name="[ExplicitConstants]" Condition="Num &gt; 0 &amp;&amp; !!ExplicitConstants">
        <DisplayString>Num={Num}</DisplayString>
        <Expand>
          <IndexListItems>
            <Size>Num</Size>
            <ValueNode Condition="Width == ELeafWidth::B8" >((uint8* )(Footer+Num))[$i],nvo</ValueNode>
            <ValueNode Condition="Width == ELeafWidth::B16">((uint16*)(Footer+Num))[$i]</ValueNode>
            <ValueNode Condition="Width == ELeafWidth::B32">((uint32*)(Footer+Num))[$i]</ValueNode>
            <ValueNode Condition="Width == ELeafWidth::B64">((uint64*)(Footer+Num))[$i]</ValueNode>
            <ValueNode>(Footer+Num)[$i]</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>
      <Synthetic Name="[SequentialConstants]" Condition="Num &gt; 0 &amp;&amp; !ExplicitConstants">
        <DisplayString>Num={Num}</DisplayString>
        <Expand>
          <CustomListItems Condition="FlagMode">
            <Variable Name="Index" InitialValue="0"/>
            <Loop Condition="Index &lt; Num">
              <Item>1 &lt;&lt; Index</Item>
              <Exec>++Index</Exec>
            </Loop>
          </CustomListItems>
          <CustomListItems Condition="!FlagMode">
            <Variable Name="Index" InitialValue="0"/>
            <Loop Condition="Index &lt; Num">
              <Item>Index</Item>
              <Exec>++Index</Exec>
            </Loop>
          </CustomListItems>
        </Expand>
      </Synthetic>
    </Expand>
  </Type>


  <!-- PlainPropsInternalBuild.h -->

  <Type Name="PlainProps::FBuiltStruct" Priority="MediumHigh">
    <DisplayString>{(PlainProps::DbgVis::FBuiltStruct*)this}</DisplayString>
    <Expand HideRawView="1">
      <ExpandedItem>(PlainProps::DbgVis::FBuiltStruct*)this</ExpandedItem>
      <Item Name="[Raw View]">(PlainProps::DbgVis::FBuiltStruct*)this,!</Item>
    </Expand>
  </Type>

  <Type Name="PlainProps::FBuiltRange" Priority="MediumHigh">
    <DisplayString>{(PlainProps::DbgVis::FBuiltRange*)this}</DisplayString>
    <Expand HideRawView="1">
      <ExpandedItem>(PlainProps::DbgVis::FBuiltRange*)this</ExpandedItem>
      <Item Name="[Raw View]">(PlainProps::DbgVis::FBuiltRange*)this,!</Item>
    </Expand>
  </Type>
  
  <Type Name="PlainProps::FBuiltStruct">
    <AlternativeType Name="PlainProps::DbgVis::FBuiltStruct"/>
    <DisplayString>NumMembers={NumMembers}</DisplayString>
    <Expand>
      <IndexListItems>
        <Size>NumMembers</Size>
        <ValueNode>Members[$i]</ValueNode>
      </IndexListItems>
    </Expand>
  </Type>
  
  <Type Name="PlainProps::FBuiltRange">
    <AlternativeType Name="PlainProps::DbgVis::FBuiltRange"/>
    <Intrinsic Name="_AsRanges"  Expression="(PlainProps::DbgVis::FBuiltRange**)Data"/>
    <Intrinsic Name="_AsStructs" Expression="(PlainProps::DbgVis::FBuiltStruct**)Data"/>
    <Intrinsic Name="_AsBools"   Expression="(bool*)Data"/>
    <Intrinsic Name="_AsU8s"     Expression="(uint8*)Data"/>
    <Intrinsic Name="_AsU16s"    Expression="(uint16*)Data"/>
    <Intrinsic Name="_AsU32s"    Expression="(uint32*)Data"/>
    <Intrinsic Name="_AsU64s"    Expression="(uint64*)Data"/>
    <Intrinsic Name="_AsS8s"     Expression="(int8*)Data"/>
    <Intrinsic Name="_AsS16s"    Expression="(int16*)Data"/>
    <Intrinsic Name="_AsS32s"    Expression="(int32*)Data"/>
    <Intrinsic Name="_AsS64s"    Expression="(int64*)Data"/>
    <Intrinsic Name="_AsFloats"  Expression="(float*)Data"/>
    <Intrinsic Name="_AsDoubles" Expression="(double*)Data"/>
    <Intrinsic Name="_AsUtf8"    Expression="(char8_t*)Data" Optional="1"/>
    <Intrinsic Name="_AsUtf8"    Expression="(char*)Data"/>
    <Intrinsic Name="_AsUtf16"   Expression="(char16_t*)Data" Optional="1"/>
    <Intrinsic Name="_AsUtf16"   Expression="(UTF16CHAR*)Data"/>
    <Intrinsic Name="_AsUtf32"   Expression="(char32_t*)Data" Optional="1"/>
    <Intrinsic Name="_AsUtf32"   Expression="(UTF32CHAR*)Data"/>

    <DisplayString IncludeView="utf8" Optional="1">{(char8_t*)Data,[Num]}</DisplayString>
    <DisplayString IncludeView="utf8">{(char*)Data,[Num]}</DisplayString>
    <DisplayString IncludeView="utf16" Optional="1">{(char16_t*)Data,[Num]}</DisplayString>
    <DisplayString IncludeView="utf32" Optional="1">{(char32_t*)Data,[Num]}</DisplayString>
    <DisplayString>NumItems={Num}, Data={(uintptr_t)Data,x}</DisplayString>

    <StringView IncludeView="utf8" Optional="1">(char8_t*)Data,[Num]</StringView>
    <StringView IncludeView="utf8">(char*)Data,[Num]</StringView>
    <StringView IncludeView="utf16" Optional="1">(char16_t*)Data,[Num]</StringView>
    <StringView IncludeView="utf32" Optional="1">(char32_t*)Data,[Num]</StringView>
    <StringView>""</StringView>

    <Expand>
      <ExpandedItem IncludeView="struct">(PlainProps::DbgVis::FBuiltStruct**)Data,[Num]</ExpandedItem>
      <ExpandedItem IncludeView="range" >(PlainProps::DbgVis::FBuiltRange**)Data,[Num]</ExpandedItem> 
      <ExpandedItem IncludeView="bool"  >(bool*)Data,[Num]</ExpandedItem>
      <ExpandedItem IncludeView="u8"    >(uint8*)Data,[Num]</ExpandedItem>
      <ExpandedItem IncludeView="u16"   >(uint16*)Data,[Num]</ExpandedItem>
      <ExpandedItem IncludeView="u32"   >(uint32*)Data,[Num]</ExpandedItem>
      <ExpandedItem IncludeView="u64"   >(uint64*)Data,[Num]</ExpandedItem>
      <ExpandedItem IncludeView="s8"    >(int8*)Data,[Num]</ExpandedItem>
      <ExpandedItem IncludeView="s16"   >(int16*)Data,[Num]</ExpandedItem>
      <ExpandedItem IncludeView="s32"   >(int32*)Data,[Num]</ExpandedItem>
      <ExpandedItem IncludeView="s64"   >(int64*)Data,[Num]</ExpandedItem>
      <ExpandedItem IncludeView="hex8"  >(uint8*)Data,[Num]x</ExpandedItem>
      <ExpandedItem IncludeView="hex16" >(uint16*)Data,[Num]x</ExpandedItem>
      <ExpandedItem IncludeView="hex32" >(uint32*)Data,[Num]x</ExpandedItem>
      <ExpandedItem IncludeView="hex64" >(uint64*)Data,[Num]x</ExpandedItem>
      <ExpandedItem IncludeView="float" >(float*)Data,[Num]</ExpandedItem>
      <ExpandedItem IncludeView="double">(double*)Data,[Num]</ExpandedItem>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[Structs]">_AsStructs(),[Num]</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[Ranges]" >_AsRanges(),[Num]</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[Bools]"  >_AsBools(),[Num]</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[U8s]"    >_AsU8s(),[Num]</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[U16s]"   >_AsU16s(),[Num]</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[U32s]"   >_AsU32s(),[Num]</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[U64s]"   >_AsU64s(),[Num]</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[S8s]"    >_AsS8s(),[Num]</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[S16s]"   >_AsS16s(),[Num]</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[S32s]"   >_AsS32s(),[Num]</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[S64s]"   >_AsS64s(),[Num]</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[Hex8s]"  >_AsU8s(),[Num]x</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[Hex16s]" >_AsU16s(),[Num]x</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[Hex32s]" >_AsU32s(),[Num]x</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[Hex64s]" >_AsU64s(),[Num]x</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[Floats]" >_AsFloats(),[Num]</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[Doubles]">_AsDoubles(),[Num]</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[Utf8]"   >_AsUtf8(),[Num]</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[Utf16]"  >_AsUtf16(),[Num]</Item>
      <Item ExcludeView="struct;range;bool;u8;u16;u32;u64;s8;s16;s32;s64;hex8;hex16;hex32;hex64;float;double;utf8;utf16;utf32" Name="[Utf32]"  >_AsUtf32(),[Num]</Item>
    </Expand>
  </Type>


  <!-- PlainPropsInternalParse.h -->

  <Type Name="PlainProps::FToken">
    <DisplayString>{Token,en}, Depth={Depth}, Value={Str,[Len]s8}</DisplayString>
    <StringView Condition="Len &gt; 0">Str,[Len]s8</StringView>
    <Expand>
      <Item Name="Token">Token</Item>
      <Item Name="Depth">Depth</Item>
      <Item Name="[Value]">Str,[Len]s8</Item>
    </Expand>
  </Type>
  
  <Type Name="PlainProps::FParsedMember">
    <DisplayString>{Kind,en} = {First} : {Second}</DisplayString>
  </Type>

  <Type Name="PlainProps::FYamlReader::FLineInfo">
    <DisplayString>{View}</DisplayString>
  </Type>

</AutoVisualizer>
