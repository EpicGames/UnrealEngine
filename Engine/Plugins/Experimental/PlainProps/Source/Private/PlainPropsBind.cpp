// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsBind.h"
#include "PlainPropsDeclare.h"
#include "PlainPropsDiff.h"
#include "PlainPropsIndex.h"
#include "PlainPropsInternalBind.h"
#include "PlainPropsInternalBuild.h"
#include "PlainPropsInternalFormat.h"
#include "PlainPropsInternalRead.h"
#include "PlainPropsSpecify.h"
#include "Templates/RefCounting.h"

namespace PlainProps
{

static_assert(sizeof(ELeafBindType) == 1);
static_assert((uint8)ELeafType::Bool		== (uint8)ELeafBindType::Bool);
static_assert((uint8)ELeafType::IntS		== (uint8)ELeafBindType::IntS);
static_assert((uint8)ELeafType::IntU		== (uint8)ELeafBindType::IntU);
static_assert((uint8)ELeafType::Float		== (uint8)ELeafBindType::Float);
static_assert((uint8)ELeafType::Hex			== (uint8)ELeafBindType::Hex);
static_assert((uint8)ELeafType::Enum		== (uint8)ELeafBindType::Enum);
static_assert((uint8)ELeafType::Unicode		== (uint8)ELeafBindType::Unicode);

////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FSchemaBinding::CalculateSize() const
{
	uint32 Out = sizeof(FSchemaBinding) + (NumMembers + NumInnerRanges) * sizeof(FMemberBindType);
	Out = Align(Out + NumMembers * sizeof(uint32), sizeof(uint32));
	Out = Align(Out + NumInnerIds * sizeof(FInnerId), sizeof(FInnerId));
	Out = Align(Out + NumInnerRanges * sizeof(FRangeBinding), sizeof(FRangeBinding));
	return Out;
}

////////////////////////////////////////////////////////////////////////////////////////////////

bool ICustomBinding::DiffCustom(const void* StructA, const void* StructB, FDiffContext& Ctx) const
{
	// Use faster non-tracking diff used by delta saving by default
	return DiffCustom(StructA, StructB, static_cast<const FBindContext&>(Ctx));
}

////////////////////////////////////////////////////////////////////////////////////////////////

FMemberVisitor::FMemberVisitor(const FSchemaBinding& InSchema)
: Schema(InSchema)
, NumMembers(InSchema.NumMembers)
{}

EMemberKind FMemberVisitor::PeekKind() const
{
	return PeekType().GetKind();
}

FMemberBindType	FMemberVisitor::PeekType() const
{
	check(HasMore());
	return Schema.Members[MemberIdx];
}

uint32 FMemberVisitor::PeekOffset() const
{
	check(HasMore());
	return Schema.GetOffsets()[MemberIdx];
}

uint64 FMemberVisitor::GrabMemberOffset()
{
	return Schema.GetOffsets()[MemberIdx++];
}

FLeafMemberBinding FMemberVisitor::GrabLeaf()
{
	FUnpackedLeafBindType Leaf = PeekType().AsLeaf();
	FOptionalEnumId Enum = Leaf.Type == ELeafBindType::Enum ? ToOptional(GrabEnumSchema()) : NoId;
	uint64 Offset = GrabMemberOffset();

	return {Leaf, Enum, Offset};
}

FStructMemberBinding FMemberVisitor::GrabStruct()
{
	checkf(!PeekType().AsStruct().IsDynamic, TEXT("Bound structs can't be dynamic"));
	return { PeekType().AsStruct(), GrabInnerId().AsStructBindId(), GrabMemberOffset() };
}

static bool HasSchema(FMemberBindType Type)
{
	return Type.IsStruct() || Type.AsLeaf().Bind.Type == ELeafBindType::Enum;
}

TConstArrayView<FMemberBindType> FMemberVisitor::GrabInnerTypes()
{
	const int32 Idx = InnerRangeIdx;
	const TConstArrayView<FMemberBindType> All(Schema.GetInnerRangeTypes(), Schema.NumInnerRanges);
	while (All[InnerRangeIdx++].IsRange()) {}
	return All.Slice(Idx, InnerRangeIdx - Idx);
}

FRangeMemberBinding FMemberVisitor::GrabRange()
{
	ESizeType MaxSize = PeekType().AsRange().MaxSize;
	const FRangeBinding* RangeBindings = Schema.GetRangeBindings() + InnerRangeIdx;
	check(MaxSize == RangeBindings[0].GetSizeType());
	FMemberBindTypeRange InnerTypes = GrabInnerTypes();
	FOptionalInnerId InnermostId = HasSchema(InnerTypes.Last()) ? ToOptional(GrabInnerId()) : NoId; 
	uint64 Offset = GrabMemberOffset();
		
	return { &InnerTypes[0], RangeBindings, static_cast<uint16>(InnerTypes.Num()), InnermostId, Offset};
}

void FMemberVisitor::SkipMember()
{
	FMemberBindType Type = PeekType();
	if (Type.IsRange())
	{
		FMemberBindTypeRange InnerTypes = GrabInnerTypes();
		InnerIdIdx += HasSchema(InnerTypes.Last());
	}
	else
	{
		InnerIdIdx += HasSchema(Type);
	}
	
	++MemberIdx;
}

FBindId FMemberVisitor::GrabSuper()
{
	check(MemberIdx == 0);
	checkSlow(Schema.Members[0].AsByte() == SuperStructType.AsByte());
	MemberIdx = 1;
	InnerIdIdx = 1;
	return Schema.GetInnerIds()[0].AsStructBindId();
}

FInnerId FMemberVisitor::GrabInnerId()
{
	check(InnerIdIdx < Schema.NumInnerIds);
	return Schema.GetInnerIds()[InnerIdIdx++];
}

////////////////////////////////////////////////////////////////////////////////////////////////

FRangeBinding::FRangeBinding(const IItemRangeBinding& Binding, ESizeType SizeType)
: Handle(uint64(&Binding) | uint8(SizeType))
{
	check(&Binding == &AsItemBinding());
	check(SizeType == GetSizeType());
}

FRangeBinding::FRangeBinding(const ILeafRangeBinding& Binding, ESizeType SizeType)
: Handle(uint64(&Binding) | uint8(SizeType) | LeafMask)
{
	check(&Binding == &AsLeafBinding());
	check(SizeType == GetSizeType());
}

////////////////////////////////////////////////////////////////////////////////////////////////

FRangeMemberBinding GetInnerRange(FRangeMemberBinding In)
{
	check(In.NumRanges > 1);
	check(In.InnerTypes[0].IsRange());
	return { In.InnerTypes + 1, In.RangeBindings + 1, static_cast<uint16>(In.NumRanges - 1), In.InnermostId };
}

////////////////////////////////////////////////////////////////////////////////////////////////

void* FLeafRangeAllocator::Allocate(uint64 Num, SIZE_T LeafSize)
{
	check(!Range);
	Range = FBuiltRange::Create(Scratch, Num, LeafSize);
	return Range->Data;
}

////////////////////////////////////////////////////////////////////////////////////////////////

inline bool Cull(FIdWindow Window, uint32 Id)
{
	return Id - Window.Min < Window.Num;
}

static void UpdateCulling(FIdWindow& Window, uint32 Id)
{
	if (Window.Num == 0)
	{
		Window = { Id, 1 };
	}
	else if (Id < Window.Min)
	{
		Window = { Id, Window.Num + Window.Min - Id };
	}
	else 
	{
		Window.Num = FMath::Max(1u + Id - Window.Min, Window.Num);
	}
}

struct FCustomBindingHandle
{
	ICustomBinding**				Value = nullptr;
	uint32							Max = 0;

	explicit						operator bool()		{ return !!Value; }
	ICustomBinding*&				GetBinding()		{ return Value[0]; }
	FInnersHandle&					GetInners()			{ return reinterpret_cast<FInnersHandle&>(Value[Max]); }
	FStructDeclarationPtr&			GetDeclaration()	{ return reinterpret_cast<FStructDeclarationPtr&>(Value[Max*2]); }

	void							Free()				{ GetDeclaration().~FStructDeclarationPtr(); }
};

FCustomBindingMap::FCustomBindingMap(FDebugIds Dbg)
: Debug(Dbg)
{}

FCustomBindingMap::~FCustomBindingMap()
{
	for (TSparseSet<FBindId>::TConstIterator It(Keys); It; ++It)
	{
		int32 Idx = It.GetId().AsInteger();
		FCustomBindingHandle Handle = {Values + Idx, MaxValues};
		Handle.Free();
	}
	FMemory::Free(Values);
}

void FCustomBindingMap::Bind(FBindId Id, ICustomBinding& Binding, FStructDeclarationPtr&& Declaration, FInnersHandle LoweredInners)
{
	// Add key
	bool bExists = false;
	int32 SetIdx = Keys.Add(Id, &bExists).AsInteger();
	checkf(!bExists, TEXT("'%s' already bound"), *Debug.Print(Id));

	// Grow values if needed
	uint32 MaxKeys = static_cast<uint32>(Keys.GetMaxIndex());
	if (MaxValues < MaxKeys)
	{
		constexpr SIZE_T ValueSize = sizeof(ICustomBinding*) + sizeof(FStructDeclarationPtr) + sizeof(FInnersHandle);

		FCustomBindingHandle Old = { Values, MaxValues };
		MaxValues = FMath::RoundUpToPowerOfTwo(FMath::Max(MaxKeys, 4u));
		Values = static_cast<ICustomBinding**>(FMemory::MallocZeroed(MaxValues * ValueSize));

		if (Old)
		{
			FCustomBindingHandle New = { Values, MaxValues };
			FMemory::Memcpy(&New.GetBinding(),		&Old.GetBinding(),		Old.Max * sizeof(ICustomBinding*));
			FMemory::Memcpy(&New.GetDeclaration(),	&Old.GetDeclaration(),	Old.Max * sizeof(FStructDeclaration*));
			FMemory::Memcpy(&New.GetInners(),		&Old.GetInners(),		Old.Max * sizeof(FInnersHandle));
			FMemory::Free(Old.Value);
		}
	}

	// Add value
	FCustomBindingHandle Value = {Values + SetIdx, MaxValues};
	Value.GetBinding() = &Binding;
	Value.GetInners() = LoweredInners;
	new (&Value.GetDeclaration()) FStructDeclarationPtr(MoveTemp(Declaration));
	check(LoweredInners.Idx == Value.GetInners().Idx);

	UpdateCulling(/* in-out */ Window, Id.Idx);
}

inline FCustomBindingHandle FCustomBindingMap::Find(FBindId Id) const
{
	if (Cull(Window, Id.Idx))
	{
		if (FSetElementId Idx = Keys.FindId(Id); Idx.IsValidId())
		{
			return {Values + Idx.AsInteger(), MaxValues};
		}
	}
	return {};
}

void FCustomBindingMap::Drop(FBindId Id)
{
	if (FSetElementId Idx = Keys.FindId(Id); Idx.IsValidId())
	{
		FCustomBindingHandle Handle = {Values + Idx.AsInteger(), MaxValues};
		Handle.Free();
	}	

	int32 NumRemoved = Keys.Remove(Id);
	checkf(NumRemoved == 1, TEXT("'%s' unbound"), *Debug.Print(Id));
}

////////////////////////////////////////////////////////////////////////////////////////////////

static FInnersHandle StoreInners(TArray<FInnerStruct>& All, TConstArrayView<FInnerStruct> In)
{
	if (uint32 Num = static_cast<uint32>(In.Num()))
	{
		// Optimizable: Inline encode if Num == 1
		// Note: Grows indefinitely, might needs improvement if type erasure becomes common
		FInnersHandle Out = { Num, static_cast<uint32>(All.Num()) };
		All.Append(In);
		return Out;
	}

	return {};
}

static TConstArrayView<FInnerStruct> FetchInners(const TArray<FInnerStruct>& All, FInnersHandle In)
{
	if (int32 Num = static_cast<int32>(In.Num))
	{
		return { &All[In.Idx], Num };
	}
	return {};
}

static bool operator==(FInnerStruct A, FInnerStruct B) { return A.Name == B.Name && A.Id == B.Id; }

////////////////////////////////////////////////////////////////////////////////////////////////

static void ValidateStructSpec(const FStructSpec& Spec, const FDebugIds& Ids)
{
#if DO_CHECK
	check(Spec.MemberNames.Num() == Spec.MemberTypes.Num());
	for (int32 Idx = 0, Num = Spec.MemberNames.Num(); Idx < Num; ++Idx)
	{
		checkf(Spec.MemberTypes[Idx], TEXT("The declared member '%s' at member index '%d' in '%s' has not been specified"),
			*Ids.Print(Spec.MemberNames[Idx]), Idx, *Ids.Print(Spec.Id));
	}
#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////

FCustomBindings::FCustomBindings(TArray<FInnerStruct>& Inners, FDebugIds Dbg) : Map(Dbg), BottomInners(Inners) {}
FCustomBindings::FCustomBindings(const FCustomBindings* Under) : Map(Under->Map.Debug), BottomInners(Under->BottomInners) {}
FCustomBindings::~FCustomBindings() {}

void FCustomBindings::BindStruct(FBindId Id, ICustomBinding& Binding, FStructDeclarationPtr&& Declaration, TConstArrayView<FInnerStruct> LoweredInners)
{
	FInnersHandle StoredInners = StoreInners(/* out */ BottomInners, LoweredInners);
	check(Algo::Compare(LoweredInners, FetchInners(BottomInners, StoredInners)));
	Map.Bind(Id, Binding, MoveTemp(Declaration), StoredInners);
}

void FCustomBindings::BindStruct(FBindId Id, ICustomBinding& Binding, const FStructSpec& Spec, TConstArrayView<FInnerStruct> LoweredInners)
{
	ValidateStructSpec(Spec, Map.Debug);
	return BindStruct(Id, Binding, Declare(Spec), LoweredInners);
}

const ICustomBinding* FCustomBindings::FindStruct(FBindId Id) const
{
	FCustomBindingHandle Handle = Find(Id);
	return Handle ? Handle.GetBinding() : nullptr;
}

const ICustomBinding* FCustomBindings::FindStruct(FBindId Id, TConstArrayView<FInnerStruct>& OutLoweredInners) const
{
	if (FCustomBindingHandle Handle = Find(Id))
	{
		OutLoweredInners = FetchInners(BottomInners, Handle.GetInners());
		return Handle.GetBinding();
	}
	return nullptr;
}

ICustomBinding* FCustomBindings::FindStructToSave(FBindId Id, const FStructDeclaration*& OutDeclaration) const
{
	if (FCustomBindingHandle Handle = Find(Id))
	{
		OutDeclaration = Handle.GetDeclaration();
		return Handle.GetBinding();
	}

	OutDeclaration = nullptr;
	return nullptr;
}

const FStructDeclaration* FCustomBindings::FindDeclaration(FBindId Id) const
{
	FCustomBindingHandle Handle = Find(Id);
	return Handle ? Handle.GetDeclaration() : nullptr;
}

void FCustomBindings::DropStruct(FBindId Id)
{
	Map.Drop(Id);
}

FCustomBindingHandle FCustomBindingsBottom::Find(FBindId Id) const
{
	return Map.Find(Id);
}

FCustomBindingHandle FCustomBindingsOverlay::Find(FBindId Id) const
{
	FCustomBindingHandle Handle = Map.Find(Id);
	return Handle ? Handle : Underlay.Find(Id);
}

////////////////////////////////////////////////////////////////////////////////////////////////

inline FType IndexInnermost(FIdIndexerBase& Ids, FMemberBindType Type, FOptionalInnerId Id)
{
	if (Type.IsStruct())
	{
		return Ids.Resolve(Id.Get().AsStruct());
	}

	FUnpackedLeafType Leaf = ToLeafType(Type.AsLeaf());
	if (Leaf.Type == ELeafType::Enum)
	{
		return Ids.Resolve(Id.Get().AsEnum());
	}

	return Ids.MakeLeafParameter(Leaf);
}

FBothType FMemberBinding::IndexParameterName(FIdIndexerBase& Ids) const
{
	FType BindType = IndexInnermost(Ids, InnermostType, InnermostId);
	FType DeclType = BindType;
	for (FRangeBinding Range : ReverseIterate(RangeBindings))
	{
		FType SizeType = Ids.MakeRangeParameter(Range.GetSizeType());
		FType RangeBindName = {NoId, FTypenameId(Range.GetBindName())};
		DeclType = Ids.MakeAnonymousParametricType({DeclType, SizeType});
		BindType = Ids.MakeParametricType(RangeBindName, {BindType, SizeType});
	}
	
	return {BindType, DeclType};
}

FSchemaBindings::FSchemaBindings(FDebugIds In) : Debug(In) {}
FSchemaBindings::~FSchemaBindings() {}

static uint16 CountInnerIds(TConstArrayView<FMemberBinding> Members)
{
	uint32 Out = 0;
	for (const FMemberBinding& Member : Members)
	{
		Out += !!Member.InnermostId;
	}
	return IntCastChecked<uint16>(Out);
}

static uint16 CountRanges(TConstArrayView<FMemberBinding> Members)
{
	int32 Out = 0;
	for (const FMemberBinding& Member : Members)
	{
		Out += Member.RangeBindings.Num();
	}
	return IntCastChecked<uint16>(Out);
}

struct FMemberBinder : FMemberBinderBase
{
	FMemberBinder(FSchemaBinding& InSchema)
	: FMemberBinderBase(InSchema)
	, InnerIdIt(const_cast<FInnerId*>(InSchema.GetInnerIds()))
	{}
	
	~FMemberBinder()
	{
		check(Align(InnerIdIt, alignof(FRangeBinding)) == (const void*)Schema.GetRangeBindings() || Schema.NumInnerRanges == 0);
	}

	void AddInnerId(FInnerId InnermostId)
	{
		*InnerIdIt++ = InnermostId;
	}

	FInnerId* InnerIdIt;
};

static void IncRef(FStructDeclarationPtr& Ptr)
{
	Ptr->AddRef();
}

static void DecRef(FStructDeclarationPtr& Ptr)
{
	if (Ptr->Release())
	{
		// Hack, recreate new TRefCountPtr to null out internal pointer in lieu of a Release function
		new (&Ptr) FStructDeclarationPtr();
	}
}

static void DeclareOrValidate(FStructDeclarationPtr& Out, FStructSpec Info, TConstArrayView<FInnerId> InnerIds, const FSchemaBinding& Binding)
{
	const uint16 Num = IntCastChecked<uint16>(Info.MemberNames.Num());

	check(Info.Id == Binding.DeclId);
	check(Num == Binding.NumMembers);
	check(!!Info.Super == Binding.HasSuper());

	if (Out)
	{
		check(Out->Id == Info.Id);
		check(Out->Super == Info.Super);
		check(Out->Version == Info.Version);
		check(Out->Occupancy == Info.Occupancy);
		check(Algo::Compare(Out->GetMemberOrder(), Info.MemberNames));
		check(Out->NumInnerIds == Binding.NumInnerIds);
		check(Out->NumInnerRanges == Binding.NumInnerRanges);
		// Todo: check Out->GetTypes() matches Binding.Members
		IncRef(Out);
		return;
	}

	// Make header and allocate
	FStructDeclaration Header = { 1, Info.Id, Info.Super, Info.Version, Num, Binding.NumInnerRanges, Binding.NumInnerIds, Info.Occupancy };
	FStructDeclaration* Ptr = new (FMemory::MallocZeroed(Header.CalculateSize())) FStructDeclaration;
	FMemory::Memcpy(Ptr, &Header, sizeof(Header));

	// Copy footer
	FMemory::Memcpy(Ptr->MemberNames, Info.MemberNames.GetData(), Info.MemberNames.NumBytes());
	FMemory::Memcpy(const_cast<FInnerId*>(Ptr->GetInnerIds()), InnerIds.GetData(), InnerIds.NumBytes());
	FMemberType* TypeIt = const_cast<FMemberType*>(Ptr->GetTypes());
	checkSlow(TypeIt + Num == Ptr->GetInnerRangeTypes() && Binding.Members + Num == Binding.GetInnerRangeTypes()); // Loop assumes inner range types follow member types
	for (FMemberBindType BindType : MakeArrayView(Binding.Members, Binding.NumMembers + Binding.NumInnerRanges))
	{
		*TypeIt++ = ToMemberType(BindType);
	}

	// Make smart pointer
	new (&Out) FStructDeclarationPtr(Ptr, /* add ref */ false);
}

FSchemaBinding* FSchemaBindings::Bind(FBindId Id, TConstArrayView<FMemberBinding> Members, FDeclId DeclId) 
{
	// Make header, allocate and copy header
	FSchemaBinding Header = { DeclId, IntCastChecked<uint16>(Members.Num()), CountInnerIds(Members), CountRanges(Members) };
	FSchemaBinding* Out = new (FMemory::MallocZeroed(Header.CalculateSize())) FSchemaBinding {Header};

	// Write footer
	FMemberBinder Footer(*Out);
	for (const FMemberBinding& Member : Members)
	{
		TConstArrayView<FRangeBinding> Ranges = Member.RangeBindings;
		if (Ranges.IsEmpty())
		{
			Footer.AddMember(Member.InnermostType, IntCastChecked<uint32>(Member.Offset));
		}
		else
		{
			Footer.AddRange(Ranges, Member.InnermostType, IntCastChecked<uint32>(Member.Offset));
		}

		if (Member.InnermostId)
		{
			Footer.AddInnerId(Member.InnermostId.Get());
		}
	}

	// Bind
	if (Id.Idx >= static_cast<uint32>(Bindings.Num()))
	{
		Bindings.SetNum(Id.Idx + 1);
	}
	checkf(!Bindings[Id.Idx], TEXT("'%s' already bound"), *Debug.Print(Id));
	Bindings[Id.Idx].Reset(Out);

	return Out;
}

FStructDeclarationPtr& FSchemaBindings::At(FDeclId Id)
{
	if (Id.Idx >= static_cast<uint32>(Declarations.Num()))
	{
		Declarations.SetNum(Id.Idx + 1);
	}
	return Declarations[Id.Idx];
}

void FSchemaBindings::BindStruct(FBindId Id, TConstArrayView<FMemberBinding> Members, FStructSpec Spec)
{
	FSchemaBinding* Schema = Bind(Id, Members, Spec.Id);

	// Extract declared struct ids, Binding::GetInnerIds() can contain unlowered FBindIds
	TArray<FInnerId, TInlineAllocator<16>> InnerIds;
	for (FMemberSpecView Member : Spec.MemberTypes)
	{
		if (Member.InnermostId)
		{
			InnerIds.Emplace(Member.InnermostId.Get());
		}
	}
	check(InnerIds.Num() == Schema->NumInnerIds);
	DeclareOrValidate(/* in-out */ At(Spec.Id), Spec, InnerIds, *Schema);
}

void FSchemaBindings::BindStruct(FBindId Id, TConstArrayView<FMemberBinding> Members, FStructDeclarationPtr&& InDecl)
{
	FSchemaBinding* Schema = Bind(Id, Members, InDecl->Id);
	
	FStructDeclarationPtr& OutDecl = At(InDecl->Id);
	check(OutDecl == nullptr || OutDecl == InDecl); // Require exact pointer equality for now
	OutDecl = InDecl;
}

const FSchemaBinding* FSchemaBindings::FindStruct(FBindId Id) const
{
	return Id.Idx < (uint32)Bindings.Num() ? Bindings[Id.Idx].Get() : nullptr;
}

const FSchemaBinding* FSchemaBindings::FindStruct(FBindId Id, const FStructDeclaration*& OutDeclaration) const
{
	if (const FSchemaBinding* Binding = FindStruct(Id))
	{
		OutDeclaration = Declarations[Binding->DeclId.Idx];
		return Binding;
	}
	OutDeclaration = nullptr;
	return nullptr;
}

const FSchemaBinding& FSchemaBindings::GetStruct(FBindId Id) const
{
	checkf(Id.Idx < (uint32)Bindings.Num() && Bindings[Id.Idx], TEXT("'%s' is unbound"), *Debug.Print(Id));
	return *Bindings[Id.Idx].Get();
}

const FSchemaBinding& FSchemaBindings::GetStruct(FBindId Id, const FStructDeclaration*& OutDecl) const
{
	const FSchemaBinding& Out = GetStruct(Id);
	OutDecl = Declarations[Out.DeclId.Idx];
	return Out;
}

const FStructDeclaration& FSchemaBindings::GetDeclaration(FBindId Id) const
{
	return *Declarations[GetStruct(Id).DeclId.Idx];
}

const FStructDeclaration* FSchemaBindings::FindDeclaration(FBindId Id) const
{
	if (const FSchemaBinding* Binding = FindStruct(Id))
	{
		return Declarations[Binding->DeclId.Idx];
	}
	return nullptr;
}

void FSchemaBindings::DropStruct(FBindId Id)
{
	checkf(Id.Idx < (uint32)Bindings.Num() && Bindings[Id.Idx], TEXT("'%s' is unbound"), *Debug.Print(Id));
	TUniquePtr<FSchemaBinding>& Binding = Bindings[Id.Idx];
	DecRef(Declarations[Binding->DeclId.Idx]);
	Binding.Reset();
}

FDeclId FSchemaBindings::Lower(FBindId Id) const
{
	return GetStruct(Id).DeclId;
}

//////////////////////////////////////////////////////////////////////////

const FStructDeclaration& FBindContext::GetDeclaration(FBindId Id) const
{
	FCustomBindingHandle Handle = Customs.Find(Id);
	return Handle ? *Handle.GetDeclaration() : Schemas.GetDeclaration(Id);
}

//////////////////////////////////////////////////////////////////////////

const FEnumDeclaration* FBindDeclarations::Find(FEnumId Id) const
{
	return Enums.Find(Id);
}

const FStructDeclaration* FBindDeclarations::Find(FStructId Id) const
{
	FCustomBindingHandle Handle = Customs.Find(FBindId(Id));
	return Handle ? &*Handle.GetDeclaration() : Schemas.FindDeclaration(FBindId(Id));
}

FDeclId FBindDeclarations::Lower(FBindId Id) const
{
	FCustomBindingHandle Handle = Customs.Find(Id);
	return Handle ? Handle.GetDeclaration()->Id : Schemas.Lower(Id);
}

//////////////////////////////////////////////////////////////////////////

TArray<FInnerId> IndexAllRuntimeIds(const FSchemaBatch& Schemas, FIdIndexerBase& Indexer)
{
	const uint8* Base = reinterpret_cast<const uint8*>(&Schemas);
	const uint32* Offsets = Schemas.SchemaOffsets;

	TArray<FInnerId> Out;
	Out.SetNumUninitialized(Schemas.NumSchemas);
	for (uint32 Idx = 0; Idx < Schemas.NumStructSchemas; ++Idx)
	{
		const FStructSchema& Schema = *reinterpret_cast<const FStructSchema*>(Base + Offsets[Idx]);
		Out[Idx] = FInnerId(Indexer.IndexDeclId(Schema.Type));
	}

	for (uint32 Idx = Schemas.NumStructSchemas; Idx < Schemas.NumSchemas; ++Idx)
	{
		const FEnumSchema& Schema = *reinterpret_cast<const FEnumSchema*>(Base + Offsets[Idx]);
		Out[Idx] = FInnerId(Indexer.IndexEnum(Schema.Type));
	}

	return Out;
}

TArray<FStructId> IndexStructRuntimeIds(const FSchemaBatch& Schemas, FIdIndexerBase& Indexer)
{
	const uint8* Base = reinterpret_cast<const uint8*>(&Schemas);
	const uint32* Offsets = Schemas.SchemaOffsets;

	TArray<FStructId> Out;
	Out.SetNumUninitialized(Schemas.NumStructSchemas);
	for (int32 Idx = 0; Idx < Out.Num(); ++Idx)
	{
		const FStructSchema& Schema = *reinterpret_cast<const FStructSchema*>(Base + Offsets[Idx]);
		Out[Idx] = Indexer.IndexDeclId(Schema.Type);
	}

	return Out;
}

uint32 FIdTranslatorBase::CalculateTranslationSize(int32 NumSavedNames, const FSchemaBatch& Batch)
{
	static_assert(sizeof(FNameId) == sizeof(FNestedScopeId));
	static_assert(sizeof(FNameId) == sizeof(FParametricTypeId));
	static_assert(sizeof(FNameId) == sizeof(FInnerId));
	return sizeof(FNameId) * (NumSavedNames + Batch.NumNestedScopes + Batch.NumParametricTypes + Batch.NumSchemas);
}

FFlatScopeId Translate(FFlatScopeId From, TConstArrayView<FNameId> ToNames)
{
	return { ToNames[From.Name.Idx] };
}

static void TranslateScopeIds(TArrayView<FNestedScopeId> Out, FIdIndexerBase& Indexer, TConstArrayView<FNameId> ToNames, TConstArrayView<FNestedScope> From)
{
	uint32 OutIdx = 0;
	for (FNestedScope Scope : From)
	{
		check(Scope.Outer.IsFlat() || Scope.Outer.AsNested().Idx < OutIdx);
		FScopeId Outer = Scope.Outer.IsFlat() ? FScopeId(Translate(Scope.Outer.AsFlat(), ToNames)) : FScopeId(Out[Scope.Outer.AsNested().Idx]);
		FFlatScopeId Inner = Translate(Scope.Inner, ToNames);
		Out[OutIdx++] = Indexer.NestFlatScope(Outer, Inner).AsNested();
	}
}

static void TranslateParametricTypeIds(TArrayView<FParametricTypeId> Out, FIdIndexerBase& Indexer, const FIdBinding& To, TConstArrayView<FParametricType> From, const FType* FromParameters)
{
	TArray<FType, TInlineAllocator<8>> Params;
	uint32 OutIdx = 0;
	for (FParametricType Parametric : From)
	{
		Params.Reset();
		for (FType FromParameter : MakeArrayView(FromParameters + Parametric.Parameters.Idx, Parametric.Parameters.NumParameters))
		{
			Params.Add(To.Remap(FromParameter));
		}
		Out[OutIdx++] = Indexer.MakeParametricTypeId(To.Remap(Parametric.Name), Params);
	}
}

static void TranslateSchemaIds(TArrayView<FInnerId> Out, FIdIndexerBase& Indexer, const FIdBinding& To, const FSchemaBatch& From)
{
	uint32 OutIdx = 0;
	for (const FStructSchema& FromSchema : GetStructSchemas(From))
	{
		FType ToType = To.Remap(FromSchema.Type);
		checkSlow(ToType.Name.NumParameters == FromSchema.Type.Name.NumParameters);
		Out[OutIdx++] = FInnerId(Indexer.IndexStruct(ToType));
	}
	
	for (const FEnumSchema& FromSchema : GetEnumSchemas(From))
	{
		FType ToType = To.Remap(FromSchema.Type);
		Out[OutIdx++] = FInnerId(Indexer.IndexEnum(ToType));
	}
}

FIdBinding FIdTranslatorBase::TranslateIds(FMutableMemoryView To, FIdIndexerBase& Indexer, TConstArrayView<FNameId> ToNames, const FSchemaBatch& From)
{
	TArrayView<FNestedScopeId> ToScopes(static_cast<FNestedScopeId*>(To.GetData()), From.NumNestedScopes);
	TArrayView<FParametricTypeId> ToParametricTypes(reinterpret_cast<FParametricTypeId*>(ToScopes.end()), From.NumParametricTypes);
	TArrayView<FInnerId> ToSchemas(reinterpret_cast<FInnerId*>(ToParametricTypes.end()), From.NumSchemas);
	FIdBinding Out = {ToNames, ToScopes, ToParametricTypes, ToSchemas};
	check(uintptr_t(To.GetDataEnd()) == uintptr_t(ToSchemas.end()));

	TranslateScopeIds(ToScopes, Indexer, ToNames, From.GetNestedScopes());
	TranslateParametricTypeIds(ToParametricTypes, Indexer, Out, From.GetParametricTypes(), From.GetFirstParameter());
	TranslateSchemaIds(ToSchemas, Indexer, Out, From);

	return Out;
}

//////////////////////////////////////////////////////////////////////////

template<class IdType>
void RemapAll(TArrayView<IdType> Ids, FIdBinding NewIds)
{
	for (IdType& Id : Ids)
	{
		Id = NewIds.Remap(Id);
	}
}

FSchemaBatch* CreateTranslatedSchemas(const FSchemaBatch& In, FIdBinding NewIds)
{
	const FMemoryView InSchemas = GetSchemaData(In);
	const uint32 Num = In.NumSchemas;
	const uint64 Size = sizeof(FSchemaBatch) + /* offsets */ sizeof(uint32) * Num + InSchemas.GetSize();

	// Allocate and copy header
	FSchemaBatch* Out = new (FMemory::Malloc(Size)) FSchemaBatch {In};
	Out->NumNestedScopes = 0;
	Out->NestedScopesOffset = 0;
	Out->NumParametricTypes = 0;

	// Initialize schema offsets
	const uint32 DroppedBytes = IntCastChecked<uint32>(uintptr_t(InSchemas.GetData()) - uintptr_t(In.SchemaOffsets + Num));
	for (uint32 Idx = 0; Idx < Num; ++Idx)
	{
		Out->SchemaOffsets[Idx] = In.SchemaOffsets[Idx] - DroppedBytes;
	}

	// Copy schemas and remap type ids if needed
	FMemory::Memcpy(reinterpret_cast<uint8*>(Out) + Out->GetSchemaOffsets()[0], InSchemas.GetData(), InSchemas.GetSize());
	for (FStructSchema& Schema : GetStructSchemas(*Out))
	{
		Schema.Type = NewIds.Remap(Schema.Type);
		RemapAll(Schema.EditMemberNames(), NewIds);
	}
	for (FEnumSchema& Schema : GetEnumSchemas(*Out))
	{
		Schema.Type = NewIds.Remap(Schema.Type);
		RemapAll(MakeArrayView(Schema.Footer, Schema.Num), NewIds);
	}

	return Out;
}

void DestroyTranslatedSchemas(const FSchemaBatch* Schemas)
{
	FMemory::Free(const_cast<FSchemaBatch*>(Schemas));
}

} // namespace PlainProps
