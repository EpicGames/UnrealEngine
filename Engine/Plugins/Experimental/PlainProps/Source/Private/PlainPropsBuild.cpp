// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsBuild.h"
#include "PlainPropsIndex.h"
#include "PlainPropsInternalBuild.h"
#include "PlainPropsInternalFormat.h"
#include "PlainPropsInternalRead.h"
#include "Serialization/VarInt.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Hash/xxhash.h"

namespace PlainProps
{

uint8* FScratchAllocator::AllocateInNewPage(SIZE_T Size, uint32 Alignment)
{
	const uint32 Padding = Alignment > MIN_ALIGNMENT ? Alignment - MIN_ALIGNMENT : 0u;

	if (Size + Padding >= DataSize || (DataSize - Size - Padding) < static_cast<uint32>(PageEnd - Cursor))
	{
		SIZE_T LonePageSize = offsetof(FPage, Data) + Size + Padding;

		FPage*& PrevPage = LastPage ? LastPage->PrevPage : LastPage;
		FPage Header = { PrevPage };
		PrevPage = new (FMemory::Malloc(LonePageSize)) FPage { Header };
		return Align(PrevPage->Data, Alignment);
	}
	
	FPage Header = { LastPage };
	LastPage = new (FMemory::Malloc(PageSize)) FPage { Header };
	uint8* Out = Align(LastPage->Data, Alignment);
	Cursor = Out + Size;
	PageEnd = LastPage->Data + DataSize;
	check(Cursor <= PageEnd);
	return Out;
}

void FScratchAllocator::Reset()
{
	for (FPage* It = LastPage; It; )
	{
		FPage* Prev = It->PrevPage;
		FMemory::Free(It);
		It = Prev;
	}

	Cursor = nullptr;
	PageEnd = nullptr;
	LastPage = nullptr;
}

FMemberType& FMemberSchema::EditInnermostType(FScratchAllocator& Scratch)
{
	if (NumInnerRanges > 1)
	{
		FMemberType* Clone = Scratch.AllocateArray<FMemberType>(NumInnerRanges);
		FMemory::Memcpy(Clone, NestedRangeTypes, NumInnerRanges * sizeof(FMemberType));
		NestedRangeTypes = Clone;
		return Clone[NumInnerRanges - 1];
	}

	return NumInnerRanges == 0 ? Type : InnerRangeType;
}

//////////////////////////////////////////////////////////////////////////

FBuiltRange* FBuiltRange::Create(FScratchAllocator& Scratch, uint64 NumItems, SIZE_T ItemSize)
{
	check(NumItems > 0);
	FBuiltRange* Out = new (Scratch.Allocate(sizeof(FBuiltRange) + NumItems * ItemSize, alignof(FBuiltRange))) FBuiltRange;
	Out->Num = NumItems;
	return Out;
}

FBuiltRange* FBuiltRange::Create(FScratchAllocator& Scratch, uint64 NumItems, SIZE_T ItemSize, const void* Items)
{
	FBuiltRange* Out = Create(Scratch, NumItems, ItemSize);
	FMemory::Memcpy(Out->Data, Items, NumItems * ItemSize);
	return Out;
}

//////////////////////////////////////////////////////////////////////////

FMemberSchema MakeNestedRangeSchema(FScratchAllocator& Scratch, ESizeType SizeType, FMemberSchema InnerRangeSchema)
{
	check(InnerRangeSchema.NumInnerRanges > 0);
	uint16 NumInnerRanges = IntCastChecked<uint16>(1 + InnerRangeSchema.NumInnerRanges);
	FMemberType* InnerRangeTypes = Scratch.AllocateArray<FMemberType>(NumInnerRanges);
	InnerRangeTypes[0] = InnerRangeSchema.Type;
	FMemory::Memcpy(&InnerRangeTypes[1], InnerRangeSchema.GetInnerRangeTypes().GetData(), InnerRangeSchema.NumInnerRanges * sizeof(FMemberType));

	return { FMemberType(SizeType), InnerRangeSchema.Type, NumInnerRanges, InnerRangeSchema.InnerSchema, InnerRangeTypes };
}

//////////////////////////////////////////////////////////////////////////

FBuiltRange* CloneLeaves(FScratchAllocator& Scratch, uint64 Num, const void* InData, SIZE_T LeafSize)
{
	return FBuiltRange::CreateIf(Scratch, Num, LeafSize, InData);
}

//	template<typename FloatType>
//	void NormalizeFloats(FloatType* Values, uint64 Num)
//	{
//		for (uint64 Idx = 0; Idx < Num; ++Idx)
//		{
//			// Reject NaN / INF and ignore negative zero for now
//			checkf(FMath::IsFinite(Values[Idx]), TEXT("Saving NaN or INF isn't supported"));
//		}
//	}
//
//	void NormalizeLeafRange(FUnpackedLeafType Leaf, FBuiltRange& Out)
//	{
//		check(Leaf.Type == ELeafType::Float);
//		if (Leaf.Width == ELeafWidth::B32)
//		{
//			NormalizeFloats(reinterpret_cast<float*>(Out.Data), Out.Num);
//		}
//		else
//		{
//			check(Leaf.Width == ELeafWidth::B64);
//			NormalizeFloats(reinterpret_cast<double*>(Out.Data), Out.Num);
//		}
//	}
//} // namespace Private

//////////////////////////////////////////////////////////////////////////

// Helper for FMemberBuilder::BuildAndReset() validations
struct FMemberSchemaIterator
{
	FMemberSchemaIterator(const FStructDeclaration& Declared)
		: Members(Declared.GetTypes(), Declared.NumMembers)
		, InnerRangeTypes(Declared.GetInnerRangeTypes(), Declared.NumInnerRanges)
		, InnerIds(Declared.GetInnerIds(), Declared.NumInnerIds)
	{}

	bool HasMore() const
	{
		return MemberIdx < Members.Num();
	}

	FMemberSchema GrabSchema()
	{
		check(HasMore());
		FMemberType Type = Members[MemberIdx++];
		FMemberType InnerRangeType = Type;
		FMemberType InnermostType = Type;
		TConstArrayView<FMemberType> InnerTypes;
		if (Type.IsRange())
		{
			InnerTypes = GrabInnerRangeTypes(InnerRangeTypes, /* in-out */ InnerRangeIdx);
			InnerRangeType = InnerTypes[0];
			InnermostType = InnerTypes.Last();
		}
		FOptionalInnerId InnerId = HasInnerId(InnermostType) ? ToOptional(InnerIds[InnerIdIdx++]) : NoId;
		uint16 NumInnerRanges = IntCastChecked<uint16>(InnerTypes.Num());
		const FMemberType* NestedRangeTypes = NumInnerRanges > 1 ? &InnerTypes[0] : nullptr;
		return { Type, InnerRangeType, NumInnerRanges, InnerId, NestedRangeTypes };
	}

private:
	bool HasInnerId(FMemberType Member) const
	{
		return Member.IsStruct() ? !Member.AsStruct().IsDynamic : IsEnum(Member);
	}

	TConstArrayView<FMemberType> Members;
	TConstArrayView<FMemberType> InnerRangeTypes;
	TConstArrayView<FInnerId> InnerIds;
	uint16 MemberIdx = 0;
	uint16 InnerRangeIdx = 0;
	uint16 InnerIdIdx = 0;
};

//////////////////////////////////////////////////////////////////////////

void FMemberBuilder::AddSuperStruct(FStructId SuperSchema, FBuiltStruct* SuperStruct)
{
	check(SuperStruct);
	check(Members.IsEmpty());
	Members.Emplace(FBuiltMember::MakeSuper(SuperSchema, SuperStruct));
	check(IsSuper(Members[0].Schema.Type));
}

void FMemberBuilder::BuildSuperStruct(FScratchAllocator& Scratch, const FStructDeclaration& Super, const FDebugIds& Debug)
{
	// If we need to support dense substructs, we need access to struct declaration here 
	// or create an empty super structs that we throw away in BuildAndReset.
	if (Members.IsEmpty() || (Members.Num() == 1 && IsSuper(Members[0].Schema.Type)))
	{
		return;
	}
	
	FBuiltStruct* OnlyMember = BuildAndReset(Scratch, Super, Debug);
	Members.Emplace(FBuiltMember::MakeSuper(Super.Id, MoveTemp(OnlyMember)));
	check(IsSuper(Members[0].Schema.Type));
}

FBuiltStruct* FMemberBuilder::BuildAndReset(FScratchAllocator& Scratch, const FStructDeclaration& Declared, const FDebugIds& Debug)
{
#if DO_CHECK
	if (Declared.Occupancy == EMemberPresence::RequireAll)
	{
		checkf(!Declared.Super, TEXT("Bug, dense substructs should fail DeclareStruct() check."));
		checkf(Members.Num() == Declared.NumMembers, TEXT("'%s' requires exactly %d members but %d were added."), *Debug.Print(Declared.Id), Declared.NumMembers, Members.Num());
	}
	
	auto MatchesDeclaredSchema = [](FMemberSchema Built, FMemberSchema Declared) -> bool
	{
		// For structs, match anything since declared struct schema may be dynamic or type-erased
		if (Built.Type.IsStruct() && Declared.Type.IsStruct())
		{
			return true;
		}
		// For struct ranges, match the range size and nested range sizes, but skip the innermost type and schema
		else if (Built.Type.IsRange() &&
			Built.GetInnerRangeTypes().Last().IsStruct() && Declared.GetInnerRangeTypes().Last().IsStruct())
		{
			return Built.Type == Declared.Type &&
				Algo::Compare(Built.GetInnerRangeTypes().LeftChop(1), Declared.GetInnerRangeTypes().LeftChop(1));
		}
		// Match all fields
		return Built.Type == Declared.Type &&
			Built.InnerSchema == Declared.InnerSchema &&
			Algo::Compare(Built.GetInnerRangeTypes(), Declared.GetInnerRangeTypes());
	};

	// Verify members were added in declared order and that they match the declared schemas.
	if (int32 Num = Members.Num())
	{
		const bool HasSuper = IsSuper(Members[0].Schema.Type);
		int32 OrderIdx = 0;
		TConstArrayView<FMemberId> Order = Declared.GetMemberOrder();
		int32 SkipSuper = Declared.Super && HasSuper;
		FMemberSchemaIterator DeclaredIt(Declared);
		for (FBuiltMember* It = Members.GetData() + SkipSuper, *End = Members.GetData() + Num; It != End; ++It)
		{
			FBuiltMember& Member = *It;
			for (; OrderIdx < Order.Num() && Order[OrderIdx] != Member.Name; ++OrderIdx)
			{
				(void)DeclaredIt.GrabSchema();
			}
			checkf(OrderIdx < Order.Num(),
				TEXT("Member '%s' in '%s' %s."),
				*Debug.Print(Member.Name), *Debug.Print(Declared.Id),
				Order.Contains(Member.Name) ? TEXT("appeared in non-declared order") : TEXT("is undeclared"));

			const FMemberSchema DeclaredSchema = DeclaredIt.GrabSchema();
			checkf(MatchesDeclaredSchema(Member.Schema, DeclaredSchema),
				TEXT("Member '%s' in '%s' is saved as '%s' but the declared schema is '%s'."),
				*Debug.Print(Member.Name), *Debug.Print(Declared.Id),
				*Debug.Print(Member.Schema), *Debug.Print(DeclaredSchema));
			++OrderIdx;
		}
	}
#endif

	uint32 Num = static_cast<uint32>(Members.Num());
	SIZE_T NumBytes = sizeof(FBuiltStruct) + Num * sizeof(FBuiltMember);
	FBuiltStruct* Out = reinterpret_cast<FBuiltStruct*>(Scratch.Allocate(NumBytes, alignof(FBuiltStruct)));
	Out->NumMembers = IntCastChecked<uint16>(Num);
	FMemory::Memcpy(Out->Members, Members.GetData(), Members.NumBytes());

	Members.Reset();

	return Out;
}

//////////////////////////////////////////////////////////////////////////

FBuiltStruct* FDenseMemberBuilder::BuildHomo(const FStructDeclaration& Declaration, FMemberType Leaf, TConstArrayView<FBuiltValue> Values) const
{
	check(Declaration.NumMembers == Values.Num());

	FMemberSchema Schema = {Leaf, Leaf};
	const uint32 Num = static_cast<uint32>(Values.Num());
	SIZE_T NumBytes = sizeof(FBuiltStruct) + Num * sizeof(FBuiltMember);
	FBuiltStruct* Out = reinterpret_cast<FBuiltStruct*>(Scratch.Allocate(NumBytes, alignof(FBuiltStruct)));
	Out->NumMembers = static_cast<uint16>(Num);

	const FMemberId* Names = Declaration.MemberNames; 
	for (uint32 Idx = 0; Idx < Num; ++Idx)
	{
		new (Out->Members + Idx) FBuiltMember(Names[Idx], Schema, Values[Idx]);
	}

	return Out;
}

//////////////////////////////////////////////////////////////////////////

template<typename IntType, typename FloatType>
inline IntType CheckFiniteBitCast(FloatType Value)
{
	// Todo: Decide if we should accept, reject or sanitize NaN / INF / -0
	//checkf(FMath::IsFinite(Value), TEXT("Saving NaN or INF isn't supported"));
	return BitCast<IntType>(Value);
}

uint64 ValueCast(float Value)
{
	return CheckFiniteBitCast<uint32>(Value);
}

uint64 ValueCast(double Value)
{ 
	return CheckFiniteBitCast<uint64>(Value);
}

//////////////////////////////////////////////////////////////////////////

template<typename InnerIdType>
static FMemberSchema MakeMemberSchema(FMemberType Type, InnerIdType InnerSchema)
{
	return {Type, Type, 0, FOptionalInnerId(InnerSchema), nullptr};
}

FBuiltMember::FBuiltMember(FMemberId Name, FUnpackedLeafType Leaf, FOptionalEnumId Enum, uint64 Value)
: FBuiltMember(Name, MakeMemberSchema(Leaf.Pack(), Enum), { .Leaf = Value})
{}

FBuiltMember::FBuiltMember(FMemberId Name, FTypedRange Range)
: FBuiltMember(Name, MoveTemp(Range.Schema), { .Range = Range.Values })
{}

FBuiltMember::FBuiltMember(FMemberId Name, FStructId Id, FBuiltStruct* Value)
: FBuiltMember(Name, MakeMemberSchema(DefaultStructType, FInnerId(Id)), { .Struct = Value })
{}

FBuiltMember FBuiltMember::MakeSuper(FStructId Id, FBuiltStruct* Value)
{
	return FBuiltMember(NoId, MakeMemberSchema(SuperStructType, FInnerId(Id)), { .Struct = Value });
}

//////////////////////////////////////////////////////////////////////////

FTypedRange FStructRangeBuilder::BuildAndReset(FScratchAllocator& Scratch, const FStructDeclaration& Declared, const FDebugIds& Debug)
{
	FTypedRange Out = { MakeStructRangeSchema(SizeType, Declared.Id) };

	if (int64 Num = Structs.Num())
	{
		Out.Values = FBuiltRange::Create(Scratch, Structs.Num(), sizeof(FBuiltStruct*));
		FBuiltStruct** OutIt = reinterpret_cast<FBuiltStruct**>(Out.Values->Data);
		for (FMemberBuilder& Struct : Structs)
		{
			*OutIt++ = Struct.BuildAndReset(Scratch, Declared, Debug);
		}
	}		

	return Out;
}

//////////////////////////////////////////////////////////////////////////

FNestedRangeBuilder::~FNestedRangeBuilder()
{
	checkf(Ranges.IsEmpty(), TEXT("Half-built range, forgot to call BuildAndReset() before destruction?"));
}

FTypedRange FNestedRangeBuilder::BuildAndReset(FScratchAllocator& Scratch, ESizeType SizeType)
{
	FBuiltRange* Out = nullptr;
	if (int64 Num = Ranges.Num())
	{
		Out = FBuiltRange::Create(Scratch, Num, sizeof(FBuiltRange*), Ranges.GetData());
		Ranges.Reset();
	}

	return { MakeNestedRangeSchema(Scratch, SizeType, Schema), Out };
}

} // namespace PlainProps
