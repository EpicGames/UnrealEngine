// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsBuildSchema.h"
#include "PlainPropsInternalBuild.h"
#include "PlainPropsInternalFormat.h"
#include "PlainPropsInternalPrint.h"
#include "Algo/Compare.h"
#include "Algo/Find.h"
#include "Containers/Map.h"
#include "Containers/StringConv.h"
#include "Misc/StringBuilder.h"

namespace PlainProps
{

static FString PrintMemberSchema(const FIds& Ids, FMemberSchema Schema)
{
	TUtf8StringBuilder<256> Utf8SchemaStr;
	PrintMemberSchema(Utf8SchemaStr, Ids, Schema);
	FString SchemaStr(FStringView(StringCast<TCHAR>(Utf8SchemaStr.GetData(), Utf8SchemaStr.Len())));
	
	FMemberType InnermostType = Schema.GetInnermostType();
	if (InnermostType.IsStruct())
	{
		return FString::Printf(TEXT("%sstruct [%d]%s%s => %s"),
			Schema.Type.IsRange() ? TEXT("Range(s) of ") : TEXT(""),
			Schema.InnerSchema.Get().Idx,
			InnermostType.AsStruct().IsSuper ? TEXT(" (super)") : TEXT(""),
			InnermostType.AsStruct().IsDynamic ? TEXT(" (dynamic)") : TEXT(""),
			*SchemaStr);
	}
	else if (IsEnum(InnermostType))
	{
		return FString::Printf(TEXT("%s%s [%d] => %s"),
			Schema.Type.IsRange() ? TEXT("Range(s) of ") : TEXT(""),
			StringCast<TCHAR>(ToString(InnermostType.AsLeaf()).GetData()).Get(),
			Schema.InnerSchema.Get().Idx,
			*SchemaStr);
	}
	return SchemaStr;
}

//////////////////////////////////////////////////////////////////////////

struct FStructSchemaBuilder
{
	FStructSchemaBuilder(FStructId InId, const FStructDeclaration& InDecl, FSchemasBuilder& InSchemas);

	const FStructDeclaration&					Declaration;
	FSchemasBuilder&							AllSchemas;
	uint16										MinMembers = 0xFFFF;
	FStructId									Id;
	FOptionalMemberId*							MemberOrder = nullptr;
	FMemberSchema*								NotedSchemas = nullptr;
	TBitArray<>									NotedMembers;

	void										NoteMembersRecursively(const FBuiltStruct& Struct);
	void										NoteRangeRecursively(ESizeType NumType, TConstArrayView<FMemberType> Types, void* InnermostSchemaBuilder, const FBuiltRange* Range, FMemberId Member);
	FBuiltStructSchema							Build() const;
};

struct FEnumSchemaBuilder
{
	const FEnumDeclaration&						Declaration;
	FDebugIds									Debug;
	FEnumId										Id;
	TOptional<ELeafWidth>						NotedWidth;
	TSet<uint64>								NotedConstants;

	void										NoteValue(ELeafWidth Width, uint64 Value, FStructId Struct, FMemberId Member);
	void										NoteEmpty(ELeafWidth Width);
	FBuiltEnumSchema							Build() const;
};

//////////////////////////////////////////////////////////////////////////

FSchemasBuilder::FSchemasBuilder(const FIds& Names, const IDeclarations& Types, FScratchAllocator& InScratch, ESchemaFormat InFormat)
: Declarations(Types)
, Ids(Names)
, Format(InFormat)
, Scratch(InScratch)
, Debug(Names)
{}

FSchemasBuilder::~FSchemasBuilder() {}

FEnumSchemaBuilder&	FSchemasBuilder::NoteEnum(FEnumId Id)
{
	checkf(!bBuilt, TEXT("Noted new members after building"));

	FSetElementId Idx = EnumIndices.FindId(Id);
	if (Idx.IsValidId())
	{
		return Enums[Idx.AsInteger()];
	}

	auto Declaration = Declarations.Find(Id);
	checkf(Declaration, TEXT("Undeclared enum '%s' noted"), *Debug.Print(Id));

	Idx = EnumIndices.Emplace(Id);
	check(Idx.AsInteger() == Enums.Num());
	Enums.Emplace(*Declaration, Debug, Id);

	return Enums.Last();
}

FStructSchemaBuilder& FSchemasBuilder::NoteStruct(FStructId Id)
{
	checkf(!bBuilt, TEXT("Noted new members after building"));

	// Id is either FBindId or FDeclId depending on Format
	// Most bind ids are the same as decl ids though
	if (FSetElementId Idx = StructIndices.FindId(Id); Idx.IsValidId())
	{
		return Structs[Idx.AsInteger()];
	}
	
	const FStructDeclaration* Declaration = Declarations.Find(Id);
	checkf(Declaration, TEXT("Undeclared struct '%s' noted"), *Debug.Print(Id));
	
	// StableNames format can have lowered ids, so builder might exist already
	FStructId NoteId = (Format == ESchemaFormat::StableNames) ? FStructId(Declaration->Id) : Id;
	if (NoteId != Id)
	{
		if (FSetElementId Idx = StructIndices.FindId(NoteId); Idx.IsValidId())
		{
			return Structs[Idx.AsInteger()];
		}
	}

	// First time noted, make new builder
	FSetElementId Idx = StructIndices.Emplace(NoteId);
	check(Idx.AsInteger() == Structs.Num());
	Structs.Emplace(NoteId, *Declaration, *this);

	return Structs.Last();
}

void FSchemasBuilder::NoteStructAndMembers(FStructId Id, const FBuiltStruct& Struct)
{
	NoteStruct(Id).NoteMembersRecursively(Struct);
}

FBuiltSchemas FSchemasBuilder::Build()
{
	checkf(!bBuilt, TEXT("Already built"));
	bBuilt = true;

	NoteInheritanceChains();

	FBuiltSchemas Out;
	Out.Structs.Reserve(Structs.Num());
	Out.Enums.Reserve(Enums.Num());

	for (FStructSchemaBuilder& Struct : Structs)
	{
		Out.Structs.Emplace(Struct.Build());
	}
	for (FEnumSchemaBuilder& Enum : Enums)
	{
		Out.Enums.Emplace(Enum.Build());
	}

	return Out;
}

void FSchemasBuilder::NoteInheritanceChains()
{
	for (int Idx = 0, Num = Structs.Num(); Idx < Num; ++Idx)
	{
		const FStructDeclaration* Declaration = &Structs[Idx].Declaration;
		while (Declaration->Super)
		{
			FDeclId Super = Declaration->Super.Get();
			if (StructIndices.Contains(Super))
			{
				break;
			}

			Declaration = Declarations.Find(Super);
			checkf(Declaration, TEXT("Undeclared super struct '%s' noted"), *Debug.Print(Super));

			FSetElementId ElemId = StructIndices.Emplace(Super);
			check(ElemId.AsInteger() == Structs.Num());
			Structs.Emplace(Super, *Declaration, *this);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

FStructSchemaBuilder::FStructSchemaBuilder(FStructId InId, const FStructDeclaration& Decl, FSchemasBuilder& Schemas)
: Declaration(Decl)
, AllSchemas(Schemas)
, Id(InId)
, NotedMembers(false, Declaration.NumMembers + int32(!!Declaration.Super))
{
	if (int32 Num = NotedMembers.Num())
	{
		MemberOrder = Schemas.GetScratch().AllocateArray<FOptionalMemberId>(Num);
		NotedSchemas = Schemas.GetScratch().AllocateArray<FMemberSchema>(Num);

		MemberOrder[0] = NoId; // In case unnamed super exists
		TConstArrayView<FMemberId> Order = Declaration.GetMemberOrder();
		FMemory::Memcpy(MemberOrder + int32(!!Declaration.Super), Order.GetData(), Order.NumBytes());
	}
}

static bool RequiresDynamicStructSchema(const FMemberSchema& A, const FMemberSchema& B)
{
	if (A.InnerSchema != B.InnerSchema && A.Type.GetKind() == B.Type.GetKind())
	{
		if (A.Type.IsStruct())
		{
			return true;
		}
		else if (A.Type.IsRange() && A.GetInnermostType().IsStruct() && B.GetInnermostType().IsStruct())
		{
			// Same range size and nested range sizes
			return	A.Type == B.Type &&	Algo::Compare(	A.GetInnerRangeTypes().LeftChop(1),
														B.GetInnerRangeTypes().LeftChop(1));
		}
	}

	return false;
}

static void SetIsDynamic(FMemberType& InOut)
{
	FStructType Type = InOut.AsStruct();
	Type.IsDynamic = true;
	InOut = FMemberType(Type);
}

static void* NoteStructOrEnum(FSchemasBuilder& AllSchemas, bool bStruct, FInnerId Id)
{
	return bStruct ? static_cast<void*>(&AllSchemas.NoteStruct(Id.AsStructBindId())) : &AllSchemas.NoteEnum(Id.AsEnum());
}

void FStructSchemaBuilder::NoteMembersRecursively(const FBuiltStruct& Struct)
{
	checkf(Declaration.Occupancy != EMemberPresence::RequireAll || Struct.NumMembers == Declaration.NumMembers,
		TEXT("'%s' with %d members noted while declared to always have all %d members"), *AllSchemas.GetDebug().Print(Declaration.Id), Struct.NumMembers, Declaration.NumMembers);
	MinMembers = FMath::Min(MinMembers, Struct.NumMembers);
	
	if (Struct.NumMembers == 0)
	{
		return;
	}

	const int32 NumNoted = NotedMembers.Num();
	int32 NoteIdx = 0;
	for (const FBuiltMember& Member : MakeArrayView(Struct.Members, Struct.NumMembers))
	{
		while (NoteIdx < NumNoted && MemberOrder[NoteIdx] != Member.Name)
		{
			++NoteIdx;
		}
		checkf(NoteIdx < NumNoted, TEXT("Member '%s' has not been declared in '%s'"),
			*AllSchemas.GetDebug().Print(Member.Name),
			*AllSchemas.GetDebug().Print(Declaration.Id));

		if (NotedMembers[NoteIdx])
		{
			FMemberSchema& NotedSchema = NotedSchemas[NoteIdx];
			if (RequiresDynamicStructSchema(NotedSchema, Member.Schema))
			{
				if (!NotedSchema.GetInnermostType().AsStruct().IsDynamic)
				{
					SetIsDynamic(NotedSchema.EditInnermostType(AllSchemas.GetScratch()));
					NotedSchema.InnerSchema = NoId;
				}
				check(NotedSchema.InnerSchema == NoId);
				
			}
			else
			{
				checkf(NotedSchema == Member.Schema, TEXT("Member '%s' in '%s' first added as '%s' and later as '%s'."),
					*AllSchemas.GetDebug().Print(Member.Name),
					*AllSchemas.GetDebug().Print(Declaration.Id),
					*PrintMemberSchema(AllSchemas.GetIds(), NotedSchema),
					*PrintMemberSchema(AllSchemas.GetIds(), Member.Schema));
			}
		}
		else
		{
			NotedMembers[NoteIdx] = true;
			NotedSchemas[NoteIdx] = Member.Schema;
		}

		++NoteIdx;
		
		const FMemberSchema& Schema = Member.Schema;
		if (Schema.InnerSchema)
		{
			checkSlow(IsStructOrEnum(Schema.GetInnermostType()));
			FInnerId InnerSchema = Schema.InnerSchema.Get();

			switch (Schema.Type.GetKind())
			{
			case EMemberKind::Leaf:
				AllSchemas.NoteEnum(InnerSchema.AsEnum()).NoteValue(Schema.Type.AsLeaf().Width, Member.Value.Leaf, Id, Member.Name.Get());
				break;

			case EMemberKind::Struct:
				AllSchemas.NoteStruct(InnerSchema.AsStructBindId()).NoteMembersRecursively(*Member.Value.Struct);
				break;
		
			case EMemberKind::Range:
				void* InnerSchemaBuilder = NoteStructOrEnum(AllSchemas, Schema.GetInnermostType().IsStruct(), InnerSchema);
				NoteRangeRecursively(Schema.Type.AsRange().MaxSize, Schema.GetInnerRangeTypes(), InnerSchemaBuilder, Member.Value.Range, Member.Name.Get());			
				break;
			}
		}
	}
}

template<typename IntType>
void NoteEnumValues(FEnumSchemaBuilder& Schema, const IntType* Values, uint64 Num, FStructId Struct, FMemberId Member)
{
	for (IntType Value : TConstArrayView64<IntType>(Values, Num))
	{
		Schema.NoteValue(LeafWidth<sizeof(IntType)>, Value, Struct, Member);
	}
}

static void NoteEnumRange(FEnumSchemaBuilder& Out, FLeafType Leaf, const FBuiltRange& Range, FStructId Struct, FMemberId Member)
{
	check(Leaf.Type == ELeafType::Enum);
	switch (Leaf.Width)
	{
	case ELeafWidth::B8:	NoteEnumValues(Out, reinterpret_cast<const uint8* >(Range.Data), Range.Num, Struct, Member); break;
	case ELeafWidth::B16:	NoteEnumValues(Out, reinterpret_cast<const uint16*>(Range.Data), Range.Num, Struct, Member); break;
	case ELeafWidth::B32:	NoteEnumValues(Out, reinterpret_cast<const uint32*>(Range.Data), Range.Num, Struct, Member); break;
	case ELeafWidth::B64:	NoteEnumValues(Out, reinterpret_cast<const uint64*>(Range.Data), Range.Num, Struct, Member); break;
	}
}

static void NoteEmptyRange(ESizeType NumType, TConstArrayView<FMemberType> Types, void* InnermostSchema)
{
	FMemberType InnermostType = Types.Last();
	if (IsEnum(InnermostType))
	{
		static_cast<FEnumSchemaBuilder*>(InnermostSchema)->NoteEmpty(InnermostType.AsLeaf().Width);
	}
}

void FStructSchemaBuilder::NoteRangeRecursively(ESizeType NumType, TConstArrayView<FMemberType> Types, void* InnermostSchema, const FBuiltRange* Range, FMemberId Member)
{
	if (Range == nullptr)
	{
		NoteEmptyRange(NumType, Types, InnermostSchema);
		return;
	}
	checkf(Range->Num > 0, TEXT("Range was built but without values"));
	FMemberType Type = Types[0];
	switch (Type.GetKind())
	{
	case EMemberKind::Struct:
		for (const FBuiltStruct* Struct : Range->AsStructs())
		{
			static_cast<FStructSchemaBuilder*>(InnermostSchema)->NoteMembersRecursively(*Struct);
		}
		break;
	case EMemberKind::Range:
		for (const FBuiltRange* InnerRange : Range->AsRanges())
		{
			NoteRangeRecursively(Type.AsRange().MaxSize, Types.RightChop(1), InnermostSchema, InnerRange, Member);
		}
		break;
	case EMemberKind::Leaf:
		NoteEnumRange(/* out */ *static_cast<FEnumSchemaBuilder*>(InnermostSchema), Type.AsLeaf(), *Range, Id, Member);
		break;
	}
}

FBuiltStructSchema FStructSchemaBuilder::Build() const
{
	FType Type = AllSchemas.GetIds().Resolve(Id);
	FBuiltStructSchema Out = { Type, Id, ToOptionalStruct(Declaration.Super), /* dense */ true };
	
	if (int32 Num = NotedMembers.CountSetBits())
	{
		Out.bDense = Declaration.Occupancy == EMemberPresence::RequireAll || MinMembers == Num;
		Out.MemberNames.Reserve(Num);
		Out.MemberSchemas.Reserve(Num);
		for (int32 NoteIdx = 0, NoteNum = NotedMembers.Num(); NoteIdx < NoteNum; ++NoteIdx)
		{
			if (NotedMembers[NoteIdx])
			{
				if (FOptionalMemberId Name = MemberOrder[NoteIdx])
				{
					Out.MemberNames.Add(Name.Get());
				}
				Out.MemberSchemas.Add(&NotedSchemas[NoteIdx]);
			}
		}

		check(Num == Out.MemberSchemas.Num());
	}

	return Out;
}

//////////////////////////////////////////////////////////////////////////

FBuiltEnumSchema FEnumSchemaBuilder::Build() const
{
	FBuiltEnumSchema Out = { Declaration.Type, Id };
	Out.Mode = Declaration.Mode;
	Out.Width = NotedWidth.GetValue();

	if (int32 Num = NotedConstants.Num())
	{
		Out.Names.Reserve(Num);
		Out.Constants.Reserve(Num);
		for (FEnumerator Enumerator : Declaration.GetEnumerators())
		{
			if (NotedConstants.Contains(Enumerator.Constant))
			{
				Out.Names.Add(Enumerator.Name);
				Out.Constants.Add(Enumerator.Constant);
			}
		}
	}
	
	checkf(	NotedConstants.Num() == Out.Constants.Num() || 
			NotedConstants.Num() == Out.Constants.Num() + (Out.Mode == EEnumMode::Flag),
			TEXT("Noted %d constants but included %d in fla%c enum %s"), 
			NotedConstants.Num(), Out.Constants.Num(), "tg"[Out.Mode == EEnumMode::Flag], *Debug.Print(Id));
	return Out;
}

void FEnumSchemaBuilder::NoteValue(ELeafWidth Width, uint64 Value, FStructId Struct, FMemberId Member)
{
	checkf(NotedWidth == NullOpt || NotedWidth == Width,
		TEXT("Enum '%s' in '%s' first added as '%.*hs' and later as '%.*hs' for value '%llx'."),
		*Debug.Print(Member), *Debug.Print(Struct),
		ToString(*NotedWidth).Len(), ToString(*NotedWidth).GetData(),
		ToString(Width).Len(), ToString(Width).GetData(),
		Value);

	NotedWidth = Width;

	if (Declaration.Mode == EEnumMode::Flag)
	{
		if (Value == 0)
		{
			// Don't validate 0 flag is declared, it isn't
			NotedConstants.Add(Value);
		}
		else
		{
			const int32 NumValidated = NotedConstants.Num();
			while (Value != 0)
			{
				uint64 HiBit = uint64(1) << FMath::FloorLog2_64(Value);
				NotedConstants.Add(HiBit);
				Value &= ~HiBit;
			}

			for (int32 Idx = NumValidated, Num = NotedConstants.Num(); Idx < Num; ++Idx)
			{
				uint64 Flag = NotedConstants.Get(FSetElementId::FromInteger(Idx));
				checkf(Algo::FindBy(Declaration.GetEnumerators(), Flag, &FEnumerator::Constant),
					TEXT("Enum flag %llx is undeclared in %s, illegal value detected in %s::%s"),
					Flag, *Debug.Print(Id), *Debug.Print(Struct), *Debug.Print(Member));
			}
		}
	}
	else
	{
		bool bValidated;
		NotedConstants.FindOrAdd(Value, /* out */ &bValidated);
		if (!bValidated)
		{
			checkf(Algo::FindBy(Declaration.GetEnumerators(), Value, &FEnumerator::Constant),
				TEXT("Enum value %llu is undeclared in %s, illegal value detected in %s::%s"),
				Value, *Debug.Print(Id), *Debug.Print(Struct), *Debug.Print(Member));
		}
	}
}

void FEnumSchemaBuilder::NoteEmpty(ELeafWidth Width)
{
	check(NotedWidth == NullOpt || NotedWidth == Width);
	NotedWidth = Width;
}

//////////////////////////////////////////////////////////////////////////

TArray<FStructId>	ExtractRuntimeIds(const FBuiltSchemas& In)
{
	TArray<FStructId>	Out;
	Out.SetNumUninitialized(In.Structs.Num());
	const FBuiltStructSchema* InIt = In.Structs.GetData();
	for (FStructId& Id : Out)
	{
		Id = (*InIt++).Id;
	}
	return Out;
}

} // namespace PlainProps
