// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsDeclare.h"
#include "PlainPropsSpecify.h"
#include "Algo/Compare.h"
#include "Containers/Set.h"
#include "Templates/RefCounting.h"

namespace PlainProps
{

// Note: For automated upgrade purposes it could be better to not strip out enum flag aliases,
//		 E.g. Saving E::All in E { A=1, B=2, All=A|B }, adding C=4, All=A|B|C and loading will load A|B
//		 It's impossible to know if a user set A|B or All when saving though, we only have the value 3.
//		 To really know, we'd need to instead save an enum oplog, i.e. {set A, set B} or {set All}
static TConstArrayView<FEnumerator> StripAliases(TArray<FEnumerator, TInlineAllocator<64>>& OutTmp, TConstArrayView<FEnumerator> In, EEnumMode Mode, const FDebugIds& Debug)
{
	TBitArray<> Aliases;
	if (Mode == EEnumMode::Flag)
	{
		bool bSeen0 = false;
		uint64 Seen = 0;
		for (FEnumerator E : In)
		{
			bool bAlias = E.Constant ? (Seen & E.Constant) == E.Constant : bSeen0;
			checkf(bAlias || FMath::CountBits64(E.Constant) <= 1, TEXT("Flag enums must use one bit per enumerator, %s is %llx"), *Debug.Print(E.Name), E.Constant);

			Aliases.Add(bAlias);
			Seen |= E.Constant;
			bSeen0 |= E.Constant == 0;
		}
	}
	else
	{
		TSet<uint64, DefaultKeyFuncs<uint64>,TInlineSetAllocator<64>> Seen;
		for (FEnumerator E : In)
		{
			bool bAlias;
			Seen.FindOrAdd(E.Constant, /* out*/ &bAlias);
			Aliases.Add(bAlias);
		}
	}

	if (int32 NumAliases = Aliases.CountSetBits())
	{
		// All aliases are frequently at the end
		int32 FirstAlias = Aliases.Find(true);
		if (FirstAlias == In.Num() - NumAliases)
		{
			return In.Slice(0, FirstAlias);
		}

		// Aliases mixed in with values, make a copy and return it
		const FEnumerator* InIt = &In[0];
		for (bool bAlias : Aliases)
		{
			if (!bAlias)
			{
				OutTmp.Add(*InIt);
			}
			++InIt;
		}
		return OutTmp;
	}

	return In;
}

static void ValidateDeclaration(const FEnumDeclaration& Enum)
{
#if DO_CHECK
	if (Enum.Mode == EEnumMode::Flag)
	{
		for (FEnumerator E : Enum.GetEnumerators())
		{
			checkf(FMath::CountBits64(E.Constant) <= 1, TEXT("Flag enums must use one bit per enumerator"));
		}
	}

	TSet<uint32, DefaultKeyFuncs<uint32>, TInlineSetAllocator<64>> Names;
	TSet<uint64, DefaultKeyFuncs<uint64>,TInlineSetAllocator<64>> Constants;
	for (FEnumerator E : Enum.GetEnumerators())
	{
		bool bDeclared;
		Names.FindOrAdd(E.Name.Idx, /* out*/ &bDeclared);
		checkf(!bDeclared, TEXT("Enumerator name declared twice"));
		Constants.FindOrAdd(E.Constant, /* out*/ &bDeclared);
		checkf(!bDeclared, TEXT("Enumerator constant declared twice"));
	}

	uint64 OverflowMasks[4] = { ~uint64(0) << 8, ~uint64(0) << 16, ~uint64(0) << 32, 0 };
	uint64 OverflowMask = OverflowMasks[(uint8)Enum.Width];
	for (FEnumerator E : Enum.GetEnumerators())
	{
		checkf(!(E.Constant & OverflowMask), TEXT("Enumerator constant %llx is larger than declared width"), E.Constant);
	}
#endif
}

template<typename T>
void CopyItems(T* It, TConstArrayView<T> Items)
{
	for (T Item : Items)
	{
		(*It++)  = Item;
	}
}

const FEnumDeclaration& FEnumDeclarations::Declare(FEnumId Id, FType Type, EEnumMode Mode, ELeafWidth Width, TConstArrayView<FEnumerator> Enumerators, EEnumAliases Policy)
{
	if (static_cast<int32>(Id.Idx) >= Declarations.Num())
	{
		Declarations.SetNum(Id.Idx + 1);
	}
	
	TUniquePtr<FEnumDeclaration>& Ptr = Declarations[Id.Idx];
	checkf(!Ptr, TEXT("'%s' is already declared"), *Debug.Print(Id));

	TArray<FEnumerator, TInlineAllocator<64>> Tmp;
	if (Policy == EEnumAliases::Strip)
	{
		Enumerators = StripAliases(/* out */ Tmp, Enumerators, Mode, Debug);
	}

	FEnumDeclaration Header{Type, Mode, Width, IntCastChecked<uint16>(Enumerators.Num())};
	void* Data = FMemory::Malloc(sizeof(FEnumDeclaration) + Enumerators.Num() * Enumerators.GetTypeSize());
	Ptr.Reset(new (Data) FEnumDeclaration(Header));
	CopyItems(Ptr->Enumerators, Enumerators);

	ValidateDeclaration(*Ptr);

	return *Ptr;
}

#if DO_CHECK
void FEnumDeclarations::Check(FEnumId Id) const
{
	checkf(Id.Idx < (uint32)Declarations.Num() && Declarations[Id.Idx], TEXT("'%s' is undeclared"), *Debug.Print(Id));
}
#endif

////////////////////////////////////////////////////////////////////////////////////////////////

uint32 FStructDeclaration::CalculateSize() const
{
	uint32 Out = sizeof(FStructDeclaration) + NumMembers * sizeof(FMemberId);
	Out = Align(Out + NumInnerIds * sizeof(FInnerId), alignof(FInnerId));
	Out = Align(Out + (NumMembers + NumInnerRanges) * sizeof(FMemberType), alignof(FMemberType));
	return Out;
}

bool FStructDeclaration::Release() const
{
	if (RefCount.fetch_sub(1, std::memory_order_acq_rel) == 1)
	{
		delete this;
		return true;
	}
	return false;
}

////////////////////////////////////////////////////////////////////////////////////////////////

FStructDeclarationPtr Declare(FStructSpec In) 
{
	checkf(!(In.Super && In.Occupancy == EMemberPresence::RequireAll), TEXT("Dense substructs are not supported"));
	check(In.MemberNames.Num() == In.MemberTypes.Num());

	// Extract inner ids
	TArray<FInnerId, TInlineAllocator<16>> InnerIds;
	uint32 NumInnerRanges = 0;
	for (FMemberSpecView Member : In.MemberTypes)
	{
		if (Member.InnermostId)
		{
			InnerIds.Emplace(Member.InnermostId.Get());
		}
		NumInnerRanges += Member.Ranges.Num();
	}

	// Make header and allocate
	uint16 NumMembers = IntCastChecked<uint16>(In.MemberNames.Num());
	uint16 NumInnerIds = IntCastChecked<uint16>(InnerIds.Num());
	FStructDeclaration Header{1, In.Id, In.Super, In.Version, NumMembers, IntCastChecked<uint16>(NumInnerRanges), NumInnerIds, In.Occupancy};
	FStructDeclaration* Out = new (FMemory::Malloc(Header.CalculateSize())) FStructDeclaration;
	FMemory::Memcpy(Out, &Header, sizeof(Header));
	
	// Copy footer
	CopyItems(Out->MemberNames, In.MemberNames);
	CopyItems(const_cast<FInnerId*>(Out->GetInnerIds()), MakeConstArrayView(InnerIds));
	FMemberType* TypeIt = const_cast<FMemberType*>(Out->GetTypes());
	FMemberType* RangeIt = TypeIt + Header.NumMembers;
	for (FMemberSpecView Member : In.MemberTypes)
	{
		if (int32 NumRanges = Member.Ranges.Num())
		{
			*TypeIt++ = FMemberType(Member.Ranges[0]);
			for (ESizeType Range : Member.Ranges.RightChop(1))
			{
				*RangeIt++ = FMemberType(Range);
			}
			*RangeIt++ = Member.InnermostType;
		}
		else
		{
			*TypeIt++ = Member.InnermostType;
		}
	}
	check(TypeIt == Out->GetInnerRangeTypes());
	check(RangeIt == Out->GetInnerRangeTypes() + NumInnerRanges);

	// Make smart pointer
	return FStructDeclarationPtr(Out, /* bAddRef */ false);	
}

////////////////////////////////////////////////////////////////////////////////////////////////

FMemberSpec::FMemberSpec(ESizeType MaxSize, FMemberSpec Inner)
: FMemberSpec(Inner)
{
	RangeWrap(MaxSize);
}

FMemberSpec::FMemberSpec(TConstArrayView<FMemberType> Members, FOptionalInnerId InInnermostId)
: FMemberSpec(Members.Last(), InInnermostId)
{
	TConstArrayView<FMemberType> RangeTypes = Members.LeftChop(1);
	for (FMemberType Range : ReverseIterate(RangeTypes))
	{
		RangeWrap(Range);
	}
}

FMemberSpec::FMemberSpec(FMemberType Type, TConstArrayView<FMemberType> InnerRangeTypes, FOptionalInnerId InInnermostId)
: FMemberSpec(InnerRangeTypes.IsEmpty() ? Type : InnerRangeTypes.Last(), InInnermostId)
{
	if (int32 NumRanges = InnerRangeTypes.Num())
	{
		TConstArrayView<FMemberType> RangeTypes = InnerRangeTypes.Slice(0, NumRanges - 1);
		for (FMemberType Nested : ReverseIterate(RangeTypes))
		{
			RangeWrap(Nested);	
		}
		RangeWrap(Type);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////

const FEnumDeclaration&	IDeclarations::Get(FEnumId Id) const
{
	const FEnumDeclaration* Out = Find(Id);
	check(Out);
	return *Out;
}

const FStructDeclaration& IDeclarations::Get(FStructId Id) const
{
	const FStructDeclaration* Out = Find(Id);
	check(Out);
	return *Out;
}

} // namespace PlainProps
