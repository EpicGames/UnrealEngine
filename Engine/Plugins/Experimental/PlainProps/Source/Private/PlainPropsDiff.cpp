// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsDiff.h"
#include "PlainPropsBind.h"
#include "PlainPropsBuild.h"
#include "PlainPropsInternalBind.h"
#include "PlainPropsInternalDiff.h"
#include "PlainPropsInternalFormat.h"
#include "PlainPropsInternalRead.h"

namespace PlainProps
{

int32 FDiffContext::AddDiff(FDiffNode&& Node)
{
	int32 Index = Diffs.Add(Node);

	LastSibling = Index;

	return Index;
}

void FDiffContext::Reset()
{
	Diffs.Reset();
	LastSibling = INDEX_NONE;
}

inline bool DiffMember(FLeafMemberBinding Member, const void* OwnerA, const void* OwnerB, const FBindContext&, FOptionalMemberId)
{
	return !!DiffLeaf(At(OwnerA, Member.Offset), At(OwnerB, Member.Offset), Member.Leaf);
}
inline bool DiffMember(FLeafMemberBinding Member, const void* OwnerA, const void* OwnerB, FDiffContext& Ctx, FOptionalMemberId Name)
{
	const uint8* A = At(OwnerA, Member.Offset);
	const uint8* B = At(OwnerB, Member.Offset);
	if (DiffLeaf(A, B, Member.Leaf))
	{
		// place breakpoint here
		Ctx.AddDiff({Member.Leaf.Pack(), Name, {Member.Enum}, A, B, Ctx.LastSibling});
		return true;
	}
	return false;
}

template<typename ContextType>
struct TLastSiblingScope
{
	TLastSiblingScope(ContextType& InCtx) : Ctx(InCtx)
	{
		if constexpr (std::is_same_v<ContextType, FDiffContext>)
		{
			OldLastSibling = InCtx.LastSibling;
			InCtx.LastSibling = INDEX_NONE;
		}
	}

	~TLastSiblingScope()
	{
		if constexpr (std::is_same_v<ContextType, FDiffContext>)
		{
			Ctx.LastSibling = OldLastSibling;
		}
	}

	ContextType& Ctx;
	int32 OldLastSibling;
};

template<typename ContextType>
bool DiffMember(FRangeMemberBinding Member, const void* OwnerA, const void* OwnerB, ContextType& Ctx, FOptionalMemberId Name)
{
	FRangeBinding Binding = Member.RangeBindings[0];
	FMemberBindType InnerType = Member.InnerTypes[0];
	EMemberKind InnerKind = InnerType.GetKind();
	const uint8* A = At(OwnerA, Member.Offset);
	const uint8* B = At(OwnerB, Member.Offset);

	bool bDiff;
	if (Binding.IsLeafBinding())
	{
		bDiff = Binding.AsLeafBinding().DiffLeaves(A, B);
	}
	else
	{
		const IItemRangeBinding& ItemBinding = Binding.AsItemBinding();
		if (InnerKind == EMemberKind::Struct)
		{
			bDiff = DiffItemRange(A, B, ItemBinding, Ctx, Member.InnermostId.Get().AsStructBindId());
		}
		else if (InnerKind == EMemberKind::Leaf)
		{
			// Todo: float/double ranges should use PreciseFP
			bDiff = DiffItemRange(A, B, ItemBinding, Ctx, SizeOf(GetItemWidth(InnerType.AsLeaf())));
		}
		else
		{
			bDiff = DiffItemRange(A, B, ItemBinding, Ctx, GetInnerRange(Member));
		}
	}

	if constexpr (std::is_same_v<ContextType, FDiffContext>)
	{
		if (bDiff)
		{
			// place breakpoint here
			Ctx.AddDiff({FMemberBindType(Binding.GetSizeType()), Name, {.Range = Binding}, A, B, Ctx.LastSibling});
		}
	}
	return bDiff;
}

template<typename ContextType>
bool DiffMembers(FBindId Id, const void* A, const void* B, ContextType& Ctx);

template<typename ContextType>
inline bool DiffStruct(FBindId Id, const void* A, const void* B, ContextType& Ctx)
{
	if (const ICustomBinding* Custom = Ctx.Customs.FindStruct(Id))
	{
		return Custom->DiffCustom(A, B, Ctx);
	}
	return DiffMembers(Id, A, B, Ctx);
}

static bool DiffMember(FStructMemberBinding Member, const void* OwnerA, const void* OwnerB, const FBindContext& Ctx, FMemberId)
{
	return DiffStruct(Member.Id, At(OwnerA, Member.Offset), At(OwnerB, Member.Offset), Ctx);
}

static bool DiffMember(FStructMemberBinding Member, const void* OwnerA, const void* OwnerB, FDiffContext& Ctx, FMemberId Name)
{
	const uint8* A = At(OwnerA, Member.Offset);
	const uint8* B = At(OwnerB, Member.Offset);
	if (DiffStruct(Member.Id, A, B, Ctx))
	{
		// place breakpoint here
		Ctx.AddDiff({FMemberBindType(Member.Type), Name, {.Struct = Member.Id}, A, B, Ctx.LastSibling});
		return true;
	}
	return false;
}

template<typename ContextType>
bool DiffMember(/* in-out */ FMemberVisitor& It, const void* A, const void* B, ContextType& Ctx, FMemberId Name)
{
	switch (It.PeekKind())
	{
	case EMemberKind::Leaf:		return DiffMember(It.GrabLeaf(), A, B, Ctx, Name);
	case EMemberKind::Range:	return DiffMember(It.GrabRange(), A, B, Ctx, Name);
	case EMemberKind::Struct:	return DiffMember(It.GrabStruct(), A, B, Ctx, Name);
	}
	return true;
}

template<typename ContextType>
bool DiffMembers(FBindId Id, const void* A, const void* B, ContextType& Ctx)
{
	const FStructDeclaration* Declaration;
	const FSchemaBinding& Schema = Ctx.Schemas.GetStruct(Id, /* out */ Declaration);

	bool bDiff = false;

	FMemberVisitor It(Schema);
	if (Schema.HasSuper())
	{
		if (DiffMembers(It.GrabSuper(), A, B, Ctx))
		{
			bDiff = true;
			if (StopOnFirstDiff(Ctx)) { return true; }
		}
	}

	TLastSiblingScope ScopeLastSibling(Ctx);

	for (FMemberId Name : Declaration->GetMemberOrder())
	{
		if (DiffMember(/* in-out */It, A, B, Ctx, Name))
		{
			bDiff = true;
			if (StopOnFirstDiff(Ctx)) { return true; }
		}
	}

	check(!It.HasMore());
	return bDiff;
}

template<typename ContextType>
bool DiffItem(const void* A, const void* B, ContextType& Ctx, FRangeMemberBinding Range)
{
	return DiffMember(Range, A, B, Ctx, NoId);
}

template<typename ContextType>
bool DiffItem(const void* A, const void* B, ContextType& Ctx, FBindId Id)
{
	return DiffStruct(Id, A, B, Ctx);
}

////////////////////////////////////////////////////////////////////////////////////////////////

bool DiffStructs(const void* A, const void* B, FBindId Id, const FBindContext& Ctx)
{
	return DiffStruct(Id, A, B, Ctx);
}

bool DiffStructs(const void* A, const void* B, FBindId Id, FDiffContext& Ctx)
{
	return DiffStruct(Id, A, B, Ctx);
}

bool DiffLeaves(float A, float B)
{
	return !::UE::PreciseFPEqual(A, B);
}

bool DiffLeaves(double A, double B)
{
	return !::UE::PreciseFPEqual(A, B);
}

bool DiffRanges(const void* A, const void* B, const IItemRangeBinding& Binding, FUnpackedLeafType Leaf)
{
	// Todo: float/double ranges should use PreciseFP
	return DiffItemRange(A, B, Binding, /* dummy ctx */ Binding, SizeOf(Leaf.Width));
}

bool DiffRanges(const void* A, const void* B, const IItemRangeBinding& Binding, FBindId ItemType, const FBindContext& Ctx)
{
	return DiffItemRange(A, B, Binding, Ctx, ItemType);
}

bool DiffRanges(const void* A, const void* B, const IItemRangeBinding& Binding, FBindId ItemType, FDiffContext& Ctx)
{
	return DiffItemRange(A, B, Binding, Ctx, ItemType);
}

bool DiffRanges(const void* A, const void* B, const IItemRangeBinding& Binding, FRangeMemberBinding ItemType, const FBindContext& Ctx)
{
	return DiffItemRange(A, B, Binding, Ctx, ItemType);
}

bool DiffRanges(const void* A, const void* B, const IItemRangeBinding& Binding, FRangeMemberBinding ItemType, FDiffContext& Ctx)
{
	return DiffItemRange(A, B, Binding, Ctx, ItemType);
}

////////////////////////////////////////////////////////////////////////////////

static uint64 CalculateSize(const FSchemaBatch& In)
{
	uint32 NumParameters = 0;
	for (FParametricType ParametricType : In.GetParametricTypes())
	{
		NumParameters += ParametricType.Parameters.NumParameters;
	}
	const uint8* End = reinterpret_cast<const uint8*>(In.GetFirstParameter() + NumParameters);
	const uint64 Size = End - reinterpret_cast<const uint8*>(&In);
	return Size;
}

static bool operator==(const FSchemaBatch& A, const FSchemaBatch& B)
{
	const uint64 SizeA = CalculateSize(A);
	const uint64 SizeB = CalculateSize(B);
	return SizeA == SizeB && FMemory::Memcmp(&A, &B, SizeA) == 0;
}

static bool operator==(const FStructSchema& A, const FStructSchema& B)
{
	const uint64 SizeA = CalculateSize(A);
	const uint64 SizeB = CalculateSize(B);
	return SizeA == SizeB && FMemory::Memcmp(&A, &B, SizeA) == 0;
}

static bool operator==(const FEnumSchema& A, const FEnumSchema& B)
{
	const uint64 SizeA = CalculateSize(A);
	const uint64 SizeB = CalculateSize(B);
	return SizeA == SizeB && FMemory::Memcmp(&A, &B, SizeA) == 0;
}

static bool operator!=(const FSchemaBatch& A, const FSchemaBatch& B)	{ return !(A == B); }
static bool operator!=(const FStructSchema& A, const FStructSchema& B)	{ return !(A == B); }
static bool operator!=(const FEnumSchema& A, const FEnumSchema& B)		{ return !(A == B); }

////////////////////////////////////////////////////////////////////////////////

static bool DiffStructSchema(const FStructSchema& A, const FStructSchema& B)
{
	const bool Diff = A != B;
#if DO_CHECK
	check(A.Type == B.Type);
	check(A.NumMembers == B.NumMembers);
	check(A.NumRangeTypes == B.NumRangeTypes);
	check(A.NumInnerSchemas == B.NumInnerSchemas);
	check(A.Inheritance == B.Inheritance);
	check(A.IsDense == B.IsDense);
	check(EqualItems(A.GetMemberTypes(), B.GetMemberTypes()));
	check(EqualItems(A.GetRangeTypes(), B.GetRangeTypes()));
	check(EqualItems(A.GetMemberNames(), B.GetMemberNames()));
	check(EqualItems(
		MakeArrayView(A.GetInnerSchemas(), A.NumInnerSchemas),
		MakeArrayView(B.GetInnerSchemas(), B.NumInnerSchemas)));
	check(A.GetSuper() == B.GetSuper());
	check(A == B);
#endif
	return Diff;
}

static bool DiffEnumSchema(const FEnumSchema& A, const FEnumSchema& B)
{
	const bool Diff = A != B;
#if DO_CHECK
	check(A.Type == B.Type);
	check(A.FlagMode == B.FlagMode);
	check(A.ExplicitConstants == B.ExplicitConstants);
	check(A.Width == B.Width);
	check(A.Num == B.Num);
	check(EqualItems(MakeArrayView(A.Footer, A.Num), MakeArrayView(B.Footer, B.Num)));
	if (A.Width == B.Width)
	{
		switch (A.Width)
		{
		case ELeafWidth::B8:  check(EqualItems(GetConstants<uint8 >(A), GetConstants<uint8 >(B))); break;
		case ELeafWidth::B16: check(EqualItems(GetConstants<uint16>(A), GetConstants<uint16>(B))); break;
		case ELeafWidth::B32: check(EqualItems(GetConstants<uint32>(A), GetConstants<uint32>(B))); break;
		case ELeafWidth::B64: check(EqualItems(GetConstants<uint64>(A), GetConstants<uint64>(B))); break;
		}
	}
	check(A == B);
#endif
	return Diff;
}

static bool DiffSchemas(const FSchemaBatch& A, const FSchemaBatch& B)
{
	const bool Diff = A != B;
#if DO_CHECK
	check(A.NumNestedScopes == B.NumNestedScopes);
	check(A.NestedScopesOffset == B.NestedScopesOffset);
	check(A.NumParametricTypes == B.NumParametricTypes);
	check(A.NumSchemas == B.NumSchemas);
	check(A.NumStructSchemas == B.NumStructSchemas);
	check(EqualItems(A.GetSchemaOffsets(), B.GetSchemaOffsets()));
	check(EqualItems(A.GetNestedScopes(), B.GetNestedScopes()));
	check(EqualItems(A.GetParametricTypes(), B.GetParametricTypes()));

	{
		TSchemaRange<const FStructSchema> AA = GetStructSchemas(A);
		TSchemaRange<const FStructSchema> BB = GetStructSchemas(B);
		TSchemaIterator<const FStructSchema> ItA = AA.begin();
		TSchemaIterator<const FStructSchema> ItB = BB.begin();
		for (; ItA != AA.end() && ItB != BB.end(); ++ItA, ++ItB)
		{
			check(!DiffStructSchema(*ItA, *ItB));
		}
	}

	{
		TSchemaRange<const FEnumSchema> AA = GetEnumSchemas(A);
		TSchemaRange<const FEnumSchema> BB = GetEnumSchemas(B);
		TSchemaIterator<const FEnumSchema> ItA = AA.begin();
		TSchemaIterator<const FEnumSchema> ItB = BB.begin();
		for (; ItA != AA.end() && ItB != BB.end(); ++ItA, ++ItB)
		{
			check(!DiffEnumSchema(*ItA, *ItB));
		}
	}

	check(!Diff);
#endif
	return Diff;
}

bool DiffSchemas(FSchemaBatchId A, FSchemaBatchId B)
{
	return DiffSchemas(GetReadSchemas(A), GetReadSchemas(B));
}

////////////////////////////////////////////////////////////////////////////////

static bool DiffLeaf(FLeafView A, FLeafView B)
{
	if (A.Leaf != B.Leaf || A.Enum != B.Enum)
	{
		return true;
	}

	if (A.Leaf.Type == ELeafType::Bool)
	{
		return A.Value.bValue != B.Value.bValue;
	}

	switch (A.Leaf.Width)
	{
	case ELeafWidth::B8:  return !!FMemory::Memcmp(A.Value.Ptr, B.Value.Ptr, 1);
	case ELeafWidth::B16: return !!FMemory::Memcmp(A.Value.Ptr, B.Value.Ptr, 2);
	case ELeafWidth::B32: return !!FMemory::Memcmp(A.Value.Ptr, B.Value.Ptr, 4);
	case ELeafWidth::B64: return !!FMemory::Memcmp(A.Value.Ptr, B.Value.Ptr, 8);
	}

	unimplemented();
	return false;
}

static bool DiffLeaves(FUnpackedLeafType Leaf, FLeafRangeView A, FLeafRangeView B, uint64& OutIdx)
{
	OutIdx = ~uint64(0);
	switch (Leaf.Type)
	{
	case ELeafType::Bool:
		check(Leaf.Width == ELeafWidth::B8);
		return DiffItems(A.AsBools(), B.AsBools(), OutIdx);
	case ELeafType::IntS:
		switch (Leaf.Width)
		{
		case ELeafWidth::B8:  return DiffItems(A.AsS8s(),  B.AsS8s(),  OutIdx);
		case ELeafWidth::B16: return DiffItems(A.AsS16s(), B.AsS16s(), OutIdx);
		case ELeafWidth::B32: return DiffItems(A.AsS32s(), B.AsS32s(), OutIdx);
		case ELeafWidth::B64: return DiffItems(A.AsS64s(), B.AsS64s(), OutIdx);
		}
	case ELeafType::IntU:
		switch (Leaf.Width)
		{
		case ELeafWidth::B8:  return DiffItems(A.AsU8s(),  B.AsU8s(),  OutIdx);
		case ELeafWidth::B16: return DiffItems(A.AsU16s(), B.AsU16s(), OutIdx);
		case ELeafWidth::B32: return DiffItems(A.AsU32s(), B.AsU32s(), OutIdx);
		case ELeafWidth::B64: return DiffItems(A.AsU64s(), B.AsU64s(), OutIdx);
		}
	case ELeafType::Float:
		if (Leaf.Width == ELeafWidth::B32)
		{
			return DiffItems(A.AsFloats(), B.AsFloats(), OutIdx);
		}
		check(Leaf.Width == ELeafWidth::B64);
		return DiffItems(A.AsDoubles(), B.AsDoubles(), OutIdx);
	case ELeafType::Hex:
		switch (Leaf.Width)
		{
		case ELeafWidth::B8:  return DiffItems(A.AsHex8s(),  B.AsHex8s(),  OutIdx);
		case ELeafWidth::B16: return DiffItems(A.AsHex16s(), B.AsHex16s(), OutIdx);
		case ELeafWidth::B32: return DiffItems(A.AsHex32s(), B.AsHex32s(), OutIdx);
		case ELeafWidth::B64: return DiffItems(A.AsHex64s(), B.AsHex64s(), OutIdx);
		}
	case ELeafType::Enum:
		switch (Leaf.Width)
		{
		case ELeafWidth::B8:  return DiffItems(A.AsUnderlyingValues<uint8>(),  B.AsUnderlyingValues<uint8>(),  OutIdx);
		case ELeafWidth::B16: return DiffItems(A.AsUnderlyingValues<uint16>(), B.AsUnderlyingValues<uint16>(), OutIdx);
		case ELeafWidth::B32: return DiffItems(A.AsUnderlyingValues<uint32>(), B.AsUnderlyingValues<uint32>(), OutIdx);
		case ELeafWidth::B64: return DiffItems(A.AsUnderlyingValues<uint64>(), B.AsUnderlyingValues<uint64>(), OutIdx);
		}
	case ELeafType::Unicode:
		switch (Leaf.Width)
		{
		case ELeafWidth::B8:  return DiffItems(A.AsUtf8(),  B.AsUtf8(),  OutIdx);
		case ELeafWidth::B16: return DiffItems(A.AsUtf16(), B.AsUtf16(), OutIdx);
		case ELeafWidth::B32: return DiffItems(A.AsUtf32(), B.AsUtf32(), OutIdx);
		case ELeafWidth::B64: check(false);
		}
	}
	unimplemented();
	return false;
}

static bool DiffMembers(FStructView A, FStructView B, FReadDiffPath& Out);
static bool DiffRange(ESizeType NumType, FRangeView A, FRangeView B, FReadDiffPath& Out);

static bool DiffStructs(FStructType Struct, FStructRangeView A, FStructRangeView B, FReadDiffPath& Out)
{
	uint64 NumA = A.Num();
	uint64 NumB = B.Num();

	uint64 DiffIdx = 0;
	bool bDiff = false;
	FStructRangeIterator ItB = B.begin();
	for (FStructView ItA : A)
	{
		if (DiffIdx >= NumB)
		{
			break;
		}
		bDiff = DiffMembers(ItA, *ItB, Out);
		if (bDiff)
		{
			break;
		}
		++ItB;
		++DiffIdx;
	}
	if (bDiff || NumA != NumB)
	{
		Out.Emplace(FMemberType(Struct), NoId, NoId, DiffIdx);
		return true;
	}
	return false;
}

static bool DiffRanges(ESizeType NumType, FNestedRangeView A, FNestedRangeView B, FReadDiffPath& Out)
{
	uint64 NumA = A.Num();
	uint64 NumB = B.Num();
	
	uint64 DiffIdx = 0;
	bool bDiff = false;
	FNestedRangeIterator ItB = B.begin();
	for (FRangeView ItA : A)
	{
		if (DiffIdx >= NumB)
		{
			break;
		}
		bDiff = DiffRange(NumType, ItA, *ItB, Out);
		if (bDiff)
		{
			break;
		}
		++ItB;
		++DiffIdx;
	}
	if (bDiff || NumA != NumB)
	{
		Out.Emplace(FMemberType(NumType), NoId, NoId, DiffIdx);
		return true;
	}
	return false;
}

static bool DiffLeaves(FUnpackedLeafType Leaf, FLeafRangeView A, FLeafRangeView B, FReadDiffPath& Out)
{
	uint64 DiffIdx;
	if (DiffLeaves(Leaf, A, B, DiffIdx))
	{
		Out.Emplace(Leaf.Pack(), NoId, NoId, DiffIdx);
		return true;
	}
	return false;
}

static bool DiffRange(ESizeType NumType, FRangeView A, FRangeView B, FReadDiffPath& Out)
{
	bool bDiff = A.GetItemType() != B.GetItemType();
	if (!bDiff)
	{
		switch (A.GetItemType().GetKind())
		{
		case EMemberKind::Leaf:
			bDiff = DiffLeaves(A.GetItemType().AsLeaf(), A.AsLeaves(), B.AsLeaves(), Out);
			break;
		case EMemberKind::Struct:
			bDiff = DiffStructs(A.GetItemType().AsStruct(), A.AsStructs(), B.AsStructs(), Out);
			break;
		case EMemberKind::Range:
			bDiff = DiffRanges(A.GetItemType().AsRange().MaxSize, A.AsRanges(), B.AsRanges(), Out);
			break;
		}
	}
	return bDiff;
}

static bool DiffMembers(FStructView A, FStructView B, FReadDiffPath& Out)
{
	FMemberReader ItA(A);
	FMemberReader ItB(B);

	for (; ItA.HasMore() && ItB.HasMore(); )
	{
		FOptionalMemberId NameA = ItA.PeekName();
		FOptionalMemberId NameB = ItB.PeekName();
		FMemberType TypeA = ItA.PeekType();
		FMemberType TypeB = ItB.PeekType();
		bool bDiff = TypeA != TypeB || NameA != NameB;
		if (!bDiff)
		{
			switch (TypeA.GetKind())
			{
			case EMemberKind::Leaf:
				bDiff = DiffLeaf(ItA.GrabLeaf(), ItB.GrabLeaf());
				break;
			case EMemberKind::Struct:
				bDiff = DiffMembers(ItA.GrabStruct(), ItB.GrabStruct(), Out);
				break;
			case EMemberKind::Range:
				bDiff = DiffRange(TypeA.AsRange().MaxSize, ItA.GrabRange(), ItB.GrabRange(), Out);
				break;
			}
		}
		if (bDiff)
		{
			Out.Emplace(TypeA, A.Schema.Id, NameA);
			return true;
		}
	}
	if (ItA.HasMore())
	{
		Out.Emplace(ItA.PeekType(), A.Schema.Id, ItA.PeekName());
		return true;
	}
	if (ItB.HasMore())
	{
		Out.Emplace(ItB.PeekType(), A.Schema.Id, ItB.PeekName());
		return true;
	}
	return false;
}

bool DiffStruct(FStructView A, FStructView B, FReadDiffPath& Out)
{
	return DiffMembers(A, B, Out);
}

bool operator==(const FDiffPath& A, const FDiffPath& B)
{
	if (A.Num() != B.Num())
	{
		return false;
	}
	else
	{
		for (int32 N = 0; N < A.Num(); ++N)
		{
			if (A[N].Name != B[N].Name)
			{
				return false;
			}
		}
	}

	return true;
}

uint32 GetTypeHash(const FDiffPath& Path)
{
	uint32 Hash = 0;

	for (const FDiffNode& Node : Path)
	{
		uint32 NameHash = GetTypeHash(Node.Name);
		Hash = HashCombineFast(Hash, NameHash);
	}

	return Hash;
}

} // namespace PlainProps
