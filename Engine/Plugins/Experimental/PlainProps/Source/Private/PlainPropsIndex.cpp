// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsIndex.h"
#include "Hash/xxhash.h"

namespace PlainProps
{

FNestedScopeId FNestedScopeIndexer::Index(FNestedScope Scope)
{
	uint32 Hash = GetTypeHash(Scope);
	FSetElementId Idx = Scopes.FindIdByHash(Hash, Scope);
	if (Idx == FSetElementId())
	{
		Idx = Scopes.AddByHash(Hash, Scope);
	}
	return { static_cast<uint32>(Idx.AsInteger()) };
}

FNestedScope FNestedScopeIndexer::Resolve(FNestedScopeId Id) const
{
	return Scopes[FSetElementId::FromInteger(Id.Idx)];
}

//////////////////////////////////////////////////////////////////////////

FParametricTypeView::FParametricTypeView(FConcreteTypenameId InName, TConstArrayView<FType> Params)
: FParametricTypeView(InName, IntCastChecked<uint8>(Params.Num()), Params.GetData())
{}

namespace ParametricTypeHash
{

static uint32 Calculate(FOptionalConcreteTypenameId Name, TConstArrayView<FType> Parameters)
{
	uint64 ParametersHash = FXxHash64::HashBuffer(Parameters.GetData(), sizeof(FType) * Parameters.Num()).Hash;
	return HashCombineFast(GetTypeHash(Name), static_cast<uint32>(ParametersHash));
}

static constexpr uint8 FreeSlotByte = 0xFF;
static constexpr uint32 FreeSlot = 0xFFFFFFFF;

static uint32* Rehash(uint32 NumSlots, TConstArrayView<FParametricType> Types, TConstArrayView<FType> Parameters)
{
	uint32* Slots = (uint32*)FMemory::Malloc(NumSlots * sizeof(uint32));
	FMemory::Memset(Slots, FreeSlotByte, NumSlots * sizeof(uint32));

	const uint32 SlotMask = NumSlots - 1;
	uint32 TypeIdx = 0;
	for (FParametricType Type : Types)
	{
		uint32 Hash = Calculate(Type.Name, Parameters.Slice(Type.Parameters.Idx, Type.Parameters.NumParameters));
		uint32 SlotIdx = Hash & SlotMask;
		while (Slots[SlotIdx] != FreeSlot)
		{
			SlotIdx = (SlotIdx + 1) & SlotMask;
		}
		Slots[SlotIdx] = TypeIdx;
		++TypeIdx;
	}

	return Slots;
}

}

FParametricTypeIndexer::~FParametricTypeIndexer()
{
	FMemory::Free(Slots);
}

FParametricTypeId FParametricTypeIndexer::Index(FParametricTypeView View)
{
	using namespace ParametricTypeHash;

	checkSlow(View.NumParameters);

	static constexpr uint32 MinSlack = 4;
	uint32 WantedSlots = FMath::RoundUpToPowerOfTwo((Types.Num() + MinSlack) * /* 90% load factor */ 10 / 9);
	if (WantedSlots > NumSlots)
	{
		FMemory::Free(Slots);
		Slots = Rehash(WantedSlots, Types, Parameters);
		NumSlots = WantedSlots;
	}

	const uint32 Hash = Calculate(View.Name, MakeArrayView(View.Parameters, View.NumParameters));
	uint32 SlotIdx = Hash & (NumSlots - 1);

	for (TArrayView<uint32> SlotsPart : { MakeArrayView(Slots + SlotIdx, NumSlots - SlotIdx), MakeArrayView(Slots, SlotIdx) } )
	{
		for (uint32& Slot : SlotsPart)
		{
			if (Slot == FreeSlot)
			{
				Slot = Types.Num();
				FParameterIndexRange ParameterIndices(View.NumParameters, Parameters.Num());
				Types.Add({View.Name, ParameterIndices});
				Parameters.Append(View.Parameters, View.NumParameters);
				
				return FParametricTypeId(View.NumParameters, Slot);
			}
			
			FParametricType Existing = Types[Slot];
			if (View.Name == Existing.Name &&
				View.NumParameters == Existing.Parameters.NumParameters &&
				!FMemory::Memcmp(View.Parameters, &Parameters[Existing.Parameters.Idx], View.NumParameters * sizeof(FType)))
			{
				return FParametricTypeId(View.NumParameters, Slot);
			}
		}
	}

	checkf(false, TEXT("No free slots despite MinSlack and load factor"));
	return FParametricTypeId(0,0);
}

FParametricTypeView FParametricTypeIndexer::Resolve(FParametricTypeId Id) const
{
	FParametricType Type = Types[Id.Idx];
	check(Id.NumParameters == Type.Parameters.NumParameters);
	return { Type.Name, static_cast<uint8>(Id.NumParameters), &Parameters[Type.Parameters.Idx] };
}

//////////////////////////////////////////////////////////////////////////

FScopeId FIdIndexerBase::NestFlatScope(FScopeId Outer, FFlatScopeId Inner)
{
	check(Outer);
	return FScopeId(NestedScopes.Index(Outer, Inner));
}

FScopeId FIdIndexerBase::NestReversedScopes(TConstArrayView<FFlatScopeId> Inners)
{
	FScopeId Out = NoId;
	for (FFlatScopeId Inner : ReverseIterate(Inners))
	{
		Out = Out ? NestFlatScope(Out, Inner) : FScopeId(Inner);
	}

	return Out;
}

FParametricTypeId FIdIndexerBase::MakeParametricTypeId(FOptionalConcreteTypenameId Name, TConstArrayView<FType> Params)
{
	return ParametricTypes.Index({Name, IntCastChecked<uint8>(Params.Num()), Params.GetData()});
}

FType FIdIndexerBase::MakeParametricType(FType Type, TConstArrayView<FType> Params)
{
	return {Type.Scope, FTypenameId(MakeParametricTypeId(Type.Name.AsConcrete(), Params))};
}

FType FIdIndexerBase::MakeAnonymousParametricType(TConstArrayView<FType> Params)
{
	return { NoId, FTypenameId(MakeParametricTypeId(NoId, Params)) };
}

FEnumId FIdIndexerBase::IndexEnum(FType Type)
{
	return { IntCastChecked<uint32>(Enums.Add(Type).AsInteger()) };
}

FStructId FIdIndexerBase::IndexStruct(FType Type)
{
	return { IntCastChecked<uint32>(Structs.Add(Type).AsInteger()) };
}

FType FIdIndexerBase::Resolve(FEnumId Id) const
{
	check(Id.Idx < uint32(Enums.Num()));
	return Enums.Get(FSetElementId::FromInteger(Id.Idx));
}

FType FIdIndexerBase::Resolve(FStructId Id) const
{
	check(Id.Idx < uint32(Structs.Num()));
	return Structs.Get(FSetElementId::FromInteger(Id.Idx));
}

} // namespace PlainProps