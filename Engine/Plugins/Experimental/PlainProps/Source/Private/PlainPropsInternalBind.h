// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsBind.h"
#include "PlainPropsVisitMember.h"

namespace PlainProps 
{

struct FMemberBinderBase
{
	FMemberBinderBase(FSchemaBinding& InSchema)
	: Schema(InSchema)
	, MemberIt(Schema.Members)
	, RangeTypeIt(const_cast<FMemberBindType*>(Schema.GetInnerRangeTypes()))
	, OffsetIt(const_cast<uint32*>(Schema.GetOffsets()))
	, RangeBindingIt(const_cast<FRangeBinding*>(Schema.GetRangeBindings()))
	{}

	~FMemberBinderBase()
	{
		check(MemberIt == Schema.GetInnerRangeTypes());
		check(Align(RangeTypeIt, alignof(uint32)) == (const void*)Schema.GetOffsets());
		check(OffsetIt == (const void*)Schema.GetInnerIds());
		check(Schema.NumInnerRanges == RangeBindingIt - Schema.GetRangeBindings());
	}

	void AddMember(FMemberBindType Type, uint32 Offset)
	{
		*MemberIt++ = Type;
		*OffsetIt++ = Offset;
	}

	void AddRange(TConstArrayView<FRangeBinding> Ranges, FMemberBindType InnermostType, uint32 Offset)
	{
		checkSlow(!Ranges.IsEmpty());
		AddMember(FMemberBindType(Ranges[0].GetSizeType()), Offset);

		for (FRangeBinding Range : Ranges.RightChop(1))
		{
			*RangeTypeIt++ = FMemberBindType(Range.GetSizeType());
		}
		*RangeTypeIt++ = InnermostType;

		FMemory::Memcpy(RangeBindingIt, Ranges.GetData(), Ranges.Num() * Ranges.GetTypeSize());
		RangeBindingIt += Ranges.Num();
	}
	
	FSchemaBinding& Schema;
	FMemberBindType* MemberIt;
	FMemberBindType* RangeTypeIt;
	uint32* OffsetIt;
	FRangeBinding* RangeBindingIt;
};


////////////////////////////////////////////////////////////////////////////////////////////////

[[nodiscard]] inline FMemberType ToMemberType(FMemberBindType In)
{
	switch (In.GetKind())
	{
	case EMemberKind::Leaf:		return FMemberType(ToLeafType(In.AsLeaf()));
	case EMemberKind::Range:	return FMemberType(In.AsRange());
	default:					return FMemberType(In.AsStruct());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////

// FSchemaId -> FInnerId lookup, see IndexAllRuntimeIds
struct FInnerIds
{
	TConstArrayView<FInnerId>		Ids;

	FInnerId						operator[](FSchemaId Schema) const					{ return Ids[Schema.Idx]; }
	FEnumId							operator[](FEnumSchemaId Schema) const				{ return Ids[Schema.Idx].AsEnum(); }
	FDeclId							operator[](FStructSchemaId Schema) const			{ return Ids[Schema.Idx].AsStructDeclId(); }
	FOptionalInnerId				operator[](FOptionalSchemaId Schema) const			{ return Schema ? ToOptional((*this)[Schema.Get()]) : NoId; }
	FOptionalEnumId					operator[](FOptionalEnumSchemaId Schema) const		{ return Schema ? ToOptional((*this)[Schema.Get()]) : NoId; }
	FOptionalDeclId					operator[](FOptionalStructSchemaId Schema) const	{ return Schema ? ToOptional((*this)[Schema.Get()]) : NoId; }
	
	int32							Num() const											{ return Ids.Num(); }
	TConstArrayView<FDeclId>		GetDeclIds(uint32 NumStructs) const					{ return MakeArrayView(reinterpret_cast<const FDeclId*>(Ids.GetData()), NumStructs); }
	TConstArrayView<FStructId>		GetStructIds(uint32 NumStructs) const				{ return MakeArrayView(reinterpret_cast<const FStructId*>(Ids.GetData()), NumStructs); }
	TConstArrayView<FEnumId>		GetEnumIds(uint32 NumEnums) const					{ return MakeArrayView(reinterpret_cast<const FEnumId*>(Ids.GetData() + Ids.Num() - NumEnums), NumEnums); }
};

} // namespace PlainProps