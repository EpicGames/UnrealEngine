// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsBuild.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"

namespace PlainProps
{

struct FBuiltStruct
{
	~FBuiltStruct() = delete; // Allocated in FScratchAllocator 
	
	uint16				NumMembers;
	FBuiltMember		Members[0];
};

struct FBuiltRange
{
	~FBuiltRange() = delete; // Allocated in FScratchAllocator

	[[nodiscard]] static FBuiltRange*					Create(FScratchAllocator& Scratch, uint64 NumItems, SIZE_T ItemSize);
	[[nodiscard]] static FBuiltRange*					Create(FScratchAllocator& Scratch, uint64 NumItems, SIZE_T ItemSize, const void* Items);
	[[nodiscard]] static FBuiltRange*					CreateIf(FScratchAllocator& Scratch, uint64 NumItems, SIZE_T ItemSize, const void* Items) { return NumItems ? Create(Scratch, NumItems, ItemSize, Items) : nullptr; }

	uint64												Num;
	uint8												Data[0];
	
	TConstArrayView64<const FBuiltRange*>				AsRanges() const	{ return { reinterpret_cast<FBuiltRange const* const*>(Data),	static_cast<int64>(Num) }; }
	TConstArrayView64<const FBuiltStruct*>				AsStructs() const	{ return { reinterpret_cast<FBuiltStruct const* const*>(Data),	static_cast<int64>(Num) }; }
};


} // namespace PlainProps