// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsBind.h"
#include "PlainPropsTypes.h"
#include "Math/PreciseFP.h"

namespace PlainProps 
{

inline const uint8* At(const void* Ptr, SIZE_T Offset)
{
	return static_cast<const uint8*>(Ptr) + Offset;
}

//  Works with FBoolRangeView, which lack GetData(), TRangeView, TArrayView, std::initializer_list and T[]
template<class RangeTypeA, class RangeTypeB>
bool DiffItems(RangeTypeA&& A, RangeTypeB&& B, uint64& OutIdx)
{
	uint64 NumA = IntCastChecked<uint64>(GetNum(A));
	uint64 NumB = IntCastChecked<uint64>(GetNum(B));
	OutIdx = 0;
	auto BIt = std::begin(B);
	for (auto bA : A)
	{
		if (OutIdx >= NumB)
		{
			break;
		}
		if (bA != *BIt++)
		{
			return true;
		}
		++OutIdx;
	}
	return NumA != NumB;
}

inline int64 DiffLeaf(const uint8* A, const uint8* B, FUnpackedLeafBindType Leaf)
{
	bool bBitfield = Leaf.Type == ELeafBindType::BitfieldBool;
	bool bFloat = Leaf.Type == ELeafBindType::Float;
	if (bBitfield | bFloat)
	{
		if (bBitfield)
		{
			return ((*A ^ *B) >> Leaf.BitfieldIdx) & 1;
		}
		
		switch (Leaf.Width)
		{
		case ELeafWidth::B32:	return !::UE::PreciseFPEqual(reinterpret_cast<const float&>(*A), reinterpret_cast<const float&>(*B));
		case ELeafWidth::B64:	return !::UE::PreciseFPEqual(reinterpret_cast<const double&>(*A), reinterpret_cast<const double&>(*B));
		default:				break;
		}

	}
	else
	{
		switch (Leaf.Width)
		{
		case ELeafWidth::B8:	return FMemory::Memcmp(A, B, 1);
		case ELeafWidth::B16:	return FMemory::Memcmp(A, B, 2);
		case ELeafWidth::B32:	return FMemory::Memcmp(A, B, 4);
		case ELeafWidth::B64:	return FMemory::Memcmp(A, B, 8);
		}
	}

	unimplemented();
	return 0;
}

inline ELeafWidth GetItemWidth(FLeafBindType Leaf)
{
	checkf(Leaf.Bind.Type != ELeafBindType::BitfieldBool, TEXT("Range of bitfields is illegal"));
	return Leaf.Basic.Width;
}

////////////////////////////////////////////////////////////////////////////////////////////////

// Helps diff ranges
struct FItemRangeReader
{
	explicit FItemRangeReader(const void* Range, const IItemRangeBinding& Binding)
	: Ctx{{Range}}
	{
		ReadItems(Binding);
	}

	FSaveRangeContext Ctx;
	const uint8* SliceIt = nullptr;
	uint64 SliceNum = 0;

	void ReadItems(const IItemRangeBinding& Binding)
	{
		Binding.ReadItems(Ctx);
		SliceIt = static_cast<const uint8*>(Ctx.Items.Slice.Data);
		SliceNum = Ctx.Items.Slice.Num;
	}

	void RefillItems(const IItemRangeBinding& Binding)
	{
		if (SliceNum == 0)
		{
			ReadItems(Binding);
			check(SliceNum > 0);
		}
	}

	const uint8* GrabItems(uint64 Num, uint32 Stride)
	{
		const uint8* Out = SliceIt;
		SliceIt += Num * Stride; 
		SliceNum -= Num;
		return Out;
	}
};

template<class ContextType>
inline bool DiffItemSlice(const uint8* A, const uint8* B, uint64 Num, uint32 Stride, ContextType&, SIZE_T LeafSize)
{
	return !!FMemory::Memcmp(A, B, Num * Stride);
}

template<typename ItemType, class ContextType>
bool DiffItemSlice(const uint8* A, const uint8* B, uint64 Num, uint32 Stride, ContextType& Ctx, ItemType&& Member)
{
	for (const uint8* EndA = A + Num * Stride; A != EndA; A += Stride, B += Stride)
	{
		if (DiffItem(A, B, Ctx, Member))
		{
			// Only gather first item even in EDiffGather::All to avoid a bloated diff.
			return true;
		}
	}

	return false;
}

template<class T, class ContextType>
bool DiffItemRange(const void* RangeA, const void* RangeB, const IItemRangeBinding& Binding, ContextType& OuterCtx, T&& ItemSchema)
{
	FItemRangeReader A(RangeA, Binding);
	FItemRangeReader B(RangeB, Binding);
	if (A.Ctx.Items.NumTotal != B.Ctx.Items.NumTotal)
	{
		return true;
	}

	if (const uint64 NumTotal = A.Ctx.Items.NumTotal)
	{
		check(A.Ctx.Items.Stride == B.Ctx.Items.Stride);
		const uint32 Stride = A.Ctx.Items.Stride;
		while (true)
		{
			uint64 Num = FMath::Min(A.SliceNum, B.SliceNum);
			if (DiffItemSlice(A.GrabItems(Num, Stride), B.GrabItems(Num, Stride), Num, Stride, OuterCtx, ItemSchema))
			{
				return true;
			}
			else if (A.Ctx.Request.NumRead + Num >= NumTotal)
			{
				check(A.Ctx.Request.NumRead + Num == NumTotal);	
				check(B.Ctx.Request.NumRead + Num == NumTotal);	
				return false;
			}
			
			A.Ctx.Request.NumRead += Num;
			B.Ctx.Request.NumRead += Num;
			A.RefillItems(Binding);
			B.RefillItems(Binding);
		}
	}

	return false;
}

} // namespace PlainProps
