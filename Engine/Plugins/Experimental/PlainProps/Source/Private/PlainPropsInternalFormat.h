// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsTypes.h"

namespace PlainProps
{

struct FSchemaBatchIterator;

struct FSchemaBatch
{
	uint32 NumNestedScopes;
	uint32 NestedScopesOffset;
	uint32 NumParametricTypes;
	uint32 NumSchemas;
	uint32 NumStructSchemas;
	uint32 SchemaOffsets[0];

	TConstArrayView<uint32> GetSchemaOffsets() const
	{
		return MakeArrayView(SchemaOffsets, NumSchemas);
	}

	TConstArrayView<FNestedScope> GetNestedScopes() const
	{
		return MakeArrayView(reinterpret_cast<const FNestedScope*>(uintptr_t(this) + NestedScopesOffset), NumNestedScopes);
	}

	TConstArrayView<FParametricType> GetParametricTypes() const
	{
		return MakeArrayView(reinterpret_cast<const FParametricType*>(GetNestedScopes().end()), NumParametricTypes);
	}

	const FType* GetFirstParameter() const
	{
		return reinterpret_cast<const FType*>(GetParametricTypes().end());
	}

	void ValidateBounds(uint64 NumBytes) const;
};

static_assert(sizeof(FSchemaBatch) == 20 && alignof(FSchemaBatch) == 4,			"Add binary format versioning and read old format in AllocateReadSchemas");
static_assert(offsetof(FSchemaBatch, NumNestedScopes) == 0,						"Add binary format versioning and read old format in AllocateReadSchemas");
static_assert(offsetof(FSchemaBatch, NestedScopesOffset) == 4,					"Add binary format versioning and read old format in AllocateReadSchemas");
static_assert(offsetof(FSchemaBatch, NumParametricTypes) == 8,					"Add binary format versioning and read old format in AllocateReadSchemas");
static_assert(offsetof(FSchemaBatch, NumSchemas) == 12,							"Add binary format versioning and read old format in AllocateReadSchemas");
static_assert(offsetof(FSchemaBatch, NumStructSchemas) == 16,					"Add binary format versioning and read old format in AllocateReadSchemas");

//////////////////////////////////////////////////////////////////////////

// Direct super schema id is always written to enable inheritance analysis
// 
// No		No inheritance
// Unused	Super member missing, due to all instances being empty
// Used		Super member exists, but IsDynamic or with a static non-direct ancestor id
// Reused	Super member exists, with the static id of the direct super struct
enum class ESuper : uint8 { No, Unused, Used, Reused };

inline bool UsesSuper(ESuper Inheritance)				{ return Inheritance == ESuper::Used   || Inheritance == ESuper::Reused; }
inline bool SkipDeclaredSuperSchema(ESuper Inheritance)	{ return Inheritance == ESuper::Unused || Inheritance == ESuper::Used; }

struct FStructSchema
{
	FType Type;
	uint16 Version;
	uint16 NumMembers;
	uint16 NumRangeTypes;
	uint16 NumInnerSchemas;
	ESuper Inheritance : 2;
	uint8 IsDense : 1;
	FMemberType Footer[0];
	
	inline uint32					NumNames() const			{ checkSlow(NumMembers > 0 || !UsesSuper(Inheritance)); return NumMembers - UsesSuper(Inheritance); }
	TConstArrayView<FMemberType>	GetMemberTypes() const		{ return MakeArrayView(GetMemberTypes(Footer), NumMembers); }
	TConstArrayView<FMemberType>	GetRangeTypes() const		{ return MakeArrayView(GetRangeTypes(Footer, NumMembers), NumRangeTypes); }
	TConstArrayView<FMemberId>		GetMemberNames() const		{ return MakeArrayView(GetMemberNames(Footer, NumMembers, NumRangeTypes), NumNames()); }
	TArrayView<FMemberId>			EditMemberNames() 			{ return MakeArrayView(const_cast<FMemberId*>(GetMemberNames(Footer, NumMembers, NumRangeTypes)), NumNames()); }
	const FSchemaId*				GetInnerSchemas() const		{ return GetInnerSchemas(Footer, NumMembers, NumRangeTypes, NumNames()); }
	FOptionalStructSchemaId			GetSuper() const			{ return Inheritance != ESuper::No ? ToOptional(static_cast<FStructSchemaId>(*GetInnerSchemas())) : NoId; }

	static const FMemberType*		GetMemberTypes(const FMemberType* Footer)
	{
		return Footer;
	}
	
	static const FMemberType*		GetRangeTypes(const FMemberType* Footer, uint32 NumMembers)
	{
		return Footer + NumMembers;
	}

	static const FMemberId*			GetMemberNames(const FMemberType* Footer, uint32 NumMembers, uint32 NumRangeTypes)
	{
		return AlignPtr<FMemberId>(Footer + NumMembers + NumRangeTypes);
	}
		
	static const FSchemaId*			GetInnerSchemas(const FMemberType* Footer, uint32 NumMembers, uint32 NumRangeTypes, uint32 NumNames)
	{
		return AlignPtr<FSchemaId>(GetMemberNames(Footer, NumMembers, NumRangeTypes) + NumNames);
	}
};

static_assert(sizeof(FStructSchema) == 20 && alignof(FStructSchema) == 4,		"Add binary format versioning and read old format in AllocateReadSchemas");
static_assert(offsetof(FStructSchema, Type) == 0,								"Add binary format versioning and read old format in AllocateReadSchemas");
static_assert(offsetof(FType, Scope) == 0,										"Add binary format versioning and read old format in AllocateReadSchemas");
static_assert(offsetof(FType, Name) == 4,										"Add binary format versioning and read old format in AllocateReadSchemas");
static_assert(offsetof(FStructSchema, Version) == 8,							"Add binary format versioning and read old format in AllocateReadSchemas");
static_assert(offsetof(FStructSchema, NumMembers) == 10,						"Add binary format versioning and read old format in AllocateReadSchemas");
static_assert(offsetof(FStructSchema, NumRangeTypes) == 12,						"Add binary format versioning and read old format in AllocateReadSchemas");
static_assert(offsetof(FStructSchema, NumInnerSchemas) == 14,					"Add binary format versioning and read old format in AllocateReadSchemas");
static_assert(offsetof(FStructSchema, Footer) == 17,							"Add binary format versioning and read old format in AllocateReadSchemas");

inline uint32 CalculateSize(const FStructSchema& In)
{
	static_assert(alignof(FMemberType) == 1);
	uint32 Out = offsetof(FStructSchema, Footer) + sizeof(FMemberType) * (uint32(In.NumMembers) + In.NumRangeTypes);
	Out = Align(Out + In.NumNames() * sizeof(FMemberId), sizeof(FMemberId));
	Out = Align(Out + (In.NumInnerSchemas) * sizeof(FSchemaId), sizeof(FSchemaId));
	return Out;
}

//////////////////////////////////////////////////////////////////////////

struct FEnumSchema
{
	FType Type;
	uint8 FlagMode : 1;
	uint8 ExplicitConstants : 1; // 0,1,2,3,4.. or 1,2,4,8..
	ELeafWidth Width;
	uint16 Num;
	FNameId Footer[0];
};

static_assert(sizeof(FEnumSchema) == 12 && alignof(FEnumSchema) == 4,	"Add binary format versioning and read old format in AllocateReadSchemas");
static_assert(offsetof(FEnumSchema, Type) == 0,							"Add binary format versioning and read old format in AllocateReadSchemas");
static_assert(offsetof(FEnumSchema, Width) == 9,						"Add binary format versioning and read old format in AllocateReadSchemas");
static_assert(offsetof(FEnumSchema, Num) == 10,							"Add binary format versioning and read old format in AllocateReadSchemas");
static_assert(offsetof(FEnumSchema, Footer) == 12,						"Add binary format versioning and read old format in AllocateReadSchemas");

inline uint32 CalculateSize(const FEnumSchema& In)
{
	return Align(offsetof(FEnumSchema, Footer) + In.Num * sizeof(In.Footer[0]) + In.ExplicitConstants * In.Num * SizeOf(In.Width), alignof(FEnumSchema));
}

template<typename T>
TConstArrayView<T> GetConstants(const FEnumSchema& In) requires (UnsignedIntegral<T> || Enumeration<T>)
{
	check(In.Width == ReflectLeaf<T>.Width);
	return MakeConstArrayView<T>(reinterpret_cast<const T*>(In.Footer + In.Num), In.ExplicitConstants ? In.Num : 0);
}
//////////////////////////////////////////////////////////////////////////

inline bool IsEnum(FMemberType Type)
{
	return Type.GetKind() == EMemberKind::Leaf && ELeafType::Enum == Type.AsLeaf().Type;
}

inline bool IsStructOrEnum(FMemberType Type)
{
	return Type.IsStruct() || IsEnum(Type);
}

inline bool IsSuper(FMemberType Type)
{
	return Type.IsStruct() && Type.AsStruct().IsSuper;
}

inline constexpr uint64 GetLeafRangeSize(uint64 Num, FUnpackedLeafType Leaf)
{
	return Leaf.Type == ELeafType::Bool ? (Num + 7) / 8 : Num * SizeOf(Leaf.Width);
}

//////////////////////////////////////////////////////////////////////////

} // namespace PlainProps
