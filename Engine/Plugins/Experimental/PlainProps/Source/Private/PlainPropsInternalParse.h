// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsParse.h"
#include "PlainPropsDeclare.h"
#include "PlainPropsIndex.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/StringView.h"
#include "Misc/Optional.h"
#include "Templates/RefCounting.h"
#include "Templates/Tuple.h"
#include "Templates/UniquePtr.h"
#include "Hash/xxhash.h"

namespace PlainProps
{

class FScratchAllocator;
class FMarkupTokenizer;
struct FBuiltStruct;
struct FEnumerator;

///////////////////////////////////////////////////////////////////////////////

enum class EToken : uint8
{
	Invalid,
	BeginDocument,
	EndDocument,
	BeginStruct,
	EndStruct,
	BeginRange,
	EndRange,
	LeafId,
	LeafValue,
	Leaf,
};

struct FToken
{ 
	FToken() = default;

	FToken(EToken InToken, uint16 InDepth) : Token(InToken), Depth(InDepth) { }

	FToken(EToken InToken, uint16 InDepth, FUtf8StringView InView)
		: Str(InView.GetData()), Len(InView.Len()), Token(InToken), Depth(InDepth)
	{ }

	inline FUtf8StringView Value() const { return FUtf8StringView(Str, Len); }

	inline bool operator==(FToken O) const
	{
		return Token == O.Token && Depth == O.Depth && Value() == O.Value();
	}

	const UTF8CHAR*					Str		= nullptr;
	int32							Len		= 0;
	EToken							Token	= EToken::Invalid;
	uint16							Depth	= 0;
};

///////////////////////////////////////////////////////////////////////////////

struct FParsedMemberSchema
{
	FMemberType						Type;
	FOptionalInnerId				InnerSchema;
	TArray<FMemberType>				InnerRangeTypes;
};

struct FParsedStructSchema
{
	FDeclId							Id;
	uint16							Version = 0;
	FOptionalDeclId					DeclaredSuper;
	FOptionalDeclId					UsedSuper;
	TConstArrayView<FMemberId>		MemberNames;	// FStructDeclaration.MemberOrder
	TArray<FParsedMemberSchema>		MemberSchemas;	// Same size as declared member names
};

struct FParsedEnumSchema
{
	FEnumId							Id;
	ELeafWidth						Width;
	TConstArrayView<FEnumerator>	Enumerators;	// FEnumDeclaration.Enumerators
};

///////////////////////////////////////////////////////////////////////////////

struct FParsedSchemas
{
	TArray<FParsedStructSchema>		Structs;		// Same size as number of parsed and declared structs
	TArray<FParsedEnumSchema>		Enums;			// Same size as number of parsed and declared enums
};

///////////////////////////////////////////////////////////////////////////////
class FSensitiveUtf8View : public FUtf8StringView
{
public:
	FSensitiveUtf8View(FUtf8StringView View) : FUtf8StringView(View) {}

	friend inline bool operator==(FSensitiveUtf8View Lhs, FSensitiveUtf8View Rhs)
	{
		return Lhs.Equals(Rhs, ESearchCase::CaseSensitive);
	}
	friend inline uint32 GetTypeHash(FSensitiveUtf8View View)
	{
		return static_cast<uint32>(FXxHash64::HashBuffer(View.GetData(), View.Len()).Hash);
	}
	friend inline bool operator!=(FSensitiveUtf8View Lhs, FSensitiveUtf8View Rhs)
	{
		return !operator==(Lhs, Rhs);
	}
	friend inline bool operator<(FSensitiveUtf8View Lhs, FSensitiveUtf8View Rhs)
	{
		return Lhs.Compare(Rhs, ESearchCase::IgnoreCase) < 0;
	}
};

class FTextIndexer : public TIdIndexer<FSensitiveUtf8View>
{
public:
	FOptionalMemberId GetMemberId(FUtf8StringView Name)
	{
		FSetElementId Id = Names.FindId(Name);
		if (Id.IsValidId())
		{
			return FMemberId{ IntCastChecked<uint32>(Id.AsInteger()) };
		}
		return NoId;
	}

	FOptionalDeclId GetStructId(FType Type) const
	{
		FSetElementId Id = Structs.FindId(Type);
		if (Id.IsValidId())
		{
			return FDeclId{ IntCastChecked<uint32>(Id.AsInteger()) };
		}
		return NoId;
	}

	FOptionalEnumId GetEnumId(FType Type) const
	{
		FSetElementId Id = Enums.FindId(Type);
		if (Id.IsValidId())
		{
			return FEnumId{ IntCastChecked<uint32>(Id.AsInteger()) };
		}
		return NoId;
	}
};

///////////////////////////////////////////////////////////////////////////////

struct FTypeTokens
{
	FUtf8StringView					Typename;
	TArray<FUtf8StringView>			Scopes;
	TArray<FTypeTokens>				Parameters;
};

TOptional<FTypeTokens> TokenizeType(FUtf8StringView String);

FType MakeType(const FTypeTokens& TypeTokens, FTextIndexer& Names);

///////////////////////////////////////////////////////////////////////////////

using FStructDeclarations = TArray<FStructDeclarationPtr>;

struct FParseDeclarations final : IDeclarations
{
	explicit FParseDeclarations(FDebugIds Dbg) : Enums(Dbg) {}
	
	const FEnumDeclaration&				Get(FEnumId Id) { return Enums.Get(Id); }
	const FStructDeclaration&			Get(FDeclId Id) { return *Structs[Id.Idx]; }

	virtual FDeclId						Lower(FBindId Id) const override final;
	virtual const FEnumDeclaration*		Find(FEnumId Id) const override { return Enums.Find(Id); }
	virtual const FStructDeclaration*	Find(FStructId Id) const override;

	FEnumDeclarations					Enums;
	FStructDeclarations					Structs;
};

class FBatchParser
{
public:
	FBatchParser(FMarkupTokenizer& InTokenizer, FScratchAllocator& InScratch)
	: Tokenizer(InTokenizer), Scratch(InScratch), Types(FDebugIds(Names))
	{}

	void Parse(TArray64<uint8>& Out);

private:
	void Tokenize();
	void ParseAll();
	void ParseStructSchemas(TConstArrayView64<FToken> TokensView);
	void ParseEnumSchemas(TConstArrayView64<FToken> TokensView);
	void ParseObjects(TConstArrayView64<FToken> TokensView);
	void Write(TArray64<uint8>& Out);

	FMarkupTokenizer&								Tokenizer;
	FScratchAllocator&								Scratch;
	FTextIndexer									Names;
	FParseDeclarations								Types;
	TArray<TTuple<FStructId, FBuiltStruct*>>		Objects;
	FParsedSchemas									Schemas;
	TArray64<FToken>								Tokens;
	int64											EnumsIdx   = -1;
	int64											StructsIdx = -1;
	int64											ObjectsIdx = -1;
};

} // namespace PlainProps
