// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/StringConv.h"
#include "Misc/StringBuilder.h"
#include "PlainPropsPrint.h"
#include "Templates/UniquePtr.h"

namespace PlainProps
{

class FMarkupBuilder;
struct FSchemaBatch;
struct FEnumSchema;
struct FStructSchema;

inline TStringBuilderWithBuffer<TCHAR, 128> Print(FUtf8StringView View)
{
	return WriteToString<128>(FStringView(StringCast<TCHAR>(View.GetData(), View.Len())));
}

///////////////////////////////////////////////////////////////////////////////

struct FMarkupBuilderDeleter
{
	PLAINPROPS_API void operator()(FMarkupBuilder* Ptr) const;
};
using FYamlBuilderPtr = TUniquePtr<FMarkupBuilder, FMarkupBuilderDeleter>;
using FJsonBuilderPtr = TUniquePtr<FMarkupBuilder, FMarkupBuilderDeleter>;

[[nodiscard]] FYamlBuilderPtr MakeYamlBuilder(FUtf8StringBuilderBase& StringBuilder);
[[nodiscard]] FJsonBuilderPtr MakeJsonBuilder(FUtf8StringBuilderBase& StringBuilder);

///////////////////////////////////////////////////////////////////////////////

class FBatchPrinter
{
public:
	FBatchPrinter(FMarkupBuilder& InMarkupBuilder, const FBatchIds& InIds);
	~FBatchPrinter();

	void PrintSchemas();
	void PrintObjects(TConstArrayView<FStructView> Structs);

private:
	void PrintStructSchema(const FStructSchema& Struct, FSchemaBatchId BatchId);
	void PrintEnumSchema(const FEnumSchema& Enum);

	template<typename IntType>
	void PrintEnumConstants(TConstArrayView<FNameId> EnumNames, TConstArrayView<IntType> Constants, bool bFlagMode);

	FMarkupBuilder& MarkupBuilder;
	const FBatchIds& Ids;
};

///////////////////////////////////////////////////////////////////////////////

} // namespace PlainProps
