// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsTypes.h"
#include "PlainPropsRead.h"
#include "PlainPropsInternalFormat.h"
#include <type_traits>

namespace PlainProps
{
	
inline FUnpackedLeafType	GetType(FLeafRangeView Range)		{ return { Range.Type, Range.Width }; }
inline const uint8*			GetValues(FLeafRangeView Range)		{ return Range.Values; }
inline FRangeSchema			GetSchema(FRangeView Range)			{ return Range.Schema; }
inline FMemoryView			GetValues(FRangeView Range)			{ return Range.Values; }

inline FOptionalEnumSchemaId GetEnumSchemaId(FLeafRangeView Range)
{
	return Range.Enum;
}

template<typename MemberType, typename IndexType>
TConstArrayView<MemberType> GrabInnerRangeTypes(TConstArrayView<MemberType> InnerRangeTypes, IndexType& InOutIdx)
{
	const int32 StartIdx = InOutIdx;
	int32 Idx = StartIdx;
	while (InnerRangeTypes[Idx++].IsRange());
	
	InOutIdx = static_cast<IndexType>(Idx);
	return MakeArrayView(&InnerRangeTypes[StartIdx], Idx - StartIdx);
}

inline uint64 GrabRangeNum(ESizeType MaxSize, FByteReader& ByteIt, FBitCacheReader& BitIt)
{
	switch(MaxSize)
	{
	case ESizeType::Uni:	return BitIt.GrabNext(/* in-out */ ByteIt) ? 1 : 0;
	case ESizeType::S8:		return IntCastChecked<uint64>(	ByteIt.Grab<int8>());
	case ESizeType::U8:		return							ByteIt.Grab<uint8>();
	case ESizeType::S16:	return IntCastChecked<uint64>(	ByteIt.Grab<int16>());
	case ESizeType::U16:	return							ByteIt.Grab<uint16>();
	case ESizeType::S32:	return IntCastChecked<uint64>(	ByteIt.Grab<int32>());
	case ESizeType::U32:	return							ByteIt.Grab<uint32>();
	case ESizeType::S64:	return IntCastChecked<uint64>(	ByteIt.Grab<int64>());
	case ESizeType::U64:	return							ByteIt.Grab<uint64>();
	}
	check(false);
	return 0;
}

inline FMemoryView GrabRangeValues(uint64 Num, FMemberType InnerType, /* in-out */ FByteReader& ByteIt)
{
	if (Num == 0)
	{
		return FMemoryView();
	}

	uint64 NumBytes = InnerType.IsLeaf() ? GetLeafRangeSize(Num, InnerType.AsLeaf()) : ByteIt.GrabVarIntU();
	return ByteIt.GrabSlice(NumBytes);
}

inline FMemberType GetInnermostType(FRangeSchema Schema)
{
	FMemberType Inner = Schema.ItemType;
	for (const FMemberType* It = Schema.NestedItemTypes; Inner.IsRange(); Inner = *It++) {}
	return Inner;
}

//////////////////////////////////////////////////////////////////////////

template<class T>
struct TSchemaIterator
{
	uintptr_t						Base;
	const uint32*					OffsetIt;
	
	void							operator++()							{ ++OffsetIt; }
	bool							operator!=(TSchemaIterator O) const		{ return OffsetIt != O.OffsetIt; }
	T&								operator*() const						{ return *reinterpret_cast<T*>(Base + *OffsetIt); }
};

template<class T>
struct TSchemaRange
{
	static constexpr bool bStructSchema = std::is_convertible_v<T, FStructSchema>;

	TSchemaRange(std::conditional_t<std::is_const_v<T>, const FSchemaBatch&, FSchemaBatch&> Batch)
	: Base(reinterpret_cast<uintptr_t>(&Batch))
	, Offsets(Batch.SchemaOffsets + (bStructSchema ? 0 : Batch.NumStructSchemas), bStructSchema ? Batch.NumStructSchemas : Batch.NumSchemas - Batch.NumStructSchemas)
	{
		static_assert(bStructSchema ^ std::is_convertible_v<T, FEnumSchema>);
	}
	
	TSchemaIterator<T>				begin() const					{ return { Base, Offsets.begin() }; }
	TSchemaIterator<T>				end() const						{ return { Base, Offsets.end() }; }
	T&								First() const					{ return *reinterpret_cast<T*>(Base + Offsets[0]); }
	T&								Last() const					{ return *reinterpret_cast<T*>(Base + Offsets.Last()); }
	T&								operator[](uint32 Idx) const	{ return *reinterpret_cast<T*>(Base + Offsets[Idx]); }
	int32							Num() const						{ return Offsets.Num(); }

	uintptr_t						Base;
	TConstArrayView<uint32>			Offsets;
};

inline TSchemaRange<FEnumSchema>			GetEnumSchemas(FSchemaBatch& Batch)				{ return { Batch }; }
inline TSchemaRange<const FEnumSchema>		GetEnumSchemas(const FSchemaBatch& Batch)		{ return { Batch }; }
inline TSchemaRange<FStructSchema>			GetStructSchemas(FSchemaBatch& Batch)			{ return { Batch }; }
inline TSchemaRange<const FStructSchema>	GetStructSchemas(const FSchemaBatch& Batch)		{ return { Batch }; }

//////////////////////////////////////////////////////////////////////////

const FSchemaBatch& GetReadSchemas(FSchemaBatchId Batch);

inline const FStructSchema& ResolveStructSchema(const FSchemaBatch& Batch, FSchemaId Id)
{
	check(Id.Idx < Batch.NumSchemas);
	return *reinterpret_cast<const FStructSchema*>(reinterpret_cast<const uint8*>(&Batch) + Batch.SchemaOffsets[Id.Idx]);
}

inline const FEnumSchema& ResolveEnumSchema(const FSchemaBatch& Batch, FSchemaId Id)
{
	check(Id.Idx < Batch.NumSchemas);
	return *reinterpret_cast<const FEnumSchema*>(reinterpret_cast<const uint8*>(&Batch) + Batch.SchemaOffsets[Id.Idx]);
}

inline FNestedScope ResolveNestedScope(const FSchemaBatch& Batch, FNestedScopeId Id)
{
	return Batch.GetNestedScopes()[Id.Idx];
}

inline FParametricTypeView ResolveParametricType(const FSchemaBatch& Batch, FParametricTypeId Id)
{
	FParametricType Type = Batch.GetParametricTypes()[Id.Idx];
	return {Type.Name, IntCastChecked<uint8>(Type.Parameters.NumParameters), Batch.GetFirstParameter() + Type.Parameters.Idx};
}

//////////////////////////////////////////////////////////////////////////

inline FMemoryView GetSchemaData(const FSchemaBatch& Batch)
{
	check(Batch.NumSchemas > 0);
	const uint8* BatchPtr = reinterpret_cast<const uint8*>(&Batch);
	TConstArrayView<uint32> Offsets(Batch.SchemaOffsets, Batch.NumSchemas);
	
	bool bHasEnums = Batch.NumSchemas > Batch.NumStructSchemas;
	uint32 LastSchemaSize = bHasEnums	? CalculateSize(GetEnumSchemas(Batch).Last())
										: CalculateSize(GetStructSchemas(Batch).Last());
	return FMemoryView(BatchPtr + Offsets[0], Offsets.Last() + LastSchemaSize - Offsets[0]);
}

//////////////////////////////////////////////////////////////////////////

template<typename T>
auto GetEnd(T&& Container)
{
	return Container.GetData() + Container.Num();
}


} // namespace PlainProps
