// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsTypes.h"
#include "Containers/ContainersFwd.h"
#include "Memory/MemoryFwd.h"

namespace PlainProps
{

class FSchemaBatchId;
struct FStructView;

[[nodiscard]] FSchemaBatchId ParseYamlBatch(TArray64<uint8>& OutData, TArray<FStructView>& OutObjects, FUtf8StringView YamlView);
[[nodiscard]] FSchemaBatchId ParseJsonBatch(TArray64<uint8>& OutData, TArray<FStructView>& OutObjects, FUtf8StringView JsonView);

} // namespace PlainProps
