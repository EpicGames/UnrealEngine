// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsTypes.h"
#include "Containers/StringView.h"

namespace PlainProps
{

struct FLiterals
{
	// '!' is used in some literals so that they are not valid identifiers

	FAnsiStringView Structs = "Structs";
	FAnsiStringView Enums = "Enums";
	FAnsiStringView Objects = "Objects";

	FAnsiStringView Version = "Version";
	FAnsiStringView DeclaredSuper = "Super";
	FAnsiStringView Width = "Width";
	FAnsiStringView FlagMode = "FlagMode";
	FAnsiStringView Constants = "Constants";
	FAnsiStringView Members = "Members";

	FAnsiStringView True = "true";
	FAnsiStringView False = "false";

	FAnsiStringView Super = "__super__";
	FAnsiStringView Dynamic = "__dynamic__";

	FAnsiStringView UnsetFlagEnum = "!UnsetFlagEnum";

	FAnsiStringView Oob = "ERR!oob";

	FAnsiStringView Kinds[3] = { ("Leaf"), ("Struct"), ("Range") };

	FAnsiStringView Ranges[9] = { 
		("Uni"), ("i8"), ("u8"), ("i16"), ("u16"), ("i32"), ("u32"), ("i64"), ("u64") };

	FAnsiStringView RangeTypes[9] = { 
		("(Uni)"), ("(i8)"), ("(u8)"), ("(i16)"), ("(u16)"), ("(i32)"), ("(u32)"), ("(i64)"), ("(u64)") };

	FAnsiStringView Leaves[8][4] = {
		{("bool"),		("ERR!b16"),	("ERR!b32"),	("ERR!b64")},
		{("i8"),		("i16"),		("i32"),		("i64")},
		{("u8"),		("u16"),		("u32"),		("u64")},
		{("ERR!fp8"),	("ERR!fp16"),	("float"),		("double")},
		{("hex8"),		("hex16"),		("hex32"),		("hex64")},
		{("enum8"),		("enum16"),		("enum32"),		("enum64")},
		{("utf8"),		("utf16"),		("utf32"),		("ERR!utf64")},
		{Oob,			Oob,			Oob,			Oob}};

	FAnsiStringView Widths[4] = { ("u8"), ("u16"), ("u32"), ("u64") };
};
extern const FLiterals GLiterals;

} // namespace PlainProps
