// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsUpgrade.h"
#include "PlainPropsBuild.h"
#include "PlainPropsInternalBind.h"
#include "PlainPropsInternalRead.h"
#include "PlainPropsRead.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/StaticArray.h"

namespace PlainProps::Upgrade
{

class FEnumLoosener;
struct FEnumTransformation;
class FInstanceUpgrader;
struct FMatchHelper;
struct FOps;
struct FStructDecomposition;
struct FVersionUpgrades;

////////////////////////////////////////////////////////////////////////////////////////////////

struct FEnumeratorIndexer : private TSet<FNameId>
{
	FEnumeratorIndexer() {}
	FEnumeratorIndexer(TSet<FNameId>&& DeclaredOrder) : TSet<FNameId>(MoveTemp(DeclaredOrder)) {}

	FEnumeratorId		Index(FNameId Member)					{ return { IntCastChecked<uint16>(Add(Member).AsInteger()) }; }
	FNameId				Resolve(FEnumeratorId Parameter) const	{ return Get(FSetElementId::FromInteger(Parameter.Idx)); }
};

using FParameterIdSet = TSet<FOptionalMemberId>;
struct FParameterIndexer : private FParameterIdSet
{
	FParameterIndexer() {}
	FParameterIndexer(FParameterIdSet&& DeclaredOrder) : FParameterIdSet(MoveTemp(DeclaredOrder)){}

	FParameterId		Index(FMemberId Member)					{ return Index(ToOptional(Member)); }
	FParameterId		Index(FOptionalMemberId Member)			{ return { IntCastChecked<uint16>(Add(Member).AsInteger()) }; }
	FParameterId		IndexSuper()							{ return { IntCastChecked<uint16>(Add(NoId).AsInteger()) }; }
	
	//FParameterId		Reindex(FOptionalMemberId Member) const;
	FOptionalMemberId	Resolve(FParameterId Parameter) const	{ return Get(FSetElementId::FromInteger(Parameter.Idx)); }	
	uint32				Num() const								{ return uint32(TSet<FOptionalMemberId>::Num()); }
};

class FLooseIndexers
{
public:
	FLooseIndexers(const IDeclarations& Types) : Declarations(Types) {}

	FEnumeratorIndexer&			Get(FEnumId Id);
	FParameterIndexer&			Get(FDeclId Id);
	const FEnumeratorIndexer&	Get(FEnumId Id) const									{ return const_cast<FLooseIndexers&>(*this).Get(Id); }
	const FParameterIndexer&	Get(FDeclId Id) const									{ return const_cast<FLooseIndexers&>(*this).Get(Id); }
//	void						Drop(FEnumId Id)										{ Enums.Remove(Id); }
//	void						Drop(FDeclId Id)										{ Structs.Remove(Id); }

private:
	TMap<FDeclId, TUniquePtr<FParameterIndexer>>	Structs;
	TMap<FEnumId, TUniquePtr<FEnumeratorIndexer>>	Enums;
	const IDeclarations&							Declarations;
};

////////////////////////////////////////////////////////////////////////////////////////////////

struct FLooseParameter
{
	FParameterId						Name;
	FLooseType							Type;
};

////////////////////////////////////////////////////////////////////////////////////////////////

// ~FLooseParameter with out parameter specific metadata
struct FOutParameter
{
	FParameterId						Name;
	uint8								NumInners = 0;	// Named struct members or enumerators/flags
	uint8								RecompIdx = 0;	// Only for enums with NumInners > 0
	FLooseType							Type;
};

////////////////////////////////////////////////////////////////////////////////////////////////

struct FLooseSchema
{
	uint16 								Version = 0;
	uint16								NumMembers = 0;
	FLooseParameter						Members[0]; // Including super

	TConstArrayView<FLooseParameter>	GetMembers() const { return MakeArrayView(Members, NumMembers); }
};

//////////////////////////////////////////////////////////////////////////////////////////////////

// FLooseMetadata::IsSet, IsFullyUpgraded and !!InnermostSchema
enum class ELooseState : uint8 { Unset, Source, Loose, UpgradedLoose };

// Struct-specific loose member
struct FStructMember
{
	FDeclId					Id;
	FOptionalStructSchemaId	Schema;	// @invariant !!Schema == (State == ELooseState::Source)
	FParameterId			Name;
	uint16					Version;
	ELooseState				State;
	EMismatch				Mismatch;
	uint64					Num;	// Num loose members or FByteReader::CheckableBytes (0 in shipping config)
	FStructData				Data;
};

// Enum-specific loose member
struct FEnumMember
{
	FEnumId					Id;
	FOptionalEnumSchemaId	Schema;
	FParameterId			Name;
	uint16					Version;
	ELooseState				State;
	uint8					FlagMode : 1;
	uint16					Num;
	FEnumData				Data;
};

// Range-specific loose member
struct FRangeMember
{
	FLooseType				Type;
	FLooseMetadata			Meta;
	FRangeData				Data;
};

////////////////////////////////////////////////////////////////////////////////////////////////

// Nameless loosened struct
struct FLooseStruct
{
	FDeclId					Id;
	uint16					Version;
	uint16					Num;
	FLooseMember*			Members;
};

////////////////////////////////////////////////////////////////////////////////////////////////

class FEnumLooseners
{
	const FEnumTransformation*		Data; 
	uint32							NumStructs;
	uint32							NumSchemas;

public:
	FEnumLooseners(TConstArrayView<FEnumTransformation> Xforms, uint32 InNumStructs);
	const FEnumLoosener&			operator[](FEnumSchemaId Id) const;
};

////////////////////////////////////////////////////////////////////////////////////////////////

// Source struct reader producing loose members
class FLooseMemberReader : protected FMemberReader
{
public:
	FLooseMemberReader(const FLooseSchema& LooseSchema, FStructView Source, FEnumLooseners InLooseners, FScratchAllocator& InScratch);

	using FMemberReader::			HasMore;
	FLooseParameter					Peek() const;		//@pre HasMore()
	FLooseValue						Grab();				//@pre HasMore()

private:
	const FLooseParameter*			Parameters;
	FScratchAllocator&				Scratch;
	FEnumLooseners					Looseners;

	FRangeMember					ToLooseRange(FParameterId Name, FLooseType Type, FRangeView Range);
	FLooseNestedRange*				LoosenRanges(FNestedRangeView Range);
};

////////////////////////////////////////////////////////////////////////////////////////////////

class FOptionalVersion
{
	uint16 No = 0xFFFF;
public:
	FOptionalVersion() = default;
	FOptionalVersion(FNullOpt) {}
	FOptionalVersion(uint16 Version) : No(Version) { check(*this); }

	explicit operator	bool() const	{ return No != 0xFFFF; }
	uint16				Get() const		{ check(*this); return No; }
};

////////////////////////////////////////////////////////////////////////////////////////////////

// Small index range, max 255 items of a max 16.7M item range
struct FMiniSlice
{
	FMiniSlice(uint32 I, uint32 N) : Idx(I), Num(N) { check(Idx == I && Num == N); }

	uint32 		Idx : 24;
	uint32		Num : 8;
};

// Compiled op and metadata
struct FOp
{
	Op 			Function;
	FMiniSlice	Outputs;
	uint8		NumOptionalInputs;
	uint8		NumRequiredInputs;
	uint16		OptionalInputIdx;

	int32		NumInputs() const { return NumOptionalInputs + NumRequiredInputs; }
};

union FMatchIdx
{
	uint16		Op;
	uint16		Decomp;
};

struct FMemberMatch
{
	FLooseType	Type; // With Innermost.IsDecomposed == 0
	FMatchIdx	Idx;
	uint8		InputIdx;
	uint8		Recipient;	// 0 = this, 1 = Outer, 2 = Outer->Outer, ...

	bool		Decomposes() const { return Recipient == DecomposeRecipient; }

	static constexpr uint8 DecomposeRecipient = 0xFF;
};

// Compiled upgrades for all versions of a particular struct
struct FStructUpgrades
{
	//~FStructUpgrades(); // todo

	FDeclId						Id;
	FDeclId						FinalId;		// Potentially renamed struct name
	uint16						FinalVersion;	// Potentially renamed struct version
	uint16						FirstVersion;	// First Relevant version
 	uint16						NumVersions;	// Versions are contiguous but can start at >0
	const FVersionUpgrades*		Versions[0];	// Indexed by Version - FirstVersion
};

// Compiled upgrades for a particular struct version and set of overrides
struct FVersionUpgrades
{
	//~FVersionUpgrades(); // todo: destroy decompositions

	uint32						OutputsOffset;		// Using offsets at the end to access Outputs faster
	uint32						NumNames;			// Declared and renamed members, duplicated across versions for simplicity
	uint32						NumMatches;
	uint16						NumOps;
	uint16						NumDecompositions;	// Struct / enum inputs with named members / enumerators / flags
	uint16						NumRecompositions;	// Enum outputs with named enumerators / flags
	uint16						NumOptionalInputs;	// Total optional inputs, enables type-checking missing optional inputs
	FOptionalVersion			NextVersion;		// Next version for this OR renamed struct
	FMiniSlice					NameMatches[0];		// #NumNames
	
	template<class T>
	const T&					At(uint32 Offset) const		{ return *reinterpret_cast<const T*>(reinterpret_cast<const uint8*>(this) + Offset); }

	const FMemberMatch*			GetMatches() const			{ return AlignPtr<FMemberMatch>(NameMatches + NumNames); }
	const FOp*					GetOps() const				{ return AlignPtr<FOp>(GetMatches() + NumMatches); }
	const uint32*				GetDecompOffsets() const	{ return AlignPtr<uint32>(GetOps() + NumOps); }
	const FLooseParameter*		GetOptionalInputs()	const	{ return AlignPtr<FLooseParameter>(GetDecompOffsets() + NumDecompositions); }
	const uint16*				GetRecompJumps() const		{ return AlignPtr<uint16>(GetOptionalInputs() + NumOptionalInputs); }
	const FOutParameter*		GetOutputs() const			{ return &At<FOutParameter>(OutputsOffset); }
};

////////////////////////////////////////////////////////////////////////////////////////////////

// Source constant -> loose value
struct FFlatEnumLoosener
{
	// todo: either track and destroy loosener or change to hashtable w/o internal allocation (e.g. null/footer container allocator, some fixed hash map) or sorted array lookup
	TMap<uint64, FEnumeratorId> ConstantNames;
};

// Source flag -> loose value
struct FFlagEnumLoosener
{
	TStaticArray<FEnumeratorId, 64> FlagNames{FEnumeratorId{}};
};

// Helps convert source enums to loose or indicate that declaration matches source schema
class FEnumLoosener
{
	static constexpr uint64 NeededBit = 1;
	static constexpr uint64 FlagBit = 2;
	static constexpr uint64 PtrMask = ~(FlagBit | NeededBit);
	uint64 Ptr = 0;

	FEnumLoosener(const void* Loosener, uint64 Flags) : Ptr(Flags | reinterpret_cast<uint64>(Loosener)) {}
public:
	FEnumLoosener() {}
	FEnumLoosener(FEnumLoosener&& O) : Ptr(O.Ptr) { O.Ptr = 0; }
	FEnumLoosener(const FEnumLoosener&) = delete;
	FEnumLoosener(bool bNeeded, /* own */ FFlatEnumLoosener* Flat UE_LIFETIMEBOUND) : FEnumLoosener(Flat, bNeeded * NeededBit) {}
	FEnumLoosener(bool bNeeded, /* ref */ const FFlagEnumLoosener& Flag UE_LIFETIMEBOUND) : FEnumLoosener(&Flag, bNeeded * NeededBit | FlagBit) {}
	~FEnumLoosener();
	FEnumLoosener& operator=(FEnumLoosener&& O) { Swap(Ptr, O.Ptr); return *this; }
	FEnumLoosener& operator=(const FEnumLoosener&) = delete;

	bool						Exists() const { return !!(Ptr); }
	bool						Needed() const { return !!(Ptr & NeededBit); }
	bool						IsFlag() const { return !!(Ptr & FlagBit); }
	const FFlagEnumLoosener&	AsFlag() const { checkSlow( IsFlag()); return *reinterpret_cast<FFlagEnumLoosener*>(Ptr & PtrMask); }
	const FFlatEnumLoosener&	AsFlat() const { checkSlow(!IsFlag()); return *reinterpret_cast<FFlatEnumLoosener*>(Ptr & PtrMask); }
};

// Loose values -> built value conversion
struct FEnumFastener
{
	const FEnumDeclaration*		Decl = nullptr; // null if loose FEnumeratorId indices match declared constants/flags
};

// Cached info for upgrading an enum
struct FEnumTransformation
{
	FEnumLoosener				Source;
	//const FEnumUpgrades*		Upgrades; // todo
	FEnumFastener				Loose;
};

// Cached info for upgrading a struct
struct FTransformation
{
	mutable const FLooseSchema*	SourceSchema = nullptr;
	const FStructUpgrades*		Upgrades = nullptr;
	const FLooseSchema*			TargetSchema = nullptr;
};

// Cached derived schema metadata from a source batch
struct FSourceSchemas
{
	uint32								NumStructs = 0;
	const FSchemaBatch*					Batch = nullptr;
	FInnerIds							RuntimeIds;
	TArray<uint16>						StructVersions;
	TBitArray<>							EnumFlagModes;
};

// Cached info for upgrading all structs and enums in a source batch
struct FTransformations
{
	TArray<FTransformation>		Structs;
	TSet<FDeclId>				StructLookup;		// If we start removing, switch to TSparseSet for FSetElementId stability
	TArray<FEnumTransformation>	Enums;
	TSet<FEnumId>				EnumLookup;			// If we start removing, switch to TSparseSet for FSetElementId stability
	FSourceSchemas				Schemas;

	const FTransformation&		operator[](FStructSchemaId Id) const	{ return Structs[Id.Idx]; }
	const FTransformation&		operator[](FDeclId Id) const			{ return Structs[StructLookup.FindId(Id).AsInteger()]; }	// TArrayView::RangeCheck checks invalid FSetElementId
	const FEnumTransformation&	operator[](FEnumSchemaId Id) const		{ return Enums[Id.Idx - Schemas.NumStructs]; }
	const FEnumTransformation&	operator[](FEnumId Id) const			{ return Enums[EnumLookup.FindId(Id).AsInteger()]; }		// TArrayView::RangeCheck checks invalid FSetElementId
};

////////////////////////////////////////////////////////////////////////////////////////////////

using FLooseSchemaPtr = TUniquePtr<FLooseSchema>;

// Help access input struct members in FStructMemberNames order and upgrade/loosen them first
struct FStructDecomposition
{
	FDeclId						Id;
	uint16						Version;
	TSet<FParameterId>			InputOrder;				// empty for declared order (iota)
	const FStructUpgrades*		Upgrades = nullptr;
	FLooseSchemaPtr				InputSchema;
};

// Help access input enum enumerators in FEnumMembers order
struct FEnumDecomposition
{
	// const FEnumUpgrades*		Upgrades = nullptr;		// todo
	TSet<FEnumeratorId>			InputOrder;				// empty for declared order (iota)
};

////////////////////////////////////////////////////////////////////////////////////////////////

// Home for all outputs
class FOutputStack
{
	FOut* const					Data;
	int32						Num = 0;
	FScratchAllocator&			Scratch; // Convenience
public:
	FOutputStack(FScratchAllocator& InScratch);
	~FOutputStack() { check(Num == 0); }

	FOuts						Push(TConstArrayView<FOutParameter> Params);
	void						Pop(int32 InNum);
};

struct FOutputScope
{
	FOutputScope(FOutputStack& InStack, TConstArrayView<FOutParameter> Params);
	~FOutputScope() { Stack.Pop(Outputs.Num()); }

	FOutputStack&				Stack;
	const FOuts					Outputs;
};

// Helps print some loose types on top of the FDebugIds API
class FDebugPrinter : public FDebugIds
{
	const FLooseIndexers&		LooseIds;
public:
	FDebugPrinter(const FIds& Ids, FLooseIndexers& Loose) : FDebugIds(Ids), LooseIds(Loose) {}

	using FDebugIds::Print;
	FString						Print(FLooseType Type) const;
	FString						Print(FDeclId Id, FParameterId Name) const	{ return Print(LooseIds.Get(Id).Resolve(Name)); }
	FString						Print(FEnumId Id, FEnumeratorId Name) const	{ return Print(LooseIds.Get(Id).Resolve(Name)); }
};

// Context for upgrading a source batch
struct FBatchContext
{
	// todo: add loose transformations for redirected and rename to source transformations
	const FTransformations&		Transformations;
	FScratchAllocator&			Scratch;
	const FIdIndexerBase&		Ids;
	FLooseIndexers&				LooseIds; // RenameMembers can index new names
	const FLooseType			Obsolete;
	const FSchemaBatchId		Batch; 
	FOutputStack				OutputStack;
	FContext					OpCtx;
	FDebugPrinter				Debug;
};

// Home for FIn of "polyary" (>1 inputs) ops
struct FInputDependencies
{
	int32						MissingRequired;
	int32						MissingOptional;
	FIn							Inputs[0];
};

// Owner of an instance upgrader that might overload and steal inputs
struct FOuter
{
	FInstanceUpgrader*			Instance = nullptr;
};

// Upgrades a single struct instance
class FInstanceUpgrader
{
	UE_NONCOPYABLE(FInstanceUpgrader);

	FBatchContext&					Ctx;
	const FStructUpgrades&			Struct;
	const FVersionUpgrades*			Current = nullptr;
	FOuter				 			Outer;
	const uint32					NumNames;
	FOptionalVersion				Limit;						// Unimplemented
	bool							bUntypedOutcome = false;	// Obsolete types and intermediate versions lack known schema
	uint16							FirstPartialOpIdx = 0xFFFF;	// First "polyary" (>1 inputs) op idx, Flush() opt
	uint8*							NameToScanIndices;			// #NumNames - note: wont fly all the way, should support loop around + track used matches
	TArrayView<FInputDependencies*>	PartialInputs;				// #Max(NumOps) for versions with polyary ops
	TArrayView<FLooseMember>		Outcome;					// #NumNames, initialized by Expect()
	TArray<FLooseMember>			Mismatches;
	
	void							Expect(const FLooseSchema* Target);
	void							Feed(FLooseMemberReader&& SourceMembers);
	void							Feed(TConstArrayView<FLooseMember> LooseMembers);
	void							Give(FLooseType Type, FLooseValue Value);
	void							Give(FLooseType Type, FLooseValue Value, FMatchHelper Helper);
	FLooseValue						Decompose(FLooseType Type, FLooseValue Value, uint16 DecompIdx);
	void							Decompose(FLooseType Type, FLooseNestedRange Range, const void* Decomp);
	FLooseValue						Decompose(FLooseType Type, FLooseValue Value, const FEnumDecomposition& Decomp);
	FLooseValue						Decompose(FEnumMember Input, const FEnumDecomposition& Decomp);
	FLooseValue						Decompose(FStructMember Input, const FStructDecomposition& Decomp);
	TArrayView<FLooseMember>		Decompose(FStructView Input, const FStructDecomposition& Decomp, FParameterId Name, EMismatch& OutMismatch);
	FEnumIndexData					Reorder(FEnumeratorData Input, const TSet<FEnumeratorId>& Order, bool bFlagMode);
	FEnumIndexData					Reorder(FEnumeratorId Flat, const TSet<FEnumeratorId>& Order);
	FEnumIndexData					Reorder(FEnumeratorIdSet Flags, const TSet<FEnumeratorId>& Order);
	void							Input(FMemberMatch Match, FLooseValue Value);
	void							Stash(FLooseType Type, FLooseValue Value);
	void							Run(FOp Op, const FIn* Inputs, uint8 Recipient);
	FOuter							Resolve(uint8 Recipient);
	void							Recompose(TArrayView<FOut> Outputs, TConstArrayView<FOutParameter> Expected);
	FEnumeratorData					Recompose(FEnumIndexData In, TConstArrayView<FEnumeratorId> Order, FInnermostEnumType Type);
	void							Recompose(FRangeData Out, uint64 Num, uint8 Depth, TConstArrayView<FEnumeratorId> Order, FInnermostEnumType Type);
	void							Flush();
	FLooseStruct					Finish(EMismatch& OutMismatch);

	FString							Print(FParameterId Parameter) const;
	FString							Print(FLooseType Type) const;
	template<class T> FString		Print(T Item) const { return Ctx.Debug.Print(Item); }
	
public:
	FInstanceUpgrader(FBatchContext& Ctx, const FStructUpgrades& Upgrades, uint16 Version);
	
	FLooseStruct					Upgrade(FStructView Source, const FLooseSchema& From, const FLooseSchema* To, EMismatch& OutMismatch);
	FLooseStruct					Upgrade(FLooseStruct In, const FLooseSchema* To, EMismatch& OutMismatch);
	FStructMember					Upgrade(FStructMember In, const FLooseSchema* To, FOuter InOuter = {});

	void							Reuse(); // Todo: Optimization for upgrading range-of-structs
};

////////////////////////////////////////////////////////////////////////////////////////////////

struct FSchemaCache
{
	const FLooseSchema*					Obtain(FDeclId Id, const IDeclarations& Types, FParameterIndexer& ParamIds);
	TMap<FDeclId, FLooseSchemaPtr>		Cache;
};

////////////////////////////////////////////////////////////////////////////////////////////////

template <typename T>
struct TDestroyOnly
{
	void operator()(T* Ptr) const
	{
		if (Ptr)
		{
			Ptr->~T();
		}
	} 
};

template<class T>
using TScratchPtr = TUniquePtr<T, TDestroyOnly<T>>;

////////////////////////////////////////////////////////////////////////////////////////////////

using FOpsPtr = TScratchPtr<FOps>;
using FStructUpgradesPtr = TUniquePtr<FStructUpgrades>;

struct FHistory
{
	const IDeclarations&				Latest;
	mutable FSchemaCache				LatestSchemas;
	FLiteralIndexerBase&				Ids;
	mutable FLooseIndexers				LooseIds;
	FScratchAllocator					CommitScratch;
	TArray<FOpsPtr>						CommitQueue;
	TMap<FDeclId, FStructUpgradesPtr>	CompiledUpgrades;
	
	void								Compile();
	void								Erase(FStructId Id);

private:
	void								CompileStruct(TArrayView<FOpsPtr> AllOps);
	void								Link(TConstArrayView<FOpsPtr> Committed);
	void								Link(FDeclId Compiled);
	void								Link(FStructDecomposition& Dependency);
};

} // namespace PlainProps::Upgrade