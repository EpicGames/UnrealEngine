// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsLoad.h"
#include "PlainPropsLoadMember.h"
#include "PlainPropsBind.h"
#include "PlainPropsInternalBind.h"
#include "PlainPropsInternalFormat.h"
#include "PlainPropsInternalRead.h"
#include "PlainPropsSave.h" // FBaseline use in dummy FLoadBindingBase::SaveCustom
#include "Algo/Compare.h"
#include "Algo/Find.h"
#include "Containers/Set.h"
#include "Misc/Optional.h"
#include <type_traits>

namespace PlainProps
{

// Duplicated runtime FSchemaBinding with FStructSchemaId load ids instead of runtime FInnerId
struct FSchemaLoadPlan
{
	const FSchemaBinding&	Clone;

	TConstArrayView<FMemberBindType>	GetMembers() const			{ return MakeArrayView(Clone.Members, Clone.NumMembers); }
	const uint32*						GetOffsets() const			{ return Clone.GetOffsets(); }
	TConstArrayView<FStructSchemaId>	GetInnerSchemas() const		{ return MakeArrayView(reinterpret_cast<const FStructSchemaId*>(Clone.GetInnerIds()), Clone.NumInnerIds); }
	TConstArrayView<FMemberBindType>	GetInnerRangeTypes() const	{ return MakeArrayView(Clone.GetInnerRangeTypes(), Clone.NumInnerRanges); }
	const FRangeBinding*				GetRangeBindings() const	{ return Clone.GetRangeBindings(); }

};

// ICustomBinding with internal type-erased/lowered struct schemas
struct FCustomLoadPlan
{
	const ICustomBinding*	Binding;
	uint32					NumLoadIds;
	FStructSchemaId			LoadIds[0];
};

// Describes how to load a saved struct into the matching in-memory representation
class FLoadStructPlan
{
public:
	FLoadStructPlan() = default;

	explicit FLoadStructPlan(FMemcpyLoadPlan Memcpy)
	: Handle((uint64(Memcpy.Size) << 32) | (uint64(Memcpy.Offset) << 2) | MemcpyMask)
	{
		check(Memcpy.Offset == AsMemcpy().Offset && Memcpy.Size == AsMemcpy().Size);
	}

	explicit FLoadStructPlan(const ICustomBinding& Custom)
	: Handle(uint64(&Custom) | CustomMask)
	{
		check(&Custom == &AsCustom());
	}

	explicit FLoadStructPlan(const FCustomLoadPlan& Custom)
	: Handle(uint64(&Custom) | CustomMask | LoadIdsBit)
	{
		check(Custom.Binding == &AsCustom());
		check(GetInnerLoadIds() == Custom.LoadIds);
	}

	// @param OffsetWidth Usage unimplemented, store size and offsets as 8/16/32/64-bit 
	explicit FLoadStructPlan(const FSchemaBinding& Schema, ELeafWidth OffsetWidth, bool bSparse)
	: Handle(uint64(&Schema) | SchemaBit | (bSparse ? SparseBit : 0) | (uint64(OffsetWidth) << SchemaOffsetShift))
	{
		static_assert(alignof(FSchemaBinding) >= 8);
		check(&Schema == &AsSchema().Clone);
		check(bSparse == IsSparseSchema());
		check(OffsetWidth == GetOffsetWidth());
	}

	friend bool operator==(FLoadStructPlan, FLoadStructPlan) = default;
	friend bool operator!=(FLoadStructPlan, FLoadStructPlan) = default;

	bool						IsSchema() const		{ return (Handle & SchemaBit) == SchemaBit; }
	bool						IsSparseSchema() const	{ return (Handle & SparseSchemaMask) == SparseSchemaMask; }
	bool						IsMemcpy() const		{ return (Handle & LoMask) == MemcpyMask; }
	bool						IsCustom() const		{ return (Handle & LoMask) == CustomMask; }
	FMemcpyLoadPlan				AsMemcpy() const		{ check(IsMemcpy()); return { static_cast<uint32>(Handle >> 32), static_cast<uint32>(Handle) >> 2 }; }
	FSchemaLoadPlan				AsSchema() const		{ check(IsSchema()); return { *AsPtr<FSchemaBinding>() }; }
	const ICustomBinding&		AsCustom() const;

	const FStructSchemaId*		GetInnerLoadIds()		{ return (Handle & TagMask) == LoadIdsMask ? AsPtr<FCustomLoadPlan>()->LoadIds : nullptr; }
	ELeafWidth					GetOffsetWidth() const	{ check(IsSchema()); return static_cast<ELeafWidth>((Handle & SchemaOffsetMask) >> SchemaOffsetShift); }

private:
	static constexpr uint64 SparseBit			= uint64(1) << FPlatformMemory::KernelAddressBit;
	static constexpr uint64 LoadIdsBit			= SparseBit;
	static constexpr uint64 TagMask				= SparseBit | 0b111;
	static constexpr uint64 PtrMask				= ~(TagMask);
	static constexpr uint64 LoMask				= 0b11;
	static constexpr uint64 MemcpyMask			= 0b00;
	static constexpr uint64 CustomMask			= 0b10;
	static constexpr uint64 SchemaBit			= 0b01;
	static constexpr uint64 SparseSchemaMask	= SchemaBit | SparseBit;
	static constexpr uint64 SchemaOffsetShift	= 1;
	static constexpr uint64 SchemaOffsetMask	= 0b110;
	static constexpr uint64 LoadIdsMask			= CustomMask | LoadIdsBit;

	template<typename T>
	const T* AsPtr() const
	{
		check(Handle & PtrMask);
		return reinterpret_cast<T*>(Handle & PtrMask);
	}

	uint64						Handle = 0;
};

const ICustomBinding& FLoadStructPlan::AsCustom() const
{ 
	check(IsCustom());
	return (Handle & LoadIdsBit) ? *AsPtr<FCustomLoadPlan>()->Binding : *AsPtr<ICustomBinding>();
}

////////////////////////////////////////////////////////////////////////////

static uint16 CountEnums(const FStructSchema& Schema)
{
	if (Schema.NumInnerSchemas == 0)
	{
		return 0;
	}
	
	uint16 Num = 0;
	TConstArrayView<FMemberType> RangeTypes = Schema.GetRangeTypes(); 
	if (RangeTypes.IsEmpty())
	{
		for (FMemberType Member : Schema.GetMemberTypes())
		{
			Num += IsEnum(Member);
		}
		return Num;
	}
	
	uint16 RangeTypeIdx = 0;
	for (FMemberType Member : Schema.GetMemberTypes())
	{
		if (Member.IsRange())
		{
			FMemberType InnermostType = GrabInnerRangeTypes(RangeTypes, /* in-out */ RangeTypeIdx).Last();
			Num += IsEnum(InnermostType);
		}
		else
		{
			Num += IsEnum(Member);
		}
	}
	check(RangeTypeIdx == Schema.NumRangeTypes);
	return Num;
}

static bool HasIncompatibleSupers(const FStructSchema& From, const FSchemaBinding& To, TConstArrayView<FStructId> ToStructIds)
{
	if (From.Inheritance == ESuper::No)
	{
		return To.HasSuper();
	}
	else if (From.Inheritance == ESuper::Used)
	{
		// From has two super schema ids, dont bother cloning it
		return true;
	}
	else if (To.HasSuper())
	{
		FStructId FromSuper = ToStructIds[From.GetSuper().Get().Idx];
		FStructId ToSuper = To.GetInnerIds()[0].AsStruct();
		return FromSuper != ToSuper;
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////

// Used to create an additional load plan, beyond the saved struct schema ids
struct FLoadIdMapping
{
	FStructSchemaId ReadId; // ~ Batch decl id, index into saved schemas and load plans
	FStructSchemaId LoadId; // ~ Batch bind id, index into load plans
	FBindId			Id; // Runtime bind id
};

// Helps load type-erased/lowered structs with ExplicitBindName by allocating new load-time struct ids
class FLoadIdBinder
{
public:
	FLoadIdBinder(TConstArrayView<FDeclId> Ids, uint32 NumPlans, FDebugIds Dbg)
	: DeclIds(Ids)
	, NextLoadIdx(NumPlans)
	, Debug(Dbg)
	{
		check(NextLoadIdx >= static_cast<uint32>(Ids.Num()));
	}

	FStructSchemaId					BindLoadId(FStructSchemaId ReadId, FBindId Id)
	{
		FStructId DeclId = DeclIds[ReadId.Idx];
		FStructSchemaId LoadId = Id == DeclId ? ReadId : MapLoadId(ReadId, Id);
		return LoadId;
	}
	FLoadIdMapping					GetMapping(int32 Idx) const { return Mappings[Idx]; }
	int32							NumMappings() const { return Mappings.Num(); }

private:
	TConstArrayView<FDeclId>			DeclIds;
	uint32								NextLoadIdx;
	TArray<FLoadIdMapping>				Mappings;

	FStructSchemaId MapLoadId(FStructSchemaId ReadId, FBindId Id)
	{
		for (FLoadIdMapping Mapping : Mappings)
		{
			if (Mapping.Id == Id)
			{
				check(Mapping.ReadId == ReadId);
				return Mapping.LoadId;
			}
		}

		FLoadIdMapping& Mapping = Mappings.AddDefaulted_GetRef();
		Mapping.ReadId = ReadId;
		Mapping.LoadId = { NextLoadIdx++ };
		Mapping.Id = Id;
		return Mapping.LoadId;
	}
public:
	FDebugIds							Debug;
};

struct FLoadIdBinderDummy
{
	FDebugIds							Debug;
	static constexpr FStructSchemaId	BindLoadId(FStructSchemaId ReadId, FBindId Id) { return ReadId; }
	static constexpr int32				NumMappings() { return 0; }
	static inline FLoadIdMapping		GetMapping(int32 Idx) { unimplemented(); return {}; }
};

////////////////////////////////////////////////////////////////////////////

struct FLateLoadPlans
{
	FLateLoadPlans* Next;
	uint8 Data[0];
};

struct FLoadBatch
{
	FSchemaBatchId			BatchId;			// Needed to access schemas for custom struct loading
	uint32					NumReadSchemas;		// Number of saved struct schemas
	uint32					NumPlans;			// NumReadSchemas + NumMappings (for bound type-erased structs)
	uint32					MaxPlans;			// MaxPlans required to handle late bound structs with potentially type-erased inners
	FLateLoadPlans*			LatePlans;			// Data for late bound structs
	FLoadStructPlan			Plans[0];

	FLoadStructPlan			operator[](FStructSchemaId LoadId) const
	{
		check(LoadId.Idx < NumPlans);
		return Plans[LoadId.Idx];
	}

	FStructSchemaId			GetReadId(FStructSchemaId LoadId) const
	{
		check(LoadId.Idx < NumPlans);
		static_assert(alignof(FLoadStructPlan) >= alignof(FStructSchemaId));
		const FStructSchemaId* SaveIds = reinterpret_cast<const FStructSchemaId*>(Plans + MaxPlans);
		return LoadId.Idx < NumReadSchemas ? LoadId : SaveIds[LoadId.Idx - NumReadSchemas];
	}
};

void FLoadBatchDeleter::operator()(FLoadBatch* Batch) const
{
	if (Batch)
	{
		for (FLateLoadPlans *It = Batch->LatePlans, *Next; It; It = Next)
		{
			Next = It->Next;
			FMemory::Free(It);
		}
		FMemory::Free(Batch);
	}
}

////////////////////////////////////////////////////////////////////////////

struct FMemberLoadBinder : FMemberBinderBase
{
	FMemberLoadBinder(FSchemaBinding& In)
	: FMemberBinderBase(In)
	, InnerIdIt(const_cast<FStructSchemaId*>(FSchemaLoadPlan(In).GetInnerSchemas().GetData()))
	{}
	
	~FMemberLoadBinder()
	{
		check(Align(InnerIdIt, alignof(FRangeBinding)) == (const void*)Schema.GetRangeBindings() || Schema.NumInnerRanges == 0);
	}

	void AddInnerId(FStructSchemaId InnermostId)
	{
		*InnerIdIt++ = InnermostId;
	}

	FStructSchemaId* InnerIdIt;
};


struct FLoadBindingBase : ICustomBinding
{
	virtual void SaveCustom(FMemberBuilder& Dst, const void* Src, FBaseline Base, const FSaveContext& Ctx) override final { unimplemented(); }
	virtual bool DiffCustom(const void*, const void*, const FBindContext&) const override final { unimplemented(); return false; }
};
struct FTypeErasedLoadBinding final : FLoadBindingBase
{
	virtual void LoadCustom(void* Dst, FStructLoadView Src, ECustomLoadMethod) const override { checkf(false, TEXT("Can't load type-erased/lowered struct binding with load id %d"), Src.Schema.LoadId.Idx); }
};
static FTypeErasedLoadBinding GLoadTypeErasedError;
struct FNoopLoadBinding final : FLoadBindingBase
{
	virtual void LoadCustom(void* Dst, FStructLoadView Src, ECustomLoadMethod Method) const override {}
};
static FNoopLoadBinding GLoadNoop;
struct FLateLoadBinding final : FLoadBindingBase
{
	virtual void LoadCustom(void* Dst, FStructLoadView Src, ECustomLoadMethod) const override { checkf(false, TEXT("Can't load missing/late struct binding with load id %d"), Src.Schema.LoadId.Idx); }
};
static FLateLoadBinding GLoadNotBoundError;

inline static bool IsBound(FLoadStructPlan Plan)
{
	return Plan != FLoadStructPlan(GLoadNotBoundError);
}

bool HasLoadPlan(FStructSchemaId LoadId, const FLoadBatch& Batch)
{
	return IsBound(Batch[LoadId]);
}

template<class T>
static TOptional<TConstArrayView<T>> GetContiguousSubset(TConstArrayView<T> View, const TBitArray<>& Keep)
{
	check(Keep.Num() == View.Num());
	const int32 Num = Keep.CountSetBits();
	check(Num < Keep.Num());

	if (Num == 0)
	{
		return TConstArrayView<T>{};
	}

	int32 StartIdx = Keep.Find(true);
	check(StartIdx != INDEX_NONE);
	if (StartIdx + 1 == Keep.Num())
	{
		return View.Slice(StartIdx, Num);
	}

	int32 EndIdx = Keep.FindFrom(false, StartIdx + 1);
	check(EndIdx != INDEX_NONE);
	if (EndIdx - StartIdx == Num)
	{
		return View.Slice(StartIdx, Num);
	}

	return NullOpt;
}

template<typename LoadIdBinder>
struct FLoadPlanner
{
	static constexpr bool bIsLoadIdBinderDummy = std::is_same_v<LoadIdBinder, FLoadIdBinderDummy>;

	// Constructor parameters
	const FSchemaBatchId							BatchId;
	const FCustomBindings&							Customs;
	const FSchemaBindings&							Schemas;
	const TConstArrayView<FStructId>				RuntimeIds;
	LoadIdBinder									LoadIds;

	// Temporary data structures	
	TArray<FLoadStructPlan, TInlineAllocator<256>>	Plans;
	TArray<uint32, TInlineAllocator<256>>			SubsetSchemaSizes;
	TArray<uint8, TInlineAllocator<1024>>			SubsetSchemaData;
	TSet<FStructSchemaId>							UnboundSaveIds;
	
	FLoadBatchPtr CreatePlans()
	{
		check(NumStructSchemas(BatchId) == RuntimeIds.Num());

		// Make load plans for saved schemas
		const uint32 NumPlans = RuntimeIds.Num();
		Plans.SetNumUninitialized(NumPlans);
		SubsetSchemaSizes.SetNumUninitialized(NumPlans);

		if constexpr (bIsLoadIdBinderDummy)
		{
			for (uint32 Idx = 0; Idx < NumPlans; ++Idx)
			{
				FLoadIdMapping Mapping;
				Mapping.ReadId = { Idx };
				Mapping.LoadId = { Idx };
				Mapping.Id = FBindId(RuntimeIds[Idx]);
				CreatePlan(Mapping);
			}
			return CreateBatch();
		}
		else
		{
			for (uint32 Idx = 0; Idx < NumPlans; ++Idx)
			{
				FLoadIdMapping Mapping;
				Mapping.ReadId = { Idx };
				Mapping.LoadId = { Idx };
				Mapping.Id = FBindId(RuntimeIds[Idx]);
				CreatePlan(Mapping);
			}

			// Make load plans for type-erased/ExplicitBindName structs needed by already created load plans
			if (LoadIds.NumMappings() > 0)
			{
				Plans.Reserve(NumPlans + LoadIds.NumMappings());
				SubsetSchemaSizes.Reserve(NumPlans + LoadIds.NumMappings());
				for (int32 Idx = 0; Idx < LoadIds.NumMappings(); ++Idx)
				{
					check(LoadIds.GetMapping(Idx).LoadId.Idx == static_cast<uint32>(Plans.Num()));
					Plans.AddUninitialized();
					SubsetSchemaSizes.AddUninitialized();
					CreatePlan(LoadIds.GetMapping(Idx)); // LoadIds might grow
				}

				// Strip all UnboundSaveIds that were bound by some load plan,
				// and set set the load plan to GLoadTypeErasedError to mark it as bound.
				for (int32 Idx = 0; !UnboundSaveIds.IsEmpty() && Idx < LoadIds.NumMappings(); ++Idx)
				{
					FStructSchemaId ReadId = LoadIds.GetMapping(Idx).ReadId;
					Plans[ReadId.Idx] = FLoadStructPlan(GLoadTypeErasedError);
					UnboundSaveIds.Remove(ReadId);
				}
			}

			// TODO: Bring back check for verifying that all structs are bound when we can reliably identify late-bound structs
			// for (FStructSchemaId UnboundSaveId : UnboundSaveIds)
			// {
			// 	checkf(false, TEXT("Unbound struct '%s' can't be loaded"), *Schemas.GetDebug().Print(RuntimeIds[UnboundSaveId.Idx]));
			// }

			// Allocate load batch, copy plans and subset schemas, and fixup subset schema plans
			return CreateBatch();
		}
	}

	void UpdatePlans(FLoadBatch& Batch)
	{
		check(Batch.NumReadSchemas == RuntimeIds.Num());
		check(Batch.NumPlans >= Batch.NumReadSchemas);

		// Initialize temp plans to GLoadNotBoundError.
		// The temp SubsetSchemaSizes will only be accessed for newly bound plans.
		const uint32 NumMappings = Batch.NumPlans - Batch.NumReadSchemas;
		Plans.Init(FLoadStructPlan(GLoadNotBoundError), Batch.NumPlans);
		SubsetSchemaSizes.SetNumUninitialized(Batch.NumPlans);

		// Eagerly create load plans for all unbound read schemas
		for (uint32 Idx = 0; Idx < Batch.NumReadSchemas; ++Idx)
		{
			if (!IsBound(Batch.Plans[Idx]))
			{
				FLoadIdMapping Mapping;
				Mapping.ReadId = { Idx };
				Mapping.LoadId = { Idx };
				Mapping.Id = FBindId(RuntimeIds[Idx]);
				CreatePlan(Mapping);
			}
		}

		// Make load plans for type-erased/ExplicitBindName structs needed by the new load plans
		if (LoadIds.NumMappings() > 0)
		{
			Plans.Reserve(Batch.NumPlans + LoadIds.NumMappings());
			SubsetSchemaSizes.Reserve(Batch.NumPlans + LoadIds.NumMappings());
			for (int32 Idx = 0; Idx < LoadIds.NumMappings(); ++Idx)
			{
				check(LoadIds.GetMapping(Idx).LoadId.Idx == static_cast<uint32>(Plans.Num()));
				Plans.AddUninitialized();
				SubsetSchemaSizes.AddUninitialized();
				CreatePlan(LoadIds.GetMapping(Idx)); // LoadIds might grow
			}

			// Remove all UnboundSaveIds for type erased structs that have now been bound by some load plan
			for (int32 Idx = 0; !UnboundSaveIds.IsEmpty() && Idx < LoadIds.NumMappings(); ++Idx)
			{
				FStructSchemaId ReadId = LoadIds.GetMapping(Idx).ReadId;
				Plans[ReadId.Idx] = FLoadStructPlan(GLoadTypeErasedError);
				UnboundSaveIds.Remove(ReadId);
			}
		}

		// Update load batch, copy new plans and subset schemas, and fixup new subset schema plans
		UpdateBatch(Batch);
	}

private:

	FLoadBatchPtr CreateBatch() const
	{
		const uint32 NumPlans = Plans.Num();
		const uint32 NumMappings = LoadIds.NumMappings();
		const uint32 NumReadSchemas = RuntimeIds.Num();
		const uint32 NumSlack = UnboundSaveIds.Num();
		const uint32 MaxPlans = NumPlans + NumSlack;
		check(NumPlans == NumReadSchemas + NumMappings);

		// Allocate plan, init header and copy plans
		SIZE_T Bytes = sizeof(FLoadBatch) + sizeof(FLoadStructPlan) * MaxPlans
					 + Align(sizeof(FStructSchemaId) * (NumMappings + NumSlack), alignof(FSchemaBinding))
					 + SubsetSchemaData.Num();
		FLoadBatch Header = { BatchId, NumReadSchemas, NumPlans, MaxPlans };
		FLoadBatch* Out = new (FMemory::Malloc(Bytes)) FLoadBatch{Header};
		FMemory::Memcpy(Out->Plans, Plans.GetData(), sizeof(FLoadStructPlan) * NumPlans);

		// Copy LoadId -> ReadId mapping so custom-bound mapped plans can form FReadSchemaHandle and FStructView
		FStructSchemaId* OutReadId = reinterpret_cast<FStructSchemaId*>(Out->Plans + MaxPlans);
		for (uint32 Idx = 0; Idx < NumMappings; ++Idx)
		{
			*OutReadId++ = LoadIds.GetMapping(Idx).ReadId;
			check(OutReadId[-1].Idx < NumReadSchemas);
		}
		OutReadId += NumSlack;

		// Copy cloned subset schemas and patch up plan pointers
		if (SubsetSchemaData.Num() > 0)
		{
			void* OutSubsetData = Align(OutReadId, alignof(FSchemaBinding));
			FMemory::Memcpy(OutSubsetData, SubsetSchemaData.GetData(), SubsetSchemaData.Num());
		
			// Update plans with actual subset schema pointers
			const uint8* It = static_cast<const uint8*>(OutSubsetData);
			for (uint32 Idx = 0; Idx < NumPlans; ++Idx)
			{
				It += UpdatePlanPointer(*Out, Idx, It);
			}
			check(It == (uint8*)OutSubsetData + SubsetSchemaData.Num());
			check(It == (uint8*)Out + Bytes);
		}
		else
		{
			check((uint8*)Align(OutReadId, alignof(FSchemaBinding)) == (uint8*)Out + Bytes);
		}

		return FLoadBatchPtr(Out);
	}

	void UpdateBatch(FLoadBatch& Batch) const
	{
		const uint32 NumPlans = Plans.Num();
		const uint32 NumMappings = LoadIds.NumMappings();
		check(NumPlans <= Batch.MaxPlans);
		check(NumMappings <= Batch.MaxPlans - Batch.NumPlans);
		check(NumMappings == NumPlans - Batch.NumPlans);

		// Copy new LoadId -> ReadId mappings so custom-bound mapped plans can form FReadSchemaHandle and FStructView
		FStructSchemaId* OutReadId = reinterpret_cast<FStructSchemaId*>(Batch.Plans + Batch.MaxPlans);
		OutReadId += Batch.NumPlans - Batch.NumReadSchemas; // skip past existing mappings
		for (uint32 Idx = 0; Idx < NumMappings; ++Idx)
		{
			*OutReadId++ = LoadIds.GetMapping(Idx).ReadId;
			check(OutReadId[-1].Idx < Batch.NumReadSchemas);
		}

		// Copy cloned subset schemas and patch up new plans
		if (SubsetSchemaData.Num() > 0)
		{
			// Allocate new subset data and add it to the linked list of dynamic allocations
			SIZE_T Bytes = Align(sizeof(FLateLoadPlans) + SubsetSchemaData.Num(), alignof(FSchemaBinding));
			FLateLoadPlans LatePlans = { Batch.LatePlans };
			FLateLoadPlans* Ptr = new (FMemory::Malloc(Bytes)) FLateLoadPlans{ LatePlans };
			Batch.LatePlans = Ptr;

			void* OutSubsetData = Align(&Ptr[1], alignof(FSchemaBinding));
			FMemory::Memcpy(OutSubsetData, SubsetSchemaData.GetData(), SubsetSchemaData.Num());

			// Update the existing and added slots with actual subset schema pointers
			const uint8* It = static_cast<const uint8*>(OutSubsetData);
			for (uint32 Idx = 0; Idx < NumPlans; ++Idx)
			{
				if (IsBound(Plans[Idx]))
				{
					if (uint32 Size = UpdatePlanPointer(Batch, Idx, It))
					{
						It += Size;
					}
					else
					{
						Batch.Plans[Idx] = Plans[Idx];
					}
				}
			}
			check(It == (uint8*)OutSubsetData + SubsetSchemaData.Num());
			check(It == (uint8*)Ptr + Bytes);
		}
		else // simply patch up new plans
		{
			for (uint32 Idx = 0; Idx < NumPlans; ++Idx)
			{
				if (IsBound(Plans[Idx]))
				{
					Batch.Plans[Idx] = Plans[Idx];
				}
			}
		}

		Batch.NumPlans = NumPlans;
	}

	[[nodiscard]] inline uint32 UpdatePlanPointer(FLoadBatch& Batch, uint32 Idx, const uint8* Src) const
	{
		uint32 Size = SubsetSchemaSizes[Idx];
		if (Size)
		{
			if (Plans[Idx].IsSchema())
			{
				check(IsAligned(Size, alignof(FSchemaBinding)));
				bool bSparse = Plans[Idx].IsSparseSchema();
				Batch.Plans[Idx] = FLoadStructPlan(*reinterpret_cast<const FSchemaBinding*>(Src), ELeafWidth::B32, bSparse);
			}
			else
			{
				check(Plans[Idx].IsCustom());
				check(IsAligned(Size, alignof(FCustomLoadPlan)));
				Batch.Plans[Idx] = FLoadStructPlan(*reinterpret_cast<const FCustomLoadPlan*>(Src));
			}
		}
		return Size;
	}

	void CreatePlan(FLoadIdMapping Mapping)
	{
		static_assert(alignof(FCustomLoadPlan) == alignof(FSchemaBinding), "Alignment equality is expected by SubsetSchemaData");
		const int32 SubsetSchemaOffset = SubsetSchemaData.Num();
		Plans[Mapping.LoadId.Idx] = CreatePlanInner(Mapping);
		SubsetSchemaSizes[Mapping.LoadId.Idx] = static_cast<uint32>(SubsetSchemaData.Num() - SubsetSchemaOffset);
	}

	// Schema iterator that scans forward to locate inner struct schema ids
	class FInnerStructSchemaIterator
	{
	public:
		FInnerStructSchemaIterator(const FStructSchema& Schema) : FInnerStructSchemaIterator(Schema, UsesSuper(Schema.Inheritance)) {}

		FStructSchemaId GrabMemberStruct(FMemberId InName)
		{
			while (true)
			{
				FMemberId Name = Names[NameIdx];
				FMemberType Type = NamedTypes[NameIdx];
				FMemberType InnermostType = Type.IsRange() ? GrabInnerRangeTypes(RangeTypes, /* in-out */ RangeTypeIdx).Last() : Type;
				++NameIdx;
				InnerIdIt += IsStructOrEnum(InnermostType);
				
				if (Name == InName)
				{
					check(InnermostType.IsStruct());
					return static_cast<FStructSchemaId>(InnerIdIt[-1]);
				}
			}
		}

	private:
		FMemberType const* const			NamedTypes;
		const TConstArrayView<FMemberId>	Names;
		const TConstArrayView<FMemberType>	RangeTypes;
		
		int32								NameIdx = 0;
		int32								RangeTypeIdx = 0;
		const FSchemaId*					InnerIdIt;

		FInnerStructSchemaIterator(const FStructSchema& Schema, bool bSkipSuper)
		: NamedTypes(FStructSchema::GetMemberTypes(Schema.Footer) + bSkipSuper)
		, Names(Schema.GetMemberNames())
		, RangeTypes(Schema.GetRangeTypes())
		, InnerIdIt(Schema.GetInnerSchemas() + /* super schema */ (bSkipSuper && !Schema.GetMemberTypes()[0].AsStruct().IsDynamic))
		{}
	};

	[[nodiscard]] FCustomLoadPlan* CreateCustomLoadIdsPlan(const ICustomBinding* Custom, const FStructSchema& Schema, TConstArrayView<FInnerStruct> Inners)
	{
		const uint32 Num = static_cast<uint32>(Inners.Num());
		check(Num);
		check(Schema.NumInnerSchemas > 0);
		check(IsAligned(SubsetSchemaData.Num(), alignof(FCustomLoadPlan)));

		// Calculate padding and size needed
		uint32 Size = offsetof(FCustomLoadPlan, LoadIds) + sizeof(FStructSchemaId) * Num;
		uint32 Pad = Align(SubsetSchemaData.Num(), alignof(FCustomLoadPlan)) - SubsetSchemaData.Num();
		SubsetSchemaData.AddUninitialized(Pad + Size);

		// Construct and init header
		FCustomLoadPlan Header = {Custom, Num};
		FCustomLoadPlan* Out = new (&SubsetSchemaData[SubsetSchemaData.Num() - Size]) FCustomLoadPlan {Header};
		check(IsAligned(Out, alignof(FCustomLoadPlan)));
		FStructSchemaId* OutLoadIdIt = Out->LoadIds;
	
		// Populate FLoadIdMapping and init Out->LoadIds
		FInnerStructSchemaIterator SchemaIt(Schema);
		for (FInnerStruct Inner : Inners)
		{
			FStructSchemaId ReadId = SchemaIt.GrabMemberStruct(Inner.Name);
			*OutLoadIdIt++ = LoadIds.BindLoadId(ReadId, Inner.Id);
		}
		check((void*)OutLoadIdIt == SubsetSchemaData.GetData() + SubsetSchemaData.Num());

		// Return unstable pointer that's later replaced in CreateBatch()
		return Out;
	}

	// Get subset of Inners present in schema being read from
	static TConstArrayView<FInnerStruct> GetLoweredMembers(TConstArrayView<FInnerStruct> Inners, TConstArrayView<FMemberId> Names, TArray<FInnerStruct, TInlineAllocator<8>>& TmpSubset)
	{
		bool bKeepAll = true;
		TBitArray<> Keep(true, Inners.Num());
		TConstArrayView<FMemberId> ScanNames = Names;
		for (int32 Idx = 0; Idx < Inners.Num(); ++Idx)
		{
			if (const FMemberId* Name = Algo::Find(ScanNames, Inners[Idx].Name))
			{
				// Names must appear in order, limit future searches to later names
				ScanNames.RightChopInline(1 + Name - &ScanNames[0]);
			}
			else
			{
				bKeepAll = false;
				Keep[Idx] = false;
			}
		}

		if (bKeepAll)
		{
			return Inners;
		}
		else if (TOptional<TConstArrayView<FInnerStruct>> ContiguousSubset = GetContiguousSubset(Inners, Keep))
		{
			return ContiguousSubset.GetValue();
		}

		for (int32 Idx = 0; Idx < Inners.Num(); ++Idx)
		{
			if (Keep[Idx])
			{
				TmpSubset.Add(Inners[Idx]);
			}
		}
		return TmpSubset;
	}

	FLoadStructPlan CreatePlanInner(FLoadIdMapping Mapping)
	{
		const FStructSchema& From = ResolveStructSchema(BatchId, Mapping.ReadId);
		
		TConstArrayView<FInnerStruct> LoweredInners;
		if constexpr (bIsLoadIdBinderDummy)
		{
			if (const ICustomBinding* Custom = Customs.FindStruct(Mapping.Id))
			{
				FMemcpyLoadPlan Memcpy;
				Custom->PlanCustom(Memcpy);
				return Memcpy.Size ? FLoadStructPlan(Memcpy) : FLoadStructPlan(*Custom);
			}
		}
		else if (const ICustomBinding* Custom = Customs.FindStruct(Mapping.Id, /* out */ LoweredInners))
		{
			FMemcpyLoadPlan Memcpy;
			Custom->PlanCustom(Memcpy);
			if (Memcpy.Size)
			{
				return FLoadStructPlan(Memcpy);
			}

			TArray<FInnerStruct, TInlineAllocator<8>> TmpSubset;
			TConstArrayView<FInnerStruct> LoweredMembers = GetLoweredMembers(LoweredInners, From.GetMemberNames(), /* out */ TmpSubset);

			if (LoweredMembers.IsEmpty())
			{
				return FLoadStructPlan(*Custom);
			}
			
			return FLoadStructPlan(*CreateCustomLoadIdsPlan(Custom, From, LoweredMembers));
		}

		if (From.NumMembers)
		{
			const FStructDeclaration* Decl;
			if (const FSchemaBinding* To = Schemas.FindStruct(Mapping.Id, /* out */ Decl))
			{
				// Possible optimization - some simple memcpy cases doesn't need to resolve the declaration
				TConstArrayView<FMemberId> ToMemberIds = Decl->GetMemberOrder();
				return MakeLoadPlan(From, *To, ToMemberIds);	
			}
			
			// Type-erased structs are expected to be handled right away in a second iteration
			//     when processing added mappings.
			// Late-bound structs are expected to be handled during loading so that the batch
			//     can be updated with a new load plan before the struct data is to be loaded.
			UnboundSaveIds.Add(Mapping.ReadId);
			return FLoadStructPlan(GLoadNotBoundError);
		}

		return FLoadStructPlan(GLoadNoop);
	}

	static void CopyMemberBinding(FLeafMemberBinding Binding, /* in-out */ const FSchemaId*& InnerIdIt, FMemberLoadBinder& Out)
	{
		InnerIdIt += (Binding.Leaf.Type == ELeafBindType::Enum); // Skip enum schema
		Out.AddMember(Binding.Leaf.Pack(), static_cast<uint32>(Binding.Offset));
	}

	void CopyMemberBinding(FStructMemberBinding Binding, /* in-out */ const FSchemaId*& InnerIdIt, FMemberLoadBinder& Out)
	{
		check(!Binding.Type.IsSuper);
		check(!Binding.Type.IsDynamic); // TODO: Build out dynamic struct support
		FStructSchemaId LoadId = LoadIds.BindLoadId(static_cast<FStructSchemaId>(*InnerIdIt), Binding.Id);
		Out.AddMember(FMemberBindType(Binding.Type), static_cast<uint32>(Binding.Offset));
		Out.AddInnerId(LoadId);
		++InnerIdIt;
	}

	void CopyMemberBinding(FRangeMemberBinding Binding, /* in-out */ const FSchemaId*& InnerIdIt, FMemberLoadBinder& Out)
	{
		FMemberBindType InnermostType = Binding.InnerTypes[Binding.NumRanges - 1];
		Out.AddRange(MakeArrayView(Binding.RangeBindings, Binding.NumRanges), InnermostType, static_cast<uint32>(Binding.Offset));
		if (InnermostType.IsStruct())
		{
			FStructSchemaId ReadId = static_cast<FStructSchemaId>(*InnerIdIt);
			FStructSchemaId LoadId = LoadIds.BindLoadId(ReadId, Binding.InnermostId.Get().AsStructBindId());
			Out.AddInnerId(LoadId);
			++InnerIdIt;
		}
		else
		{
			InnerIdIt += (InnermostType.AsLeaf().Bind.Type == ELeafBindType::Enum); // Skip enum schema
		}
	}

	void CopyMemberBinding(/* in-out */ FMemberVisitor& BindIt, /* in-out */ const FSchemaId*& InnerIdIt, FMemberLoadBinder& Out)
	{
		switch (BindIt.PeekKind())
		{
			case EMemberKind::Leaf:		CopyMemberBinding(BindIt.GrabLeaf(), InnerIdIt, Out);	break;
			case EMemberKind::Range:	CopyMemberBinding(BindIt.GrabRange(), InnerIdIt, Out);	break;
			case EMemberKind::Struct:	CopyMemberBinding(BindIt.GrabStruct(), InnerIdIt, Out);	break;
			default:					check(false);											break;
		}
	}


	void CreateSubsetBindingWithoutEnumIds(const FStructSchema& From, const FSchemaBinding& To, TConstArrayView<FMemberId> ToNames, uint16 NumEnums)
	{
		check(To.NumMembers == ToNames.Num() + To.HasSuper());
		check(To.NumMembers >= From.NumMembers);
		check(IsAligned(SubsetSchemaData.Num(), alignof(FSchemaBinding)));

		// Allocate and init header 
		bool bSkipSuperSchema = SkipDeclaredSuperSchema(From.Inheritance);
		FSchemaBinding Header = { To.DeclId, From.NumMembers, From.NumInnerSchemas - NumEnums - bSkipSuperSchema, From.NumRangeTypes };
		int32 Offset = SubsetSchemaData.Num();
		SubsetSchemaData.AddUninitialized(Header.CalculateSize());
		FSchemaBinding* Schema = new (&SubsetSchemaData[Offset]) FSchemaBinding {Header};
		check(IsAligned(Schema, alignof(FSchemaBinding)));
		
		// Copy subset of member bindings...
		FMemberVisitor ToIt(To);
		FMemberLoadBinder Footer(*Schema);
		const FSchemaId* InnerIdIt = From.GetInnerSchemas() + bSkipSuperSchema;

		// ...first the unnamed super member...
		if (From.Inheritance != ESuper::No)
		{
			FBindId DeclBindId = ToIt.GrabSuper();
			if (UsesSuper(From.Inheritance))
			{
				FStructType FromType = From.Footer[0].AsStruct(); // To.Members[0].AsStruct().IsDynamic isn't set
				check(FromType.IsSuper);
				Footer.AddMember(FMemberBindType(FromType), 0);
			
				if (!FromType.IsDynamic)
				{
					FStructSchemaId ReadId = static_cast<FStructSchemaId>(*InnerIdIt);
					FStructSchemaId LoadId;
					if (From.Inheritance == ESuper::Reused)
					{
						LoadId = LoadIds.BindLoadId(ReadId, DeclBindId);
					}
					else
					{
						check(From.Inheritance == ESuper::Used);
						// TODO: Handle type erasure in the used super scenario too,
						// i.e. when reading a base of the declared super, needs UsedBindId.
						LoadId = ReadId;
					}
					Footer.AddInnerId(LoadId);
					++InnerIdIt;
				}
			}
		}

		// ...then a subset of named members
		const FMemberId* ToNameIt = ToNames.GetData();
		for (FMemberId FromName : From.GetMemberNames())
		{
			while (FromName != *ToNameIt++)
			{
				ToIt.SkipMember();
				check(ToNameIt != ToNames.end());
			}
			
			CopyMemberBinding(/* in-out */ ToIt, /* in-out */ InnerIdIt, /* out */ Footer);	
		}
		check(InnerIdIt == From.GetInnerSchemas() + From.NumInnerSchemas);
	}

	// @pre No enum members
	void CloneBindingWithReplacedStructIds(const FSchemaId* FromIds, const FSchemaBinding& To)
	{
		check(IsAligned(SubsetSchemaData.Num(), alignof(FSchemaBinding)));

		// Clone runtime FSchemaBinding including inner schema bind ids
		uint32 Size = To.CalculateSize();
		SubsetSchemaData.AddUninitialized(Size);
		FSchemaBinding* Schema = reinterpret_cast<FSchemaBinding*>(&SubsetSchemaData[SubsetSchemaData.Num() - Size]);
		check(IsAligned(Schema, alignof(FSchemaBinding)));
		FMemory::Memcpy(Schema, &To, Size);

		// Replace inner bind ids with batch load ids
		const FStructSchemaId* ReadIdIt = static_cast<const FStructSchemaId*>(FromIds);
		FInnerId* InnerIt = const_cast<FInnerId*>(Schema->GetInnerIds());
		for (FInnerId& Inner : MakeArrayView(InnerIt, To.NumInnerIds))
		{
			FBindId MemcopiedBindId = Inner.AsStructBindId();
			reinterpret_cast<FStructSchemaId&>(Inner) = LoadIds.BindLoadId(*ReadIdIt++, MemcopiedBindId);
		}
	}

	[[nodiscard]] FLoadStructPlan MakeSchemaLoadPlan(const FStructSchema& From, const FSchemaBinding& To, TConstArrayView<FMemberId> ToMemberIds)
	{
		uint16 NumEnums = CountEnums(From);
		if (From.NumMembers < To.NumMembers || NumEnums || HasIncompatibleSupers(From, To, RuntimeIds))
		{
			CreateSubsetBindingWithoutEnumIds(From, To, ToMemberIds, NumEnums);
		}
		else
		{
			check(From.NumMembers == To.NumMembers);
			check(From.NumInnerSchemas == To.NumInnerIds);
			check(From.NumRangeTypes == To.NumInnerRanges);

			if (From.NumInnerSchemas > 0)
			{
				CloneBindingWithReplacedStructIds(From.GetInnerSchemas(), To);
			}
			// else reuse existing bindings
		}

		// Pointer to created subset load schema will be remapped later
		return FLoadStructPlan(To, ELeafWidth::B32, !From.IsDense);
	}

	[[nodiscard]] TOptional<FMemcpyLoadPlan> TryMakeMemcpyPlan(const FStructSchema& From, const FSchemaBinding& To, TConstArrayView<FMemberId> ToNames)
	{
		// Can't memcopy sparse members nor range-bound members nor super structs
		if (!From.IsDense || From.NumRangeTypes > 0 || UsesSuper(From.Inheritance))
		{
			return NullOpt;
		}

		// Can't memcpy non-contiguous members
		TConstArrayView<FMemberId> FromNames = From.GetMemberNames();
		int32 SkipToIdx = ToNames.Find(FromNames[0]);
		check(SkipToIdx != INDEX_NONE);
		if (!Algo::Compare(FromNames, ToNames.Slice(SkipToIdx, FromNames.Num())))
		{	
			return NullOpt;
		}
		
		// Check all schema members are contiguous leaves
		const uint32* OffsetIt = To.GetOffsets() + SkipToIdx + To.HasSuper();
		const uint32 StartPos = *OffsetIt;
		uint32 EndPos = *OffsetIt;
		for (FMemberType Member : From.GetMemberTypes())
		{
			// Note: By adding FStructType::IsDense flag and some struct size lookup, memcpying of nested non-dynamic non-custom-bound structs might be possible
			uint32 Offset = *OffsetIt++;
			if (Offset != EndPos ||	// Non-contiguous or padded
				Member.IsStruct() || // Nested structs have a skippable size prefix that can't be memcopied
				Member.AsLeaf().Type == ELeafType::Bool) // bool values are stored as bits, see FBitCacheReader
			{
				return NullOpt;
			}

			EndPos += SizeOf(Member.AsLeaf().Width);
		}

		return FMemcpyLoadPlan(EndPos - StartPos, StartPos);
	}

	[[nodiscard]] FLoadStructPlan MakeLoadPlan(const FStructSchema& From, const FSchemaBinding& To, TConstArrayView<FMemberId> ToMemberIds)
	{
		TOptional<FMemcpyLoadPlan> Memcpy = TryMakeMemcpyPlan(From, To, ToMemberIds);
		return Memcpy ? FLoadStructPlan(Memcpy.GetValue()) : MakeSchemaLoadPlan(From, To, ToMemberIds);
	}
};

FLoadBatchPtr CreateLoadPlans(FSchemaBatchId BatchId, const FCustomBindings& Customs, const FSchemaBindings& Schemas, TConstArrayView<FStructId> RuntimeIds, ESchemaFormat Format)
{
	if (Format == ESchemaFormat::InMemoryNames)
	{
		FLoadIdBinderDummy LoadIds{Schemas.GetDebug()};
		return FLoadPlanner{BatchId, Customs, Schemas, RuntimeIds, LoadIds}.CreatePlans();
	}
	else
	{
		TConstArrayView<FDeclId> DeclIds(static_cast<const FDeclId*>(RuntimeIds.GetData()), RuntimeIds.Num());
		FLoadIdBinder LoadIds(DeclIds, static_cast<uint32>(DeclIds.Num()), Schemas.GetDebug());
		return FLoadPlanner{BatchId, Customs, Schemas, RuntimeIds, LoadIds}.CreatePlans();
	}
}

void UpdateLoadPlans(FLoadBatch& Batch, const FCustomBindings& Customs, const FSchemaBindings& Schemas, TConstArrayView<FStructId> RuntimeIds)
{
	TConstArrayView<FDeclId> DeclIds(static_cast<const FDeclId*>(RuntimeIds.GetData()), RuntimeIds.Num());
	FLoadIdBinder LoadIds(DeclIds, Batch.NumPlans, Schemas.GetDebug());
	FLoadPlanner{Batch.BatchId, Customs, Schemas, RuntimeIds, LoadIds}.UpdatePlans(Batch);
}

////////////////////////////////////////////////////////////////////////////

inline void SetBit(uint8& Out, uint8 Idx, bool bValue)
{
	checkSlow(Idx < 8);
	uint8 Mask(1 << Idx);
	if (bValue)
	{
		Out |= Mask;
	}
	else
	{
		Out &= ~Mask;
	}	
}

struct FLoadRangePlan
{
	ESizeType MaxSize;
	FOptionalStructSchemaId InnermostStruct;
	TConstArrayView<FMemberBindType> InnerTypes;
	const FRangeBinding* Bindings = nullptr;

	FLoadRangePlan Tail() const
	{
		return { InnerTypes[0].AsRange().MaxSize, InnermostStruct, InnerTypes.RightChop(1), Bindings + 1 };
	}
};

inline static FMemberBindType ToBindType(FMemberType Member)
{
	switch (Member.GetKind())
	{
	case EMemberKind::Leaf:		return FMemberBindType(Member.AsLeaf());
	case EMemberKind::Range:	return FMemberBindType(Member.AsRange());
	default:					return FMemberBindType(Member.AsStruct());	
	}
}

class FRangeLoader
{
public:
	static void LoadView(uint8* Member, FRangeLoadView Src, TConstArrayView<FRangeBinding> Bindings)
	{
		TArray<FMemberBindType, TFixedAllocator<16>> InnerTypes;
		InnerTypes.Add(ToBindType(Src.Schema.ItemType));
		if (Src.Schema.ItemType.IsRange())
		{
			for (const FMemberType* It = Src.Schema.NestedItemTypes; It; It = It->IsRange() ? (It + 1) : nullptr)
			{
				InnerTypes.Add(ToBindType(*It));
			}
		}
		check(Bindings.Num() == InnerTypes.Num());
			
		FOptionalStructSchemaId InnermostStruct = InnerTypes.Last().IsStruct() ? static_cast<FOptionalStructSchemaId>(Src.Schema.InnermostId) : NoId;
		ESizeType Unused = ESizeType::Uni;
		FLoadRangePlan Plan = { Unused, InnermostStruct, InnerTypes, Bindings.GetData() };
		
		LoadRangePlan(Member, Src.NumItems, Src.Values, Src.Schema.Batch, Plan);
	}

	// T is FByteReader& for normal load plans but FMemoryView for FRangeLoadView case where GrabRangeValues() is already called
	template<typename T>
	static void LoadRangePlan(uint8* Member, uint64 Num, T& ByteItOrValues, const FLoadBatch& Batch, const FLoadRangePlan& Range)
	{
		FRangeBinding Binding = Range.Bindings[0];
		FMemberBindType InnerType = Range.InnerTypes[0];
		
		if (Binding.IsLeafBinding())
		{
			LoadLeafRange(Member, Num, Binding.AsLeafBinding(), ByteItOrValues, UnpackNonBitfield(InnerType.AsLeaf()));
		}
		else if (Num)
		{
			const IItemRangeBinding& ItemBinding = Binding.AsItemBinding();
			switch (InnerType.GetKind())
			{
				case EMemberKind::Leaf:		LoadRangeValues(Member, Num, ItemBinding, ByteItOrValues, Batch, UnpackNonBitfield(InnerType.AsLeaf())); break;
				case EMemberKind::Range:	LoadRangeValues(Member, Num, ItemBinding, ByteItOrValues, Batch, Range.Tail()); break;
				case EMemberKind::Struct:	LoadRangeValues(Member, Num, ItemBinding, ByteItOrValues, Batch, Range.InnermostStruct.Get()); break;
			}
		}
		else
		{
			FLoadRangeContext NoItemsCtx{.Request = {Member, 0}};
			(Binding.AsItemBinding().MakeItems)(NoItemsCtx);
		}
	}	
	
	static void LoadLeafRange(uint8* Member, uint64 Num, const ILeafRangeBinding& Binding, FByteReader& ByteIt, FUnpackedLeafType Leaf)
	{
		FMemoryView Values = Num ? GrabRangeValues(ByteIt, Num, Leaf) : FMemoryView();
		LoadLeafRange(Member, Num, Binding, Values, Leaf);
	}

	static void LoadLeafRange(uint8* Member, uint64 Num, const ILeafRangeBinding& Binding, FMemoryView Values, FUnpackedLeafType Leaf)
	{
		Binding.LoadLeaves(Member, FLeafRangeLoadView(Values.GetData(), Num, Leaf));
	}

	template<class SchemaType>
	static void LoadRangeValues(uint8* Member, uint64 Num, const IItemRangeBinding& Binding, FByteReader& ByteIt, const FLoadBatch& Batch, SchemaType&& Schema)
	{
		LoadRangeValues(Member, Num, Binding, GrabRangeValues(ByteIt, Num, Schema), Batch, Forward<SchemaType>(Schema));
	}

	template<class SchemaType>
	static void LoadRangeValues(uint8* Member, uint64 Num, const IItemRangeBinding& Binding, FMemoryView Values, const FLoadBatch& Batch, SchemaType&& Schema)
	{
		FByteReader ValueIt(Values);
		FBitCacheReader BitIt; // Only used by ranges of ESizeType::Uni ranges
		FLoadRangeContext Ctx{.Request = {Member, Num}};
		
		while (Ctx.Request.Index < Num)
		{
			(Binding.MakeItems)(Ctx);
			CopyRangeValues(Ctx.Items, ValueIt, BitIt, Batch, Schema);
			Ctx.Request.Index += Ctx.Items.Num;
		}
		ValueIt.CheckEmpty();

		if (Ctx.Items.bNeedFinalize)
		{
			(Binding.MakeItems)(Ctx);
		}
	}

	static FMemoryView GrabRangeValues(FByteReader& ByteIt, uint64 Num, FUnpackedLeafType Leaf)
	{
		check(Num > 0);
		return ByteIt.GrabSlice(GetLeafRangeSize(Num, Leaf));
	}

	template<class SchemaType>
	static FMemoryView GrabRangeValues(FByteReader& ByteIt, uint64, SchemaType&&)
	{
		return ByteIt.GrabSkippableSlice();
	}
		
	static void CopyRangeValues(const FConstructedItems& Items, FByteReader& ByteIt, FBitCacheReader&, const FLoadBatch& Batch, FUnpackedLeafType Leaf)
	{
		if (Items.Size == SizeOf(Leaf.Width))
		{
			if (Leaf.Type != ELeafType::Bool)
			{
				FMemory::Memcpy(Items.Data, ByteIt.GrabBytes(Items.NumBytes()), Items.NumBytes());
			}
			else
			{
				FBoolRangeView Bits(ByteIt.GrabBytes(Align(Items.Num, 8)/8), Items.Num);
				uint8* It = Items.Data;
				for (bool bBit : Bits)
				{
					reinterpret_cast<bool&>(*It++) = bBit;
				}
			}
		}
		else // Strided
		{
			check(Items.Size > SizeOf(Leaf.Width));
			check(false); // todo
		}
	}
	
	static void CopyRangeValues(const FConstructedItems& Items, FByteReader& ByteIt, FBitCacheReader&, const FLoadBatch& Batch, FStructSchemaId Id)
	{
		uint64 ItemSize = Items.Size;
		uint8* End = Items.Data + Items.NumBytes();
		if (Items.bUnconstructed)
		{
			for (uint8* It = Items.Data; It != End; It += ItemSize)
			{
				PlainProps::ConstructAndLoadStruct(It, FByteReader(ByteIt.GrabSkippableSlice()), Id, Batch);
			}
		}
		else
		{
			for (uint8* It = Items.Data; It != End; It += ItemSize)
			{
				PlainProps::LoadStruct(It, FByteReader(ByteIt.GrabSkippableSlice()), Id, Batch);
			}
		}
	}
	
	static void CopyRangeValues(const FConstructedItems& Items, FByteReader& ByteIt, FBitCacheReader& BitIt, const FLoadBatch& Batch, const FLoadRangePlan& Plan)
	{
		uint64 ItemSize = Items.Size;
		for (uint8* It = Items.Data, *End = It + Items.NumBytes(); It != End; It += ItemSize)
		{
			uint64 Num = GrabRangeNum(Plan.MaxSize, ByteIt, BitIt);
			LoadRangePlan(It, Num, ByteIt, Batch, Plan);
		}
	}
};

////////////////////////////////////////////////////////////////////////////

template<bool bSparse, typename OffsetType>
class TMemberLoader : public FRangeLoader
{
public:
	TMemberLoader(FByteReader Values, FSchemaLoadPlan Schema, const FLoadBatch& InBatch)
	: Types(Schema.GetMembers())
	, Offsets(Schema.GetOffsets())
	, InnerStructSchemas(Schema.GetInnerSchemas())
	, InnerRangeTypes(Schema.GetInnerRangeTypes())
	, RangeBindings(Schema.GetRangeBindings())
	, Batch(InBatch)
	, ByteIt(Values)
	{}

	void Load(void* Struct)
	{
		SkipMissingSparseMembers();
	
		while (MemberIdx < Types.Num())
		{
			LoadMember(Struct);
			++MemberIdx;
			SkipMissingSparseMembers();
		}
	}

private:
	const TConstArrayView<FMemberBindType>		Types;
	const OffsetType* const						Offsets;
	const TConstArrayView<FStructSchemaId>		InnerStructSchemas;
	const TConstArrayView<FMemberBindType>		InnerRangeTypes;
	const FRangeBinding*						RangeBindings;
	const FLoadBatch&							Batch;
	
	FByteReader ByteIt;
	FBitCacheReader BitIt;
	int32 MemberIdx = 0;
	int32 InnerRangeIdx = 0;
	int32 InnerStructIdx = 0;	

	void SkipMissingSparseMembers() {}
	void SkipMissingSparseMembers() requires (bSparse)
	{
		// Make code changes in FMemberReader::SkipMissingSparseMembers() too

		while (MemberIdx < Types.Num() && BitIt.GrabNext(ByteIt))
		{
			FMemberBindType Type = Types.GetData()[MemberIdx];		
			EMemberKind Kind = Type.GetKind();
			if (Kind == EMemberKind::Struct)
			{
				InnerStructIdx += !(Type.AsStruct().IsDynamic);
			}
			else if (Kind == EMemberKind::Range)
			{
				FMemberBindType InnermostType = GrabInnerRangeTypes(InnerRangeTypes, /* in-out */ InnerRangeIdx).Last();
				InnerStructIdx += InnermostType.IsStruct() && !(InnermostType.AsStruct().IsDynamic);
			}
			++MemberIdx;
		}
	}

	void LoadMember(void* Struct)
	{
		FMemberBindType Type = Types[MemberIdx];
		uint8* Member = static_cast<uint8*>(Struct) + Offsets[MemberIdx];

		switch (Type.GetKind())
		{
			case EMemberKind::Leaf:		LoadMemberLeaf(Member, Type.AsLeaf()); break;
			case EMemberKind::Range:	LoadMemberRange(Member, GrabInnerRanges(Type.AsRange())); break;
			case EMemberKind::Struct:	LoadMemberStruct(Member, GrabInnerStruct(Type.AsStruct())); break;
		}
	}

	inline FStructSchemaId GrabInnerStruct(FStructBindType Type)
	{
		return Type.IsDynamic ? FStructSchemaId { ByteIt.Grab<uint32>() } : InnerStructSchemas[InnerStructIdx++];
	}

	FLoadRangePlan GrabInnerRanges(FRangeBindType Type)
	{
		const FRangeBinding* Bindings = RangeBindings + InnerRangeIdx;
		TConstArrayView<FMemberBindType> InnerTypes = GrabInnerRangeTypes(InnerRangeTypes, /* in-out */ InnerRangeIdx);	
		FOptionalStructSchemaId InnermostStruct = InnerTypes.Last().IsStruct() ? ToOptional(GrabInnerStruct(InnerTypes.Last().AsStruct())) : NoId;
		return { Type.MaxSize, InnermostStruct, InnerTypes, Bindings };
	}
	
	void LoadMemberLeaf(uint8* Member, FLeafBindType Leaf)
	{
		switch (Leaf.Bind.Type)
		{
			case ELeafBindType::Bool:
				reinterpret_cast<bool&>(*Member) = BitIt.GrabNext(ByteIt);
				break;
			case ELeafBindType::BitfieldBool:
				SetBit(*Member, Leaf.Bitfield.Idx, BitIt.GrabNext(ByteIt));
				break;
			default:
				switch (Leaf.Basic.Width)
				{
					case ELeafWidth::B8:	FMemory::Memcpy(Member, ByteIt.GrabBytes(1), 1); break;
					case ELeafWidth::B16:	FMemory::Memcpy(Member, ByteIt.GrabBytes(2), 2); break;
					case ELeafWidth::B32:	FMemory::Memcpy(Member, ByteIt.GrabBytes(4), 4); break;
					case ELeafWidth::B64:	FMemory::Memcpy(Member, ByteIt.GrabBytes(8), 8); break;
				}
				break;
		}
	}

	void LoadMemberStruct(uint8* Member, FStructSchemaId Id)
	{
		PlainProps::LoadStruct(Member, FByteReader(ByteIt.GrabSkippableSlice()), Id, Batch);
	}

	void LoadMemberRange(uint8* Member, const FLoadRangePlan& Plan)
	{
		uint64 Num = GrabRangeNum(Plan.MaxSize, ByteIt, BitIt);
		LoadRangePlan(Member, Num, ByteIt, Batch, Plan);
	}
};

////////////////////////////////////////////////////////////////////////////

void LoadStruct(void* Dst, FByteReader Src, FStructSchemaId LoadId, const FLoadBatch& Batch)
{
	FLoadStructPlan Plan = Batch[LoadId];
	
	if (Plan.IsSchema())
	{
		if (Plan.IsSparseSchema())
		{
			TMemberLoader< true, uint32>(Src, Plan.AsSchema(), Batch).Load(Dst);
		}
		else
		{
			TMemberLoader<false, uint32>(Src, Plan.AsSchema(), Batch).Load(Dst);
		}
	}
	else if (Plan.IsMemcpy())
	{
		static_assert(PLATFORM_LITTLE_ENDIAN);
		FMemcpyLoadPlan MemcpyPlan = Plan.AsMemcpy();
		Src.CheckSize(MemcpyPlan.Size);
		FMemory::Memcpy(static_cast<uint8*>(Dst) + MemcpyPlan.Offset, Src.Peek(), MemcpyPlan.Size);
	}
	else
	{
		FSchemaLoadHandle Schema{LoadId, Batch};
		Plan.AsCustom().LoadCustom(Dst, { Schema, Src }, ECustomLoadMethod::Assign);
	}
}

void ConstructAndLoadStruct(void* Dst, FByteReader Src, FStructSchemaId Id, const FLoadBatch& Batch)
{
	FLoadStructPlan Plan = Batch[Id];
	checkf(!Plan.IsSchema(), TEXT("Non-default constructible types requires ICustomBinding or in rare cases memcpying"));

	if (Plan.IsMemcpy())
	{
		Src.CheckSize(Plan.AsMemcpy().Size);
		FMemory::Memcpy(static_cast<uint8*>(Dst) + Plan.AsMemcpy().Offset, Src.Peek(), Plan.AsMemcpy().Size);
	}
	else
	{
		FSchemaLoadHandle Schema{Id, Batch};
		Plan.AsCustom().LoadCustom(Dst, { Schema, Src }, ECustomLoadMethod::Construct);
	}
}

////////////////////////////////////////////////////////////////////////////

FRangeLoadView FNestedRangeLoadIterator::operator*() const 
{
	FByteReader PeekBytes = ByteIt;
	FBitCacheReader PeekBits = BitIt;

	FRangeLoadSchema OutSchema = { Schema.NestedItemTypes[0], Schema.InnermostId, /* Only valid for nested ranges */ Schema.NestedItemTypes + 1, Schema.Batch };
	uint64 OutNumItems = GrabRangeNum(Schema.ItemType.AsRange().MaxSize, /* in-out */ PeekBytes,  /* in-out */ PeekBits);
	FMemoryView OutValues = GrabRangeValues(OutNumItems, OutSchema.ItemType, /* in-out */ PeekBytes);

	return { OutSchema, OutNumItems, OutValues };
}

void FNestedRangeLoadIterator::operator++()
{
	uint64 Num = GrabRangeNum(Schema.ItemType.AsRange().MaxSize, /* in-out */ ByteIt, /* in-out */ BitIt);
	(void)GrabRangeValues(Num, Schema.NestedItemTypes[0], /* in-out */ ByteIt);
}

///////////////////////////////////////////////////////////////////////////

FStructRangeLoadView FRangeLoadView::AsStructs() const
{
	check(IsStructRange());
	FStructSchemaId LoadId = static_cast<FStructSchemaId>(Schema.InnermostId.Get());
	return { NumItems, Values, {LoadId, Schema.Batch} };
}

FNestedRangeLoadView FRangeLoadView::AsRanges() const
{
	check(IsNestedRange());
	return { NumItems, Values, Schema };
}

////////////////////////////////////////////////////////////////////////////

static FStructView ToReadView(FStructLoadView In)
{
	FStructSchemaHandle ReadSchema = { In.Schema.Batch.GetReadId(In.Schema.LoadId), In.Schema.Batch.BatchId };
	return { ReadSchema, In.Values };
}

FMemberLoader::FMemberLoader(FStructLoadView In)
: Reader(ToReadView(In))
, LoadIdIt(In.Schema.Batch[In.Schema.LoadId].GetInnerLoadIds())
, Batch(In.Schema.Batch)
{}

FRangeLoadView FMemberLoader::GrabRange()
{
	const FRangeView In = Reader.GrabRange();

	// Replace ReadId with LoadId
	FRangeSchema InSchema = GetSchema(In);
	FOptionalSchemaId InnermostId = InSchema.InnermostSchema;
	if (LoadIdIt && InnermostId && GetInnermostType(InSchema).IsStruct())
	{
		check(LoadIdIt->Idx > InnermostId.Get().Idx);
		check(InnermostId == Batch.GetReadId(*LoadIdIt));
		InnermostId = *LoadIdIt++;
	}

	FRangeLoadSchema OutSchema = { InSchema.ItemType, InnermostId, InSchema.NestedItemTypes, Batch };
	return { OutSchema, In.Num(), GetValues(In) };
}

FStructLoadView FMemberLoader::GrabStruct()
{
	const FStructView In = Reader.GrabStruct();
	FStructSchemaId LoadId = LoadIdIt ? *LoadIdIt++ : In.Schema.Id;
	check(In.Schema.Id == Batch.GetReadId(LoadId));

	return { {LoadId, Batch}, In.Values };
}

////////////////////////////////////////////////////////////////////////////

void LoadRange(void* Dst, FRangeLoadView Src, TConstArrayView<FRangeBinding> Bindings)
{
	FRangeLoader::LoadView(static_cast<uint8*>(Dst), Src, Bindings);
}

void LoadRange(void* Dst, FByteReader& SrcBytes, FBitCacheReader& SrcBits, ESizeType MaxSize, FRangeLoadSchema Schema, TConstArrayView<FRangeBinding> Bindings)
{
	if (uint64 Num = GrabRangeNum(MaxSize, SrcBytes, SrcBits))
	{
		FMemoryView Values = GrabRangeValues(Num, Schema.ItemType, SrcBytes);
		LoadRange(Dst, {Schema, Num, Values}, Bindings);
	}
}

void LoadStruct(void* Dst, FStructLoadView Src)
{
	LoadStruct(Dst, Src.Values, Src.Schema.LoadId, Src.Schema.Batch);
}

void ConstructAndLoadStruct(void* Dst, FStructLoadView Src)
{
	ConstructAndLoadStruct(Dst, Src.Values, Src.Schema.LoadId, Src.Schema.Batch);
}

void FSchemaLoadHandle::GetInnerLoadIds(TArrayView<FOptionalSchemaId> Out) const
{
	const FStructSchema& ReadSchema = FStructSchemaHandle{ Batch.GetReadId(LoadId), Batch.BatchId }.Resolve();
	TConstArrayView<FMemberType> MemberTypes = ReadSchema.GetMemberTypes();
	TConstArrayView<FMemberType> RangeTypes = ReadSchema.GetRangeTypes();
	check(Out.Num() == MemberTypes.Num());

	FOptionalSchemaId*  OutIt = Out.GetData();
	uint32 RangeTypeIdx = 0;
	if (const FStructSchemaId* InnerLoadIds = Batch[LoadId].GetInnerLoadIds())
	{
		for (FMemberType Member : MemberTypes)
		{
			FMemberType Innermost = Member.IsRange() ? GrabInnerRangeTypes(RangeTypes, /* in-out */ RangeTypeIdx).Last() : Member;
			*OutIt++ = Innermost.IsStruct() ? ToOptional(FSchemaId(*InnerLoadIds++)) : NoId;	
		}
	}
	else
	{
		const FSchemaId* InnerIdIt = ReadSchema.GetInnerSchemas();
		for (FMemberType Member : MemberTypes)
		{
			FMemberType Innermost = Member.IsRange() ? GrabInnerRangeTypes(RangeTypes, /* in-out */ RangeTypeIdx).Last() : Member;
			if (Innermost.IsStruct())
			{
				*OutIt++ = *InnerIdIt++;
			}
			else
			{
				*OutIt++ = NoId;
				InnerIdIt += Innermost.AsLeaf().Type == ELeafType::Enum;
			}
		}
	}
	check (OutIt == Out.end());
}

void LoadSoleStruct(void* Dst, FStructLoadView Src)
{
	// Todo: Optimize
	LoadStruct(Dst, FMemberLoader(Src).GrabStruct());
	//FLoadStructPlan Plan = Src.Schema.Batch[Src.Schema.LoadId];
}

} // namespace PlainProps
