// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsModule.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FPlainPropsModule, PlainProps);