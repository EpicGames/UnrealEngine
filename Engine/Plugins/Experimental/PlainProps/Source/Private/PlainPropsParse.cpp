// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsParse.h"
#include "Containers/Map.h"
#include "Containers/StringConv.h"
#include "Misc/AsciiSet.h"
#include "Misc/StringBuilder.h"
#include "PlainPropsBind.h"
#include "PlainPropsBuild.h"
#include "PlainPropsSpecify.h"
#include "PlainPropsBuildSchema.h"
#include "PlainPropsInternalBuild.h"
#include "PlainPropsInternalParse.h"
#include "PlainPropsInternalPrint.h"
#include "PlainPropsInternalText.h"
#include "PlainPropsVisualize.h"
#include "PlainPropsWrite.h"
#include <charconv>

namespace PlainProps
{

// specialization used by FTextIndexer/TIdIndexer<FSensitiveUtf8View>
template<>
void AppendString(FUtf8StringBuilderBase& Out, const FSensitiveUtf8View& Str)
{
	Out.Append(Str);
}

TStringBuilderWithBuffer<TCHAR, 128> Print(FToken Token)
{
	return WriteToString<128>(FStringView(StringCast<TCHAR>(Token.Str, Token.Len)));
}

inline const FParsedStructSchema& ResolveStructSchema(const FParsedSchemas& Schemas, FStructId Id)
{
	check(Id.Idx < static_cast<uint32>(Schemas.Structs.Num()));
	return Schemas.Structs[Id.Idx];
}

inline const FParsedEnumSchema& ResolveEnumSchema(const FParsedSchemas& Schemas, FEnumId Id)
{
	check(Id.Idx < static_cast<uint32>(Schemas.Enums.Num()));
	return Schemas.Enums[Id.Idx];
}

///////////////////////////////////////////////////////////////////////////////

static constexpr FUnpackedLeafType GLeaves[] = {
	ReflectArithmetic<bool>,
	ReflectArithmetic<int8>,
	ReflectArithmetic<int16>,
	ReflectArithmetic<int32>,
	ReflectArithmetic<int64>,
	ReflectArithmetic<uint8>,
	ReflectArithmetic<uint16>,
	ReflectArithmetic<uint32>,
	ReflectArithmetic<uint64>,
	ReflectArithmetic<float>,
	ReflectArithmetic<double>,
	FUnpackedLeafType(ELeafType::Hex, ELeafWidth::B8),
	FUnpackedLeafType(ELeafType::Hex, ELeafWidth::B16),
	FUnpackedLeafType(ELeafType::Hex, ELeafWidth::B32),
	FUnpackedLeafType(ELeafType::Hex, ELeafWidth::B64),
	FUnpackedLeafType(ELeafType::Enum, ELeafWidth::B8),
	FUnpackedLeafType(ELeafType::Enum, ELeafWidth::B16),
	FUnpackedLeafType(ELeafType::Enum, ELeafWidth::B32),
	FUnpackedLeafType(ELeafType::Enum, ELeafWidth::B64),
	ReflectArithmetic<char8_t>,
	ReflectArithmetic<char16_t>,
	ReflectArithmetic<char32_t>
};

template<>
TOptional<ESizeType> Parse(FUtf8StringView String)
{
	if (String.Len() >= 2)
	{
		constexpr uint8 N = sizeof(GLiterals.Ranges)/sizeof(GLiterals.Ranges[0]);
		for (uint8 I = (String[0] == 'i'); I < N; I += 2)
		{
			if (String == GLiterals.Ranges[I])
			{
				return ESizeType{ I };
			}
		}
	}
	return NullOpt;
}

template<>
TOptional<FUnpackedLeafType> Parse(FUtf8StringView String)
{
	static int32 MaxLeafLen = 0;
	static TMap<FUtf8StringView, FUnpackedLeafType> LeavesMap = [](int32& MaxLeafLen)
	{
		TMap<FUtf8StringView, FUnpackedLeafType> Map;
		for (FUnpackedLeafType Leaf : GLeaves)
		{
			FAnsiStringView String = ToString(Leaf);
			LeavesMap.Add(String, Leaf);
			if (String.Len() > MaxLeafLen)
			{
				MaxLeafLen = String.Len();
			}
		}
		return Map;
	}(MaxLeafLen);
	
	if (String.Len() > MaxLeafLen)
	{
		return NullOpt;
	}

	if (FUnpackedLeafType* Leaf = LeavesMap.Find(String))
	{
		return *Leaf;
	}
	return NullOpt;
}

template<>
TOptional<ELeafWidth> Parse(FUtf8StringView String)
{
	constexpr uint8 N = sizeof(GLiterals.Widths)/sizeof(GLiterals.Widths[0]);
	for (uint8 I = 0; I < N; ++I)
	{
		if (String == GLiterals.Widths[I])
		{
			return ELeafWidth{ I };
		}
	}
	return NullOpt;
}

///////////////////////////////////////////////////////////////////////////////

template<Arithmetic T> 
TOptional<T> Parse(FUtf8StringView String) requires (std::is_integral_v<T>)
{
	const char* First = reinterpret_cast<const char*>(String.GetData());
	const char* Last = First + String.Len();
	T Out;
	std::from_chars_result Result = std::from_chars(First, Last, Out, 10);
	return Result.ec == std::errc{} ? Out : TOptional<T>{ NullOpt };
}

template<UnsignedIntegral T> 
TOptional<T> ParseHex(FUtf8StringView String)
{
	const char* First = reinterpret_cast<const char*>(String.GetData());
	const char* Last = First + String.Len();
	T Out;
	std::from_chars_result Result = std::from_chars(First, Last, Out, 16);
	return Result.ec == std::errc{} ? Out : TOptional<T>{ NullOpt };
}

template<Arithmetic T> 
TOptional<T> Parse(FUtf8StringView String) requires (std::is_floating_point_v<T>)
{
	const char* First = reinterpret_cast<const char*>(String.GetData());
	const char* Last = First + String.Len();
	T Out;
	std::from_chars_result Result = std::from_chars(First, Last, Out, std::chars_format::general);
	return Result.ec == std::errc{} ? Out : TOptional<T>{ NullOpt };
}

template<>
TOptional<bool> Parse(FUtf8StringView String)
{
	if (String == GLiterals.True)
	{
		return true;
	}
	if (String == GLiterals.False)
	{
		return false;
	}
	return NullOpt;
}

static inline TOptional<uint32> ParseCodepoint4(FUtf8StringView HexString)
{
	check(HexString.Len() == 4);
	uint32 Codepoint = 0;
	for (int32 I = 0; I < 4; ++I)
	{
		Codepoint <<= 4;
		UTF8CHAR C = HexString[I];
		if (C >= '0' && C <= '9')
		{
			Codepoint += C - '0';
		}
		else if (C >= 'a' && C <= 'f')
		{
			Codepoint += C + 10 - 'a';
		}
		else if (C >= 'A' && C <= 'F')
		{
			Codepoint += C + 10 - 'A';
		}
		else
		{
			return NullOpt;
		}
	}
	return Codepoint;
}

// Matches PlainPropsPrint EscapeChar()
// The quotation mark (U+0022), backslash (U+005C),
// and control characters U+0000 to U+001F are escaped (JSON Standard ECMA-404).
// PP-TEXT: Missing support for Json two-character escape sequences \uxxxx\uxxxx
// PP-TEXT: Missing support for Yaml escaping
static inline TOptional<uint32> GrabEscapedCodepoint(FUtf8StringView& String)
{
	if (String.Len() < 2)
	{
		return NullOpt;
	}
	const UTF8CHAR* C = String.GetData();
	if (*C++ != '\\')
	{
		return NullOpt;
	}
	int32 Len = 2;
	TOptional<uint32> Codepoint;
	switch (*C)
	{
		case '"':	Codepoint = *C;		break;
		case '\\':	Codepoint = *C;		break;
		case 'b':	Codepoint = '\b';	break;
		case 'f':	Codepoint = '\f';	break;
		case 'n':	Codepoint = '\n';	break;
		case 'r':	Codepoint = '\r';	break;
		case 't':	Codepoint = '\t';	break;
		case 'u':
			if (String.Len() >= 6)
			{
				if ((Codepoint = ParseCodepoint4(String.Mid(2, 4))))
				{
					Len = 6;
					break;
				}
			}
		default:
			checkf(false, TEXT("Invalid escape sequence '%s'"), *Print(String));
			break;
	}
	if (Codepoint)
	{
		String.RightChopInline(Len);
	}
	return Codepoint;
}

template<>
TOptional<char8_t> Parse(FUtf8StringView String)
{
	if (String.Len() == 1 && String[0] <= 127)
	{
		return static_cast<char8_t>(String[0]);
	}
	if (TOptional<uint32> Codepoint = GrabEscapedCodepoint(String))
	{
		if (Codepoint.GetValue() <= 127)
		{
			return static_cast<char8_t>(Codepoint.GetValue());
		}
	}
	return NullOpt;
}

template<>
TOptional<char16_t> Parse(FUtf8StringView String)
{
	if (String.Len() == 1 && String[0] <= 127)
	{
		return static_cast<char16_t>(String[0]);
	}
	if (TOptional<uint32> Codepoint = GrabEscapedCodepoint(String))
	{
		if (Codepoint.GetValue() <= 0xFFFF)
		{
			return static_cast<char16_t>(Codepoint.GetValue());
		}
	}
	UTF16CHAR Dest;
	if (UTF16CHAR* DestEnd = FPlatformString::Convert(&Dest, 1, String.GetData(), String.Len()))
	{
		check(DestEnd == &Dest + 1);
		return static_cast<char16_t>(Dest);
	}
	return NullOpt;
}

template<>
TOptional<char32_t> Parse(FUtf8StringView String)
{
	if (String.Len() == 1 && String[0] <= 127)
	{
		return static_cast<char32_t>(String[0]);
	}
	if (TOptional<uint32> Codepoint = GrabEscapedCodepoint(String))
	{
		if (Codepoint.GetValue() <= 0xFFFF)
		{
			return static_cast<char32_t>(Codepoint.GetValue());
		}
	}
	// PP-TEXT: FPlatformString::Convert missing for UTF32CHAR in Parse()
	checkf(false, TEXT("Missing conversion from UTF8CHAR to UTF32CHAR"));
	return NullOpt;
}

///////////////////////////////////////////////////////////////////////////////

constexpr FAsciiSet LinebreakSet("\r\n");
constexpr FAsciiSet WhitespaceSet(" ");
constexpr FAsciiSet SingleQuoteSet("\'");
constexpr FAsciiSet DoubleQuoteSet("\"");
constexpr FAsciiSet BackslashSet("\\");
constexpr FAsciiSet ScopeSet(".");
constexpr FAsciiSet OpenRangeSet("(");
constexpr FAsciiSet CloseRangeSet(")");
constexpr FAsciiSet RangeSet(OpenRangeSet | CloseRangeSet);
constexpr FAsciiSet OpenParameterSet("<[");
constexpr FAsciiSet CloseParameterSet(">]");
constexpr FAsciiSet NextParameterSet(",");
constexpr FAsciiSet ParameterSet(OpenParameterSet | CloseParameterSet | NextParameterSet);

constexpr bool PeekIsColon(FUtf8StringView View)			{ return *View.GetData() == ':'; }
constexpr bool IsRangeTokenLine(FUtf8StringView View)		{ return *View.GetData() == '-'; }
constexpr bool IsEmptyOrCommentLine(FUtf8StringView View)	{ return View.IsEmpty() || *View.GetData() == '#'; }

///////////////////////////////////////////////////////////////////////////////

bool TokenizeType(FTypeTokens& Out, FUtf8StringView& String)
{
	check(String != GLiterals.Dynamic);

	// tokenize scopes and concrete typename
	FUtf8StringView Typename = FAsciiSet::FindPrefixWithout(String, ParameterSet);
	if (int32 Len = Typename.Len())
	{
		FUtf8StringView Scope = FAsciiSet::FindPrefixWithout(Typename, ScopeSet);
		while (Scope.Len() < Typename.Len())
		{
			Out.Scopes.Add(Scope);
			Typename.RightChopInline(Scope.Len() + 1);
			Scope = FAsciiSet::FindPrefixWithout(Typename, ScopeSet);
		}
		String.RightChopInline(Len);
	}
	Out.Typename = Typename;

	// tokenize parameters for parametric types
	UTF8CHAR OpenDelimiter{ };
	while (String.Len() > 0)
	{ 
		UTF8CHAR C = String[0];

		if (OpenParameterSet.Contains(C))
		{
			OpenDelimiter = C;
			String.RightChopInline(1);
			FTypeTokens Param;
			if (!TokenizeType(Param, String))
			{
				return false;
			}
			Out.Parameters.Add(Param);
		}
		else if (CloseParameterSet.Contains(C))
		{
			// PP-TEXT: Remove hardcoded parameter delimiters when tweaking print format
			if ((OpenDelimiter == '<' && C == '>') || (OpenDelimiter == '[' && C == ']'))
			{
				String.RightChopInline(1);
			}
			else if (OpenDelimiter)
			{
				checkf(false, TEXT("Mismatched delimiter '%s'"), *Print(String));
				return false;
			}
			return true;
		}
		else if (C == ',')
		{
			if (OpenDelimiter)
			{
				String.RightChopInline(1);
				FTypeTokens Param;
				if (!TokenizeType(Param, String))
				{
					return false;
				}
				Out.Parameters.Add(Param);
			}
			else
			{
				return true;
			}
		}
		else
		{
			checkf(false, TEXT("Invalid or unexpected type character '%s'"), *Print(String));
			return false;
		}
	}
	return true;
}

TOptional<FTypeTokens> TokenizeType(FUtf8StringView String)
{
	FTypeTokens Out;
	if (TokenizeType(Out, String))
	{
		return MoveTemp(Out);
	}
	return NullOpt;
}

FType MakeType(const FTypeTokens& TypeTokens, FTextIndexer& Names)
{
	// build parameter array for parametric types
	TArray<FType> Params;
	if (int32 Num = TypeTokens.Parameters.Num())
	{
		Params.Reserve(Num);
		for (const FTypeTokens& Parameter : TypeTokens.Parameters)
		{
			Params.Emplace(MakeType(Parameter, Names));
		}
	}

	// build scopes and concrete typename
	const FTypenameId TypenameId = Names.MakeTypename(TypeTokens.Typename);
	FScopeId ScopeId = TypeTokens.Scopes.Num() ? Names.MakeScope(TypeTokens.Scopes[0]) : FScopeId{ NoId };
	for (const FUtf8StringView& Scope : MakeArrayView(TypeTokens.Scopes).RightChop(1))
	{
		ScopeId = Names.NestScope(ScopeId, Scope);
	}

	// last create the full type
	if (Params.Num())
	{
		if (TypeTokens.Typename.Len())
		{
			return Names.MakeParametricType({ ScopeId, TypenameId }, Params);
		}
		return Names.MakeAnonymousParametricType(Params);
	}
	return { ScopeId, TypenameId };
}

////////////////////////////////////////////////////////////////////////////////

template <UnsignedIntegral T>
static void ParseEnumValue(
	FUtf8StringView String,
	T& OutValue,
	const FParsedEnumSchema& Schema,
	const FTextIndexer& Names)
{
	OutValue = 0;

	if (String == GLiterals.UnsetFlagEnum)
	{
		return;
	}

	while (!String.IsEmpty())
	{
		static constexpr FAsciiSet Separators(" |");

		String = FAsciiSet::TrimPrefixWith(String, Separators);
		FUtf8StringView Element = FAsciiSet::FindPrefixWithout(String, Separators);
		check(!Element.IsEmpty());

		String = String.Mid(Element.Len());

		bool bEnumeratorFound = false;
		for (const auto& [Name, Constant] : Schema.Enumerators)
		{
			FUtf8StringView NameView = Names.ResolveName(Name);
			if (NameView != Element)
			{
				continue;
			}

			OutValue |= Constant;
			bEnumeratorFound = true;
			break;
		}

		checkf(bEnumeratorFound, TEXT("Unable to match enumerator '%.*hs'"), String.Len(), String.GetData());
	}
}

////////////////////////////////////////////////////////////////////////////////

struct FLineInfo
{
	FUtf8StringView			View;
	FUtf8StringView			FullView;
	uint32					LineNumber		= 0;
	int32					NumSpaces		= 0;
	bool					HasRangeToken	= false;
	bool					EndOfFile		= false;
};

class FMarkupTokenizer
{
public:
	virtual						~FMarkupTokenizer() = default;
	virtual bool				HasMore() const = 0;
	virtual TOptional<FToken>	GrabToken() = 0;

	// PP-TEXT: GetLastError is a very temporary API while errors are unexpected
	FUtf8StringView GetLastError() const;

protected:
							FMarkupTokenizer(FUtf8StringView InStringView);
	FLineInfo				ReadOneLine();
	void					SetLastError(FUtf8StringView Message, FUtf8StringView Context);

private:
	FUtf8StringView			ReadView;
	TUtf8StringBuilder<256>	LastError;
	uint32					LineNumber = 0;
};

FMarkupTokenizer::FMarkupTokenizer(FUtf8StringView InStringView)
: ReadView(InStringView)
{
}

FUtf8StringView FMarkupTokenizer::GetLastError() const
{
	return LastError;
}

FLineInfo FMarkupTokenizer::ReadOneLine()
{
	if (ReadView.IsEmpty())
	{
		return {
			.View = ReadView,
			.FullView = ReadView,
			.LineNumber = LineNumber,
			.EndOfFile = true
		};
	}

	const FUtf8StringView LeadingSpaces = FAsciiSet::FindPrefixWith(ReadView, WhitespaceSet);
	ReadView.RightChopInline(LeadingSpaces.Len());

	const FUtf8StringView Line = FAsciiSet::FindPrefixWithout(ReadView, LinebreakSet);
	ReadView.RightChopInline(Line.Len());

	const FUtf8StringView TrailingLineBreaks = FAsciiSet::FindPrefixWith(ReadView, LinebreakSet);
	ReadView.RightChopInline(TrailingLineBreaks.Len());

	return { Line, Line, ++LineNumber, LeadingSpaces.Len() };
}

void FMarkupTokenizer::SetLastError(FUtf8StringView Message, FUtf8StringView Context)
{
	LastError.Reset();
	if (Message.Len())
	{
		LastError.Appendf("'%.*s' in line %u: '%.*s'",
			Message.Len(), Message.GetData(),
			LineNumber,
			Context.Len(),
			Context.GetData());
	}
}

////////////////////////////////////////////////////////////////////////////////

class FYamlTokenizer
	: public FMarkupTokenizer
{
public:
	FYamlTokenizer(FUtf8StringView InStringView);
	~FYamlTokenizer();

	virtual bool HasMore() const override { return ReadLine.EndOfFile == false || ScopeStack.Num() > 0; }
	virtual TOptional<FToken> GrabToken() override;

private:
	enum class EScope : uint8
	{
		Document,
		Struct,
		Range
	};

	static constexpr EToken ScopeEndTokens[3] =
	{
		EToken::EndDocument,
		EToken::EndStruct,
		EToken::EndRange
	};

	struct FScopeInfo
	{
		EScope					Scope;
		int32					NumSpaces		= 0;
	};

	TOptional<FToken>		 	GrabBeginDocument();
	TOptional<FToken>			GrabTokenInternal();

	void						PushScopeStack(EScope Scope);
	FToken						PopScopeStack();

	void						AdvanceReadLines();

	TArray<FScopeInfo, TInlineAllocator<32>>	ScopeStack;
	TOptional<FToken>							CachedToken;
	FLineInfo									ReadLine;
	FLineInfo									NextLine;
};

////////////////////////////////////////////////////////////////////////////////

class FTokenReader
{
	const FToken* It = nullptr;
	const FToken* End = nullptr;

public:
	FTokenReader(TConstArrayView64<FToken> Tokens)
		: It(Tokens.GetData())
		, End(Tokens.GetData() + Tokens.Num())
	{}

	bool HasMore() const							{ return It < End; }
	const FToken* Peek() const						{ check(It < End); return It; }
	[[nodiscard]] FToken GrabToken()				{ check(It < End); return *It++; }
	[[nodiscard]] FToken GrabToken(EToken Token)	{ check(It->Token == Token); return GrabToken(); }
};

class FScopedTokenReader
{
public:
	explicit FScopedTokenReader(FTokenReader& InTokens)
		: Tokens(InTokens), Depth(InTokens.Peek()->Depth)
	{ }

	FScopedTokenReader(FTokenReader& InTokens, FToken ParentToken)
		: Tokens(InTokens), Depth(ParentToken.Depth + 1)
	{ }

	bool HasMore() const							{ return Tokens.Peek()->Depth >= Depth; }
	uint16 GetDepth() const							{ return Depth; }
	FTokenReader& GetTokens()						{ return Tokens; }
	const FToken* Peek() const						{ return Tokens.Peek(); }
	FToken GrabToken()								{ check(HasMore()); return Tokens.GrabToken(); }
	[[nodiscard]] FToken GrabToken(EToken Token)	{ return Tokens.GrabToken(Token); }

private:
	FTokenReader& Tokens;
	uint16 Depth;
};

///////////////////////////////////////////////////////////////////////////////

class FEnumSchemaParser
{
public:
	FEnumSchemaParser(FTokenReader& InTokens, FTextIndexer& InNames, FEnumDeclarations& InEnums)
	: ScopedReader(InTokens)
	, Names(InNames)
	, Declarations(InEnums)
	{}

	bool HasMore() const { return ScopedReader.HasMore(); }

	bool ParseEnumSchema(FParsedEnumSchema& Out);

private:
	FParsedEnumSchema DeclareEnumSchema(
		const FTypeTokens& TypeTokens,
		EEnumMode Mode,
		ELeafWidth Width,
		TConstArrayView<FUtf8StringView> ConstantNames,
		TConstArrayView<uint64> ConstantValues);

	FScopedTokenReader	ScopedReader;
	FTextIndexer&	  	Names;
	FEnumDeclarations& 	Declarations;
};

////////////////////////////////////////////////////////////////////////////////

class FStructSchemaParser
{
public:
	FStructSchemaParser(FTokenReader& InTokens, FTextIndexer& InNames, FStructDeclarations& InStructs, FParsedSchemas& InSchemas)
	: ScopedReader(InTokens), Names(InNames), Declarations(InStructs), Schemas(InSchemas)
	{}

	bool HasMore() const { return ScopedReader.HasMore(); }

	bool ParseStructSchema(FParsedStructSchema& Out);

private:
	bool ParseMemberSchema(FParsedMemberSchema& Out, FUtf8StringView String);

	FParsedStructSchema ParseMembersAndDeclare(
		const FTypeTokens& TypeTokens,
		const TOptional<FTypeTokens>& ParsedSuper,
		uint16 Version,
		TConstArrayView<FUtf8StringView> MemberNames,
		TConstArrayView<FUtf8StringView> MemberTypes);

	FScopedTokenReader		ScopedReader;
	FTextIndexer&	  		Names;
	FStructDeclarations&	Declarations;
	FParsedSchemas&			Schemas;
};

////////////////////////////////////////////////////////////////////////////////

class FMemberParser
{
public:
	FMemberParser(
		FScratchAllocator& InScratch,
		const FParsedStructSchema& Schema,
		FScopedTokenReader InMemberTokens,
		FTextIndexer& InNames,
		FParseDeclarations& InTypes,
		FParsedSchemas& InSchemas);

	FBuiltStruct* ParseMembers(FDeclId Id);

private:
	FDebugIds GetDebug() const { return FDebugIds(Names); }
	bool HasMore() const { return MemberTokens.HasMore(); }
	bool PeekKind(EMemberKind& MemberKind) const;
	bool ParseAll();
	bool ParseLeaf();
	bool ParseStruct();
	bool ParseRange();

	bool AdvanceToNextMember();
	FDeclId ParseDynamicStructType();

	[[nodiscard]] FBuiltStruct* BuildAndReset(FDeclId Id);

	[[nodiscard]] FTypedRange	ParseLeaves(const FMemberSchema& RangeSchema, FScopedTokenReader RangeTokens);
	[[nodiscard]] FTypedRange	ParseStructs(const FMemberSchema& RangeSchema, FScopedTokenReader RangeTokens);
	[[nodiscard]] FTypedRange	ParseRanges(const FMemberSchema& RangeSchema, FScopedTokenReader RangeTokens);

	[[nodiscard]] FTypedRange	ParseRangeInternal(const FMemberSchema& RangeSchema);

	[[nodiscard]] FBuiltStruct*	ParseMembersInternal(
		FStructType StructType,
		FDeclId Id,
		FScopedTokenReader InnerTokens);

	[[nodiscard]] FTypedRange	ParseUnicodeLeafValueAsRange(
		ESizeType NumType,
		FUnpackedLeafType Leaf,
		FUtf8StringView String);

	FScratchAllocator&						Scratch;
	FMemberBuilder							MemberBuilder;
	TConstArrayView<FMemberId>				MemberNames;
	TConstArrayView<FParsedMemberSchema>	MemberSchemas;
	FScopedTokenReader						MemberTokens;
	FTextIndexer&							Names;
	FParseDeclarations&						Types;
	FParsedSchemas&							Schemas;
	FOptionalDeclId							UsedSuper;
	uint32									TokenIdx = 0;
	uint32								 	MemberIdx = 0;
	bool									HasSuper = false;
};

///////////////////////////////////////////////////////////////////////////////

FYamlTokenizer::FYamlTokenizer(FUtf8StringView InStringView)
: FMarkupTokenizer(InStringView)
{
	AdvanceReadLines();
	AdvanceReadLines();
}

FYamlTokenizer::~FYamlTokenizer()
{
	checkf(!HasMore(), TEXT("FYamlTokenizer has ungrabbed tokens"));
}

static FToken StealToken(TOptional<FToken>& Opt)
{
	FToken Token = Opt.GetValue();
	Opt.Reset();
	return Token;
}

TOptional<FToken> FYamlTokenizer::GrabToken()
{
	if (ScopeStack.IsEmpty())
	{
		return GrabBeginDocument();
	}

	if (CachedToken.IsSet())
	{
		return StealToken(CachedToken);
	}

	// Pop stack to return end scope tokens
	if (ReadLine.NumSpaces < ScopeStack.Last().NumSpaces || ReadLine.EndOfFile)
	{
		return PopScopeStack();
	}

	TOptional<FToken> Result = GrabTokenInternal();

	AdvanceReadLines();

	return Result;
}

TOptional<FToken> FYamlTokenizer::GrabBeginDocument()
{
	if (ReadLine.EndOfFile)
	{
		return NullOpt;
	}
	FUtf8StringView View;
	if (ReadLine.FullView.StartsWith("---"))
	{
		View = ReadLine.FullView;
		AdvanceReadLines();
	}
	check(ScopeStack.IsEmpty());
	ScopeStack.Emplace(EScope::Document);
	return FToken{ EToken::BeginDocument, /*Depth*/ 0, View };
}

static FUtf8StringView GrabTokenFromLine(FLineInfo& Line)
{
	if (SingleQuoteSet.Contains(*Line.View.GetData()))
	{
		Line.View.RightChopInline(1);

		const FUtf8StringView Token = FAsciiSet::FindPrefixWithout(Line.View, SingleQuoteSet);
		Line.View.RightChopInline(Token.Len());

		check(SingleQuoteSet.Contains(*Line.View.GetData()));
		Line.View.RightChopInline(1);

		const FUtf8StringView Skip = FAsciiSet::FindPrefixWith(Line.View, WhitespaceSet);
		Line.View.RightChopInline(Skip.Len());

		return Token;
	}
	else if (DoubleQuoteSet.Contains(*Line.View.GetData()))
	{
		Line.View.RightChopInline(1);
		int32 Len = 0;
		const UTF8CHAR* Start = Line.View.GetData();

		FUtf8StringView Part = FAsciiSet::FindPrefixWithout(Line.View, DoubleQuoteSet);
		Line.View.RightChopInline(Part.Len());
		Len += Part.Len();

		// if the double quote " was preceeded by an uneven number of back slashes,
		// then it was escaped and should be consumed as being part of the string
		FUtf8StringView PartEscapeSuffix = FAsciiSet::FindSuffixWith(Part, BackslashSet);
		while ((PartEscapeSuffix.Len() % 2))
		{
			check(DoubleQuoteSet.Contains(*Line.View.GetData()));
			Line.View.RightChopInline(1);
			Len += 1;

			Part = FAsciiSet::FindPrefixWithout(Line.View, DoubleQuoteSet);
			Line.View.RightChopInline(Part.Len());
			Len += Part.Len();

			PartEscapeSuffix = FAsciiSet::FindSuffixWith(Part, BackslashSet);
		}

		check(DoubleQuoteSet.Contains(*Line.View.GetData()));
		Line.View.RightChopInline(1);

		const FUtf8StringView Skip = FAsciiSet::FindPrefixWith(Line.View, WhitespaceSet);
		Line.View.RightChopInline(Skip.Len());

		return FUtf8StringView(Start, Len);
	}
	return {};
}

static bool GrabColonFromLine(FLineInfo& Line)
{
	if (PeekIsColon(Line.View))
	{
		Line.View.RightChopInline(1);

		const FUtf8StringView Skip = FAsciiSet::FindPrefixWith(Line.View, WhitespaceSet);
		Line.View.RightChopInline(Skip.Len());

		return true;
	}
	return false;
}

TOptional<FToken> FYamlTokenizer::GrabTokenInternal()
{
	const int32 TokenDepth = ScopeStack.Num();
	TOptional<FToken> ParsedToken;
	FUtf8StringView Error;

	// Parse scalars
	FUtf8StringView FirstToken = GrabTokenFromLine(ReadLine);
	const bool HasColon = !FirstToken.IsEmpty() && GrabColonFromLine(ReadLine);
	FUtf8StringView SecondToken = HasColon ? GrabTokenFromLine(ReadLine) : FUtf8StringView{};

	// Leaf with id and value, value may be empty string
	if (SecondToken.GetData() != nullptr)
	{
		if (FirstToken.Len() > 0)
		{
			ParsedToken.Emplace(EToken::LeafId, TokenDepth, FirstToken);
			CachedToken.Emplace(EToken::LeafValue, TokenDepth, SecondToken);
		}
		else
		{
			Error = UTF8TEXTVIEW("EmptyLeafId");
		}
	}
	// Leaf with value only
	else if (FirstToken.GetData() != nullptr && !HasColon)
	{
		if (FirstToken.Len() > 0)
		{
			ParsedToken.Emplace(EToken::Leaf, TokenDepth, FirstToken);
		}
		else
		{
			Error = UTF8TEXTVIEW("EmptyValue");
		}
	}

	// else we have a range or struct member with or without a name in FirstToken

	// Empty struct
	else if (*ReadLine.View.GetData() == '{')
	{
		ParsedToken.Emplace(EToken::BeginStruct, TokenDepth, FirstToken);
		CachedToken.Emplace(EToken::EndStruct, TokenDepth, ReadLine.View);
	}
	// Empty range
	else if (*ReadLine.View.GetData() == '[')
	{
		ParsedToken.Emplace(EToken::BeginRange, TokenDepth, FirstToken);
		CachedToken.Emplace(EToken::EndRange, TokenDepth, ReadLine.View);
	}
	// Document end
	else if (*ReadLine.View.GetData() == '.')
	{
		if (ReadLine.NumSpaces == 0 && ReadLine.View.StartsWith("..."))
		{
			ParsedToken.Emplace(PopScopeStack());
		}
		else
		{
			Error = UTF8TEXTVIEW("InvalidEndDocument");
		}
	}
	// Non-empty range or struct
	else if (NextLine.NumSpaces > ReadLine.NumSpaces)
	{
		if (*ReadLine.View.GetData() == '\n' || *ReadLine.View.GetData() == '#')
		{
			const EToken Token = NextLine.HasRangeToken ? EToken::BeginRange : EToken::BeginStruct;
			const EScope Scope = NextLine.HasRangeToken ? EScope::Range : EScope::Struct;
			ParsedToken.Emplace(Token, TokenDepth, FirstToken);
			PushScopeStack(Scope);
		}
		else
		{
			Error = NextLine.HasRangeToken ? UTF8TEXTVIEW("InvalidRange") : UTF8TEXTVIEW("InvalidStruct");
		}
	}
	else
	{
		Error = UTF8TEXTVIEW("UnknownToken");
	}

	check(ParsedToken.IsSet() == Error.IsEmpty());

	SetLastError(Error, ReadLine.View);
	return ParsedToken;
}

void FYamlTokenizer::PushScopeStack(EScope Scope)
{
	ScopeStack.Emplace(Scope, NextLine.NumSpaces);
}

FToken FYamlTokenizer::PopScopeStack()
{
	FScopeInfo Last = ScopeStack.Pop(EAllowShrinking::No);
	return { ScopeEndTokens[static_cast<uint8>(Last.Scope)], IntCastChecked<uint16>(ScopeStack.Num()) };
}

void FYamlTokenizer::AdvanceReadLines()
{
	ReadLine = NextLine;
	NextLine = ReadOneLine();

	while (IsEmptyOrCommentLine(NextLine.View) && !NextLine.EndOfFile)
	{
		NextLine = ReadOneLine();
	}

	NextLine.HasRangeToken = IsRangeTokenLine(NextLine.View);
	if (NextLine.HasRangeToken)
	{
		NextLine.View.RightChopInline(NextLine.HasRangeToken);
		FUtf8StringView Skip = FAsciiSet::FindPrefixWith(NextLine.View, WhitespaceSet);
		NextLine.View.RightChopInline(Skip.Len());
	}
}



////////////////////////////////////////////////////////////////////////////////

class FJsonTokenizer
	: public FMarkupTokenizer
{
public:
								FJsonTokenizer(FUtf8StringView InData);
								~FJsonTokenizer();
	virtual bool				HasMore() const override;
	virtual TOptional<FToken>	GrabToken() override;

private:
	enum class EScope : uint8 { Document, Struct, Range, };

	enum : uint16
	{
		Flag_AllowStruct	= 1 << 0,
		Flag_AllowRange		= 1 << 1,
		Flag_AllowOpen		= Flag_AllowStruct | Flag_AllowRange,
		Flag_AllowClose		= 1 << 2,
		Flag_ExpectKey		= 1 << 3,
		Flag_ExpectColon	= 1 << 4,
		Flag_ExpectValue	= 1 << 5 | Flag_AllowOpen,
		Flag_ExpectComma	= 1 << 6,
		Flag_HasComma		= 1 << 7,
	};

	struct FScopeInfo
	{
		EScope			Scope;
		uint8			Flags;
	};

	TOptional<FToken>	GrabTokenInternal();
	FToken				SetLastError(FUtf8StringView Msg);
	void				AdvanceData(uint32 ChopLength);
	TOptional<FToken>	ParseOpen();
	TOptional<FToken>	ParseClose();
	TOptional<FToken>	ParseKey();
	TOptional<FToken>	ParseColon();
	TOptional<FToken>	ParseValue();
	TOptional<FToken>	ParseComma();
	FUtf8StringView		Data;
	TArray<FScopeInfo>	Scopes;
	TOptional<FToken>	PendingToken;

};

FJsonTokenizer::FJsonTokenizer(FUtf8StringView InData)
: FMarkupTokenizer(InData)
{
	Scopes.Emplace(FScopeInfo{
		.Flags = Flag_AllowStruct,
	});

	AdvanceData(0);
}

FJsonTokenizer::~FJsonTokenizer()
{
	check(!PendingToken.IsSet());
	check(Scopes.IsEmpty());
}

bool FJsonTokenizer::HasMore() const
{
	return !Scopes.IsEmpty();
}

void FJsonTokenizer::AdvanceData(uint32 ChopLength)
{
	Data.RightChopInline(ChopLength);
	FUtf8StringView Trim = FAsciiSet::FindPrefixWith(Data, WhitespaceSet);
	Data.RightChopInline(Trim.Len());

	if (!Data.IsEmpty())
	{
		return;
	}

	FLineInfo Line;
	do
	{
		Line = ReadOneLine();
		if (Line.EndOfFile)
		{
			return;
		}
	}
	while (Line.View.IsEmpty());

	Data = Line.View;
}

FToken FJsonTokenizer::SetLastError(FUtf8StringView Msg)
{
	FMarkupTokenizer::SetLastError(Msg, Data);
	Scopes.Reset();
	return FToken{ EToken::Invalid, 0 };
}

TOptional<FToken> FJsonTokenizer::ParseOpen()
{
	FScopeInfo& Last = Scopes.Last();

	int32 Char = Data[0];
	if (Char == '[')
	{
		AdvanceData(1);

		if ((Last.Flags & Flag_AllowRange) == 0)
		{
			return SetLastError("Range not allowed here");
		}

		Last.Flags = Flag_ExpectComma | Flag_AllowClose;

		Scopes.Emplace(FScopeInfo{
			.Scope = EScope::Range,
			.Flags = Flag_ExpectValue | Flag_AllowClose,
		});
		uint16 Depth = uint16(Scopes.Num() - 1);

		return FToken{ EToken::BeginRange, Depth };
	}

	if (Char == '{')
	{
		check(Last.Flags & Flag_AllowStruct);

		AdvanceData(1);

		if (Last.Flags == Flag_AllowStruct)
		{
			check(Scopes.Num() == 1);
			Last.Flags = Flag_ExpectKey | Flag_AllowClose;
			return FToken{ EToken::BeginDocument, 0 };
		}

		Last.Flags = Flag_ExpectComma | Flag_AllowClose;

		Scopes.Emplace(FScopeInfo{
			.Scope = EScope::Struct,
			.Flags = Flag_ExpectKey | Flag_AllowClose,
		});
		uint16 Depth = uint16(Scopes.Num() - 1);

		return FToken{ EToken::BeginStruct, Depth };
	}

	return {};
}

TOptional<FToken> FJsonTokenizer::ParseClose()
{
	int32 Char = Data[0];
	if ((Char != ']') && (Char != '}'))
	{
		return {};
	}

	const FScopeInfo& Last = Scopes.Last();

	if (Last.Flags & Flag_HasComma)
	{
		return SetLastError("Trailing seperator");
	}

	bool bValid = (Char == '}') & (Last.Scope != EScope::Range);
	bValid |= (Char == ']') & (Last.Scope == EScope::Range);
	if (!bValid)
	{
		return SetLastError("Mismatched scope");
	}

	AdvanceData(1);

	Scopes.Pop(EAllowShrinking::No);

	EToken Token = EToken::EndRange;
	if (Char == '}')
	{
		Token = (Scopes.IsEmpty()) ? EToken::EndDocument : EToken::EndStruct;
	}

	return FToken{ Token, uint16(Scopes.Num()) };
}

TOptional<FToken> FJsonTokenizer::ParseKey()
{
	int32 Char = Data[0];
	if (Char != '\"')
	{
		return SetLastError("Key string expected");
	}

	FLineInfo LineInfo = { .View = Data, };
	FUtf8StringView Key = GrabTokenFromLine(LineInfo);
	if (Key.IsEmpty())
	{
		return SetLastError("Empty key");
	}

	AdvanceData(Key.Len() + 2);

	Scopes.Last().Flags = Flag_ExpectColon;
	return FToken{ EToken::LeafId, uint16(Scopes.Num()), Key };
}

TOptional<FToken> FJsonTokenizer::ParseColon()
{
	int32 Char = Data[0];
	if (Char != ':')
	{
		return SetLastError("Key should be followed by a colon");
	}

	AdvanceData(1);

	Scopes.Last().Flags = Flag_ExpectValue;
	return {};
}

TOptional<FToken> FJsonTokenizer::ParseValue()
{
	int32 Char = Data[0];

	if (TOptional<FToken> Token = ParseOpen(); Token.IsSet())
	{
		return Token;
	}

	if (Char != '\"')
	{
		return SetLastError("Only string-type values are supported");
	}

	FLineInfo LineInfo = { .View = Data, };
	FUtf8StringView Value = GrabTokenFromLine(LineInfo);
	AdvanceData(Value.Len() + 2);

	Scopes.Last().Flags = Flag_ExpectComma | Flag_AllowClose;

	EToken Token = (Scopes.Last().Scope == EScope::Range) ? EToken::Leaf : EToken::LeafValue;
	return FToken{ Token, uint16(Scopes.Num()), Value };
}

TOptional<FToken> FJsonTokenizer::ParseComma()
{
	int32 Char = Data[0];
	if (Char != ',')
	{
		return SetLastError("Comma separator expected");
	}

	AdvanceData(1);

	FScopeInfo& Last = Scopes.Last();
	Last.Flags = (Last.Scope == EScope::Range) ? Flag_ExpectValue : Flag_ExpectKey;
	Last.Flags |= Flag_HasComma;
	return {};
}

TOptional<FToken> FJsonTokenizer::GrabTokenInternal()
{
	if (Data.IsEmpty())
	{
		return SetLastError("Encountered EOF");
	}

	if (Scopes.IsEmpty())
	{
		return SetLastError("Unexpected token");
	}

	static constexpr struct {
		uint32					Flag;
		TOptional<FToken>		(FJsonTokenizer::*Method)(void);
	} Rota[] = {
		{ Flag_AllowOpen,		&FJsonTokenizer::ParseOpen },
		{ Flag_AllowClose,		&FJsonTokenizer::ParseClose },
		{ Flag_ExpectComma, 	&FJsonTokenizer::ParseComma },
		{ Flag_ExpectKey,		&FJsonTokenizer::ParseKey },
		{ Flag_ExpectColon,		&FJsonTokenizer::ParseColon },
		{ Flag_ExpectValue,		&FJsonTokenizer::ParseValue },
	};

	for (const FScopeInfo& Last = Scopes.Last();;)
	{
		for (const auto& [RotaFlag, Method] : Rota)
		{
			if (Last.Flags & RotaFlag)
			{
				if (TOptional<FToken> Token = (this->*Method)(); Token.IsSet())
				{
					return Token;
				}
			}
		}

		if (Last.Flags & (Flag_ExpectKey | Flag_ExpectValue))
		{
			continue;
		}

		SetLastError("Parsing reached the unreachable");
		return {};
	}
}

TOptional<FToken> FJsonTokenizer::GrabToken()
{
	TOptional<FToken> Token;
	if (PendingToken.IsSet())
	{
		Token = PendingToken.GetValue();
		PendingToken.Reset();
	}
	else
	{
		Token = GrabTokenInternal();
	}

	if (!Token.IsSet())						return {};
	if (Token->Token == EToken::Invalid)	return {};
	if (Token->Token != EToken::LeafId)		return Token;

	TOptional<FToken> NextToken = GrabTokenInternal();
	if (NextToken.IsSet())
	{
		if (NextToken->Token == EToken::Invalid)
		{
			return {};
		}

		if (NextToken->Token == EToken::BeginStruct || NextToken->Token == EToken::BeginRange)
		{
			NextToken->Str = Token->Str;
			NextToken->Len = Token->Len;
			return NextToken;
		}
	}

	PendingToken = NextToken;
	return Token;
}



////////////////////////////////////////////////////////////////////////////////

bool GrabToNextBeginStruct(FToken& Out, FScopedTokenReader& ScopedReader)
{
	while (ScopedReader.HasMore())
	{
		FToken Token = ScopedReader.GrabToken();
		if (Token.Depth == ScopedReader.GetDepth() && Token.Token == EToken::BeginStruct)
		{
			Out = Token;
			return true;
		}
	}
	check(!ScopedReader.HasMore());
	return false;
}

bool FEnumSchemaParser::ParseEnumSchema(FParsedEnumSchema& Out)
{
	FToken Token;
	if (!GrabToNextBeginStruct(Token, ScopedReader))
	{
		return false;
	}

	TOptional<FTypeTokens> TypeTokens = TokenizeType(Token.Value());
	if (!TypeTokens)
	{
		checkf(false, TEXT("Invalid enum schema typename '%s'"), *Print(Token));
		return false;
	}

	// Parse Enum Properties
	TOptional<EEnumMode> Mode;
	TOptional<ELeafWidth> Width;
	uint16 TokenDepth = Token.Depth + 1;
	while (ScopedReader.Peek()->Depth == TokenDepth)
	{
		Token = ScopedReader.GrabToken();
		if (Token.Token == EToken::LeafId)
		{
			if (Token.Value() == UTF8TEXTVIEW("FlagMode"))
			{
				Token = ScopedReader.GrabToken();
				TOptional<bool> FlagMode = Parse<bool>(Token.Value());
				if (FlagMode.IsSet())
				{
					Mode = FlagMode.GetValue() ? EEnumMode::Flag : EEnumMode::Flat;
				}
				else
				{
					checkf(false, TEXT("Unknown value '%s' for enum flag mode"), *Print(Token));
				}
			}
			else if (Token.Value() == UTF8TEXTVIEW("Width"))
			{
				Token = ScopedReader.GrabToken();
				Width = Parse<ELeafWidth>(Token.Value());
				checkf(Width, TEXT("Unknown value '%s' for enum width"), *Print(Token));
			}
			else
			{
				checkf(false, TEXT("Unknown property '%s' for enum"), *Print(Token));
				(void)ScopedReader.GrabToken();
			}
		}
		else if (Token.Token == EToken::BeginStruct)
		{
			checkf(Token.Value() == GLiterals.Constants,
				TEXT("Invalid range '%s' for enum"), *Print(Token));
			++TokenDepth;
			break;
		}
		else
		{
			checkf(false, TEXT("Invalid token %d '%s' for enum"), EnumToUnderlyingType(Token.Token), *Print(Token));
		}
	}

	// Parse Enum Constants
	TArray<FUtf8StringView, TInlineAllocator<64>> ConstantNames;
	TArray<uint64, TInlineAllocator<64>> ConstantValues;
	while (ScopedReader.Peek()->Depth == TokenDepth)
	{
		Token = ScopedReader.GrabToken();
		if (Token.Token == EToken::LeafId)
		{
			// uint64 Value;
			FToken TokenValue = ScopedReader.GrabToken();
			if (TOptional<uint64> ParsedValue = Parse<uint64>(TokenValue.Value()))
			{
				ConstantNames.Add(Token.Value());
				ConstantValues.Add(ParsedValue.GetValue());
			}
			else
			{
				checkf(false, TEXT("Invalid value '%s' for enum constant '%s'"),
					*Print(TokenValue), *Print(Token));
			}
		}
		else
		{
			checkf(false, TEXT("Invalid token %d '%s' in enum constants"), EnumToUnderlyingType(Token.Token), *Print(Token));
		}
	}

	Out = DeclareEnumSchema(TypeTokens.GetValue(), Mode.GetValue(), Width.GetValue(), ConstantNames, ConstantValues);
	return true;
}

FParsedEnumSchema FEnumSchemaParser::DeclareEnumSchema(
	const FTypeTokens& TypeTokens,
	EEnumMode Mode,
	ELeafWidth Width,
	TConstArrayView<FUtf8StringView> ConstantNames,
	TConstArrayView<uint64> ConstantValues)
{
	FType Type = MakeType(TypeTokens, Names);
	FEnumId Id = Names.IndexEnum(Type);

	TArray<FEnumerator, TInlineAllocator<64>> Enumerators;
	Enumerators.Reserve(ConstantNames.Num());
	for (int32 I = 0, N = ConstantNames.Num(); I < N; ++I)
	{
		Enumerators.Emplace(Names.MakeName(ConstantNames[I]), ConstantValues[I]);
	}

	const FEnumDeclaration& Decl = Declarations.Declare(Id, Type, Mode, Width, Enumerators, EEnumAliases::Strip);

	return { Id, Width, Decl.GetEnumerators() };
}

///////////////////////////////////////////////////////////////////////////////

bool FStructSchemaParser::ParseStructSchema(FParsedStructSchema& Out)
{
	// Tokenize
	FToken Token;
	if (!GrabToNextBeginStruct(Token, ScopedReader))
	{
		return false;
	}

	TOptional<FTypeTokens> TypeTokens = TokenizeType(Token.Value());
	if (!TypeTokens)
	{
		checkf(false, TEXT("Invalid struct schema typename '%s'"), *Print(Token));
		return false;
	}

	// Parse Struct Properties
	TOptional<FTypeTokens> ParsedSuper;
	uint16 TokenDepth = Token.Depth + 1;
	uint16 Version = 0;
	while (ScopedReader.Peek()->Depth == TokenDepth)
	{
		Token = ScopedReader.GrabToken();
		if (Token.Token == EToken::LeafId)
		{
			if (Token.Value() == GLiterals.Version)
			{
				Token = ScopedReader.GrabToken();
				Version = *Parse<uint16>(Token.Value());
			}
			else if (Token.Value() == GLiterals.DeclaredSuper)
			{
				Token = ScopedReader.GrabToken();
				ParsedSuper = TokenizeType(Token.Value());
				checkf(ParsedSuper, TEXT("Invalid typename '%s' for declared super"), *Print(Token));
			}
			else
			{
				checkf(false, TEXT("Unknown property '%s' for struct"), *Print(Token));
				(void)ScopedReader.GrabToken();
			}
		}
		else if (Token.Token == EToken::BeginStruct)
		{
			checkf(Token.Value() == GLiterals.Members,
				TEXT("Invalid range '%s' for struct"), *Print(Token));
			++TokenDepth;
			break;
		}
		else
		{
			checkf(false, TEXT("Invalid token %d '%s' for struct"), EnumToUnderlyingType(Token.Token), *Print(Token));
		}
	}

	// Parse Struct Members
	TArray<FUtf8StringView, TInlineAllocator<64>> MemberNames;
	TArray<FUtf8StringView, TInlineAllocator<64>> MemberTypes;
	while (ScopedReader.Peek()->Depth == TokenDepth)
	{
		Token = ScopedReader.GrabToken();
		if (Token.Token == EToken::LeafId)
		{
			MemberNames.Add(Token.Value());
			Token = ScopedReader.GrabToken();
			MemberTypes.Add(Token.Value());
		}
		else
		{
			checkf(false, TEXT("Invalid token %d '%s' in struct members"), EnumToUnderlyingType(Token.Token), *Print(Token));
		}
	}

	Out = ParseMembersAndDeclare(TypeTokens.GetValue(), ParsedSuper, Version, MemberNames, MemberTypes);

	return true;
}

FParsedStructSchema FStructSchemaParser::ParseMembersAndDeclare(
	const FTypeTokens& TypeTokens,
	const TOptional<FTypeTokens>& ParsedSuper,
	uint16 Version,
	TConstArrayView<FUtf8StringView> MemberNames,
	TConstArrayView<FUtf8StringView> MemberTypes)
{
	const int32 Num = MemberNames.Num();
	const FDeclId Id = Names.IndexDeclId(MakeType(TypeTokens, Names));
	const FOptionalDeclId DeclaredSuper = ParsedSuper ? ToOptional(Names.IndexDeclId(MakeType(ParsedSuper.GetValue(), Names))) : NoId;

	FOptionalDeclId UsedSuper;

	TArray<FMemberId, TInlineAllocator<64>> MemberIds;
	TArray<FParsedMemberSchema> MemberSchemas;
	MemberIds.Reserve(Num);
	MemberSchemas.Reserve(Num);

	const int32 HasSuper = DeclaredSuper && Num && MemberNames[0] == GLiterals.Super;
	if (HasSuper)
	{
		if (MemberTypes[0] != GLiterals.Dynamic)
		{
			TOptional<FTypeTokens> ParsedSuperMember = TokenizeType(MemberTypes[0]);
			check(ParsedSuperMember);
			FType UsedSuperType = MakeType(ParsedSuperMember.GetValue(), Names);
			UsedSuper = Names.IndexDeclId(UsedSuperType);
		}
	}

	for (int32 I = HasSuper; I < Num; ++I)
	{
		FParsedMemberSchema MemberSchema;
		if (ParseMemberSchema(MemberSchema, MemberTypes[I]))
		{
			MemberIds.Emplace(Names.MakeName(MemberNames[I]));
			MemberSchemas.Emplace(MoveTemp(MemberSchema));
		}
		else
		{
			checkf(false, TEXT("Failed to parse member schema %d '%s'"), I - HasSuper, *Print(MemberTypes[I]));
		}
	}

	TArray<FMemberSpec, TInlineAllocator<64>> MemberSpecs;
	MemberSpecs.Reserve(MemberIds.Num());
	for (const FParsedMemberSchema& Member : MemberSchemas)
	{
		MemberSpecs.Emplace(Member.Type, Member.InnerRangeTypes, Member.InnerSchema);
	}

	// Might want to switch to TMap
	if (Id.Idx >= (uint32)Declarations.Num())
	{
		Declarations.SetNum(Id.Idx + 1);
	}
	check(!Declarations[Id.Idx]);
	Declarations[Id.Idx] = Declare({Id, DeclaredSuper, Version, EMemberPresence::AllowSparse, MemberIds, MemberSpecs});

	return { Id, Version, DeclaredSuper, UsedSuper, Declarations[Id.Idx]->GetMemberOrder(), MoveTemp(MemberSchemas) };
}

static TOptional<ESizeType> GrabRangeSize(FUtf8StringView& String)
{
	if (String.Len() > 2 && OpenRangeSet.Contains(String[0]))
	{
		FUtf8StringView Remainder = String.RightChop(1);
		FUtf8StringView Skip = FAsciiSet::FindPrefixWith(Remainder, WhitespaceSet | OpenRangeSet);
		Remainder.RightChopInline(Skip.Len());
		FUtf8StringView SizeStr = FAsciiSet::FindPrefixWithout(Remainder, WhitespaceSet | CloseRangeSet);
		Remainder.RightChopInline(SizeStr.Len());
		Skip = FAsciiSet::FindPrefixWith(Remainder, WhitespaceSet | CloseRangeSet);
		Remainder.RightChopInline(Skip.Len());
		if (TOptional<ESizeType> MaxSize = Parse<ESizeType>(SizeStr))
		{
			String = Remainder;
			return MaxSize;
		}
	}
	return NullOpt;
}

///////////////////////////////////////////////////////////////////////////////

bool FStructSchemaParser::ParseMemberSchema(FParsedMemberSchema& Out, FUtf8StringView String)
{
	FUtf8StringView TypeOrLeaf = FAsciiSet::FindPrefixWithout(String, RangeSet);

	if (TOptional<FUnpackedLeafType> Leaf = Parse<FUnpackedLeafType>(TypeOrLeaf))
	{
		Out.Type = FMemberType(Leaf.GetValue().Pack());
	}
	else if (TypeOrLeaf == GLiterals.Super)
	{
		checkf(TypeOrLeaf.Len() == String.Len(), TEXT("Super can't be used in a range"));
		Out.Type = SuperStructType;
		return true;
	}
	else if (TypeOrLeaf == GLiterals.Dynamic)
	{
		Out.Type = DynamicStructType;
	}
	else if (TOptional<FTypeTokens> TypeTokens = TokenizeType(TypeOrLeaf))
	{
		FType Type = MakeType(TypeTokens.GetValue(), Names);
		FOptionalEnumId Enum = Names.GetEnumId(Type);
		if (Enum)
		{
			// All Enum types have already been parsed and are known before parsing structs and members
			const FParsedEnumSchema& EnumSchema = ResolveEnumSchema(Schemas, Enum.Get());
			Out.Type = FMemberType(ELeafType::Enum, EnumSchema.Width);
			Out.InnerSchema = FInnerId(Enum.Get());
		}
		else
		{
			// PP-TEXT: Index all Struct types in a pass before parsing struct members?
			Out.Type = DefaultStructType;
			Out.InnerSchema = FInnerId(Names.IndexStruct(Type));
		}
	}

	if (TypeOrLeaf.Len() < String.Len())
	{
		FUtf8StringView RangeSizes = String.RightChop(TypeOrLeaf.Len());
		FMemberType InnermostType = Out.Type;
		if (TOptional<ESizeType> RangeSize = GrabRangeSize(RangeSizes))
		{
			Out.Type = FMemberType(RangeSize.GetValue());
			while ((RangeSize = GrabRangeSize(RangeSizes)))
			{
				Out.InnerRangeTypes.Emplace(RangeSize.GetValue());
			}
			Out.InnerRangeTypes.Add(InnermostType);
		}
		else
		{
			return false;
		}
	}
	return true;
}

///////////////////////////////////////////////////////////////////////////////

FMemberParser::FMemberParser(
	FScratchAllocator& InScratch,
	const FParsedStructSchema& Schema,
	FScopedTokenReader InMemberTokens,
	FTextIndexer& InNames,
	FParseDeclarations& InTypes,
	FParsedSchemas& InSchemas)
: Scratch(InScratch)
, MemberNames(Schema.MemberNames)
, MemberSchemas(Schema.MemberSchemas)
, MemberTokens(InMemberTokens)
, Names(InNames)
, Types(InTypes)
, Schemas(InSchemas)
, UsedSuper(Schema.UsedSuper)
, HasSuper(Schema.DeclaredSuper)
{
}

FBuiltStruct* FMemberParser::ParseMembers(FDeclId Id)
{
	ParseAll();
	return BuildAndReset(Id);
}

bool FMemberParser::PeekKind(EMemberKind& MemberKind) const
{
	switch (MemberTokens.Peek()->Token)
	{
		case EToken::BeginStruct:	MemberKind = EMemberKind::Struct;	return true;
		case EToken::BeginRange:	MemberKind = EMemberKind::Range;	return true;
		case EToken::LeafId:		MemberKind = EMemberKind::Leaf;		return true;
		case EToken::Leaf:			MemberKind = EMemberKind::Leaf;		return true;
	}
	return false;
}

bool FMemberParser::ParseAll()
{
	while (HasMore())
	{
		EMemberKind Kind;
		if (PeekKind(Kind))
		{
			switch (Kind)
			{
				case EMemberKind::Leaf:
					verify(ParseLeaf());
					break;
				case EMemberKind::Struct:
					verify(ParseStruct());
					break;
				case EMemberKind::Range:
					verify(ParseRange());
					break;
			}
		}
		else
		{
			FToken Token = MemberTokens.GrabToken();
			checkf(false, TEXT("Invalid member token %d '%s'"), EnumToUnderlyingType(Token.Token), *Print(Token))
		}
	}
	return true;
}

template<Arithmetic T>
void AddMember(FMemberBuilder& Members, FMemberId Id, FUtf8StringView String)
{
	if (TOptional<T> Value = Parse<T>(String))
	{
		Members.Add(Id, Value.GetValue());
	}
}

template<UnsignedIntegral T>
void AddEnum(FMemberBuilder& Members, FMemberId Id, const FParsedEnumSchema& Schema, const FTextIndexer& Names, FUtf8StringView String)
{
	T Value;
	ParseEnumValue(String, Value, Schema, Names);
	Members.AddEnum(Id, Schema.Id, Value);
}

template<UnsignedIntegral T>
void AddHex(FMemberBuilder& Members, FMemberId Id, FUtf8StringView String)
{
	if (TOptional<T> Value = ParseHex<T>(String))
	{
		Members.AddHex(Id, Value.GetValue());
	}
}

static bool IsUnicodeString(const FParsedMemberSchema& MemberSchema)
{
	return MemberSchema.Type.IsRange() &&
		MemberSchema.InnerRangeTypes.Num() == 1 &&
		MemberSchema.InnerRangeTypes[0].AsLeaf().Type == ELeafType::Unicode;
}

bool FMemberParser::ParseLeaf()
{
	if (!AdvanceToNextMember())
	{
		return false;
	}

	FToken Token = MemberTokens.GrabToken(EToken::LeafId);
	Token = MemberTokens.GrabToken(EToken::LeafValue);
	const FUtf8StringView String = Token.Value();

	const FMemberId MemberId = MemberNames[MemberIdx];
	const FParsedMemberSchema& MemberSchema = MemberSchemas[MemberIdx];

	if (IsUnicodeString(MemberSchema))
	{
		FRangeType RangeType = MemberSchema.Type.AsRange();
		FUnpackedLeafType Leaf = MemberSchema.InnerRangeTypes[0].AsLeaf();
		MemberBuilder.AddRange(MemberId, ParseUnicodeLeafValueAsRange(RangeType.MaxSize, Leaf, String));
		return true;
	}

	const FUnpackedLeafType Leaf = MemberSchema.Type.AsLeaf();
	const FOptionalEnumId Enum = ToOptionalEnum(MemberSchema.InnerSchema);

	switch (Leaf.Type)
	{
		case ELeafType::Bool:
			AddMember<bool>(MemberBuilder, MemberId, String);
			break;
		case ELeafType::IntS:
			switch (Leaf.Width)
			{
				case ELeafWidth::B8:	AddMember<int8 >(MemberBuilder, MemberId, String); break;
				case ELeafWidth::B16:	AddMember<int16>(MemberBuilder, MemberId, String); break;
				case ELeafWidth::B32:	AddMember<int32>(MemberBuilder, MemberId, String); break;
				case ELeafWidth::B64:	AddMember<int64>(MemberBuilder, MemberId, String); break;
			}
			break;
		case ELeafType::IntU:
			switch (Leaf.Width)
			{
				case ELeafWidth::B8:	AddMember<uint8 >(MemberBuilder, MemberId, String);	break;
				case ELeafWidth::B16:	AddMember<uint16>(MemberBuilder, MemberId, String);	break;
				case ELeafWidth::B32:	AddMember<uint32>(MemberBuilder, MemberId, String);	break;
				case ELeafWidth::B64:	AddMember<uint64>(MemberBuilder, MemberId, String);	break;
			}
			break;
		case ELeafType::Float:
			switch (Leaf.Width)
			{
				case ELeafWidth::B8:	check(false);										break;
				case ELeafWidth::B16:	check(false);										break;
				case ELeafWidth::B32:	AddMember<float >(MemberBuilder, MemberId, String);	break;
				case ELeafWidth::B64:	AddMember<double>(MemberBuilder, MemberId, String);	break;
			}
			break;
		case ELeafType::Hex:
			switch (Leaf.Width)
			{
				case ELeafWidth::B8:	AddHex<uint8 >(MemberBuilder, MemberId, String);	break;
				case ELeafWidth::B16:	AddHex<uint16>(MemberBuilder, MemberId, String);	break;
				case ELeafWidth::B32:	AddHex<uint32>(MemberBuilder, MemberId, String);	break;
				case ELeafWidth::B64:	AddHex<uint64>(MemberBuilder, MemberId, String);	break;
			}
			break;
		case ELeafType::Enum:
			switch (Leaf.Width)
			{
				case ELeafWidth::B8:	AddEnum<uint8 >(MemberBuilder, MemberId, ResolveEnumSchema(Schemas, Enum.Get()), Names, String); break;
				case ELeafWidth::B16:	AddEnum<uint16>(MemberBuilder, MemberId, ResolveEnumSchema(Schemas, Enum.Get()), Names, String); break;
				case ELeafWidth::B32:	AddEnum<uint32>(MemberBuilder, MemberId, ResolveEnumSchema(Schemas, Enum.Get()), Names, String); break;
				case ELeafWidth::B64:	AddEnum<uint64>(MemberBuilder, MemberId, ResolveEnumSchema(Schemas, Enum.Get()), Names, String); break;
			}
			break;
		case ELeafType::Unicode:
			switch (Leaf.Width)
			{
				case ELeafWidth::B8:	AddMember<char8_t >(MemberBuilder, MemberId, String);	break;
				case ELeafWidth::B16:	AddMember<char16_t>(MemberBuilder, MemberId, String);	break;
				case ELeafWidth::B32:	AddMember<char32_t>(MemberBuilder, MemberId, String);	break;
				case ELeafWidth::B64:	check(false);											break;
			}
			break;
	}

	return true;
}

bool FMemberParser::ParseStruct()
{
	if (HasSuper && MemberIdx == 0 && MemberTokens.Peek()->Value() == GLiterals.Super)
	{
		FToken Token = MemberTokens.GrabToken(EToken::BeginStruct);
		if (!UsedSuper)
		{
			UsedSuper = ParseDynamicStructType();
		}

		FBuiltStruct* Struct = ParseMembersInternal(
			SuperStructType.AsStruct(), UsedSuper.Get(), { MemberTokens.GetTokens(), Token });
		MemberBuilder.AddSuperStruct(UsedSuper.Get(), Struct);

		Token = MemberTokens.GrabToken(EToken::EndStruct);
		return true;
	}

	if (!AdvanceToNextMember())
	{
		return false;
	}

	const FMemberId MemberId = MemberNames[MemberIdx];
	const FParsedMemberSchema& MemberSchema = MemberSchemas[MemberIdx];
	const FStructType StructType = MemberSchema.Type.AsStruct();
	check(!StructType.IsSuper);
	check(MemberSchema.InnerSchema || StructType.IsDynamic);

	FToken Token = MemberTokens.GrabToken(EToken::BeginStruct);

	FDeclId Id = StructType.IsDynamic ? ParseDynamicStructType() : MemberSchema.InnerSchema.Get().AsStructDeclId();

	FBuiltStruct* Struct = ParseMembersInternal(StructType, Id, { MemberTokens.GetTokens(), Token });
	MemberBuilder.AddStruct(MemberId, Id, Struct);

	Token = MemberTokens.GrabToken(EToken::EndStruct);
	return true;
}

bool FMemberParser::ParseRange()
{
	if (!AdvanceToNextMember())
	{
		return false;
	}

	const FMemberId MemberId = MemberNames[MemberIdx];
	const FParsedMemberSchema& MemberSchema = MemberSchemas[MemberIdx];

	FMemberSchema RangeSchema
	{
		.Type = MemberSchema.Type,
		.InnerRangeType = MemberSchema.InnerRangeTypes[0],
		.NumInnerRanges = IntCastChecked<uint16>(MemberSchema.InnerRangeTypes.Num()),
		.InnerSchema = MemberSchema.InnerSchema,
		.NestedRangeTypes =
			MemberSchema.InnerRangeTypes.Num() > 1 ?
			MemberSchema.InnerRangeTypes.GetData() :
			nullptr,
	};

	MemberBuilder.AddRange(MemberId, ParseRangeInternal(RangeSchema));
	return true;
}

bool FMemberParser::AdvanceToNextMember()
{
	const FUtf8StringView MemberName = MemberTokens.Peek()->Value();
	const FOptionalMemberId MemberId = Names.GetMemberId(MemberName);

	if (!MemberId)
	{
		checkf(false, TEXT("Member name '%s' not found in any schema"), *Print(MemberName));
		return false;
	}

	for (uint16 Idx = MemberIdx, N = MemberNames.Num(); Idx < N; ++Idx)
	{
		if (MemberId == MemberNames[Idx])
		{
			MemberIdx = Idx;
			return true;
		}
	}
#if DO_CHECK
	for (uint16 Idx = 0; Idx < MemberIdx; ++Idx)
	{
		if (MemberId == MemberNames[Idx])
		{
			checkf(false, TEXT("Member '%s' appeared in non-declared order"), *Print(MemberName));
			return false;
		}
	}
#endif

	checkf(false, TEXT("Member '%s' not found in struct schema"), *Print(MemberName));
	return false;
}

FDeclId FMemberParser::ParseDynamicStructType()
{
	FToken Token = MemberTokens.GrabToken(EToken::LeafId);
	check(Token.Value() == GLiterals.Dynamic);

	Token = MemberTokens.GrabToken(EToken::LeafValue);
	const FUtf8StringView StructName = Token.Value();
	TOptional<FTypeTokens> TypeTokens = TokenizeType(StructName);
	check(TypeTokens);
	FType Type = MakeType(TypeTokens.GetValue(), Names);
	FOptionalDeclId Out = Names.GetStructId(Type);
	checkf(Out, TEXT("Failed to parse dynamic struct schema '%s'"), *Print(Token));
	return Out.Get();
}

FBuiltStruct* FMemberParser::BuildAndReset(FDeclId Id)
{
	return MemberBuilder.BuildAndReset(Scratch, Types.Get(Id), GetDebug());
}

template<Arithmetic T>
[[nodiscard]] static FTypedRange ParseLeafRangeValues(
	FScratchAllocator& Scratch,
	ESizeType MaxSize,
	FScopedTokenReader& RangeTokens)
{
	TArray<T, TInlineAllocator64<64>> Values;
	while (RangeTokens.HasMore())
	{
		FToken Token = RangeTokens.GrabToken();
		if (TOptional<T> Value = Parse<T>(Token.Value()))
		{
			Values.Add(Value.GetValue());
		}
	}
	return BuildLeafRange(Scratch, MaxSize, TConstArrayView<T>(Values));
}

template<UnsignedIntegral T>
[[nodiscard]] static FTypedRange ParseEnumRange(
	FScratchAllocator& Scratch,
	const FParsedEnumSchema& Schema,
	const FTextIndexer& Names,
	ESizeType MaxSize,
	FScopedTokenReader& RangeTokens)
{
	TArray<T, TInlineAllocator64<64>> Values;
	while (RangeTokens.HasMore())
	{
		FToken Token = RangeTokens.GrabToken();
		FUtf8StringView String = Token.Value();
		T Value;
		ParseEnumValue(String, Value, Schema, Names);
		Values.Add(Value);
	}
	return BuildEnumRange(Scratch, Schema.Id, MaxSize, TConstArrayView<T>(Values));
}

template<UnsignedIntegral T>
[[nodiscard]] static FTypedRange ParseHexRange(
	FScratchAllocator& Scratch,
	ESizeType MaxSize,
	FScopedTokenReader& RangeTokens)
{
	TArray<T, TInlineAllocator64<64>> Values;
	while (RangeTokens.HasMore())
	{
		FToken Token = RangeTokens.GrabToken();
		if (TOptional<T> Value = ParseHex<T>(Token.Value()))
		{
			Values.Add(Value.GetValue());
		}
	}
	return BuildHexRange(Scratch, MaxSize, TConstArrayView<T>(Values));
}

FTypedRange FMemberParser::ParseLeaves(const FMemberSchema& RangeSchema, FScopedTokenReader RangeTokens)
{
	const FOptionalEnumId Enum = ToOptionalEnum(RangeSchema.InnerSchema);
	const FMemberType InnermostType = RangeSchema.GetInnerRangeTypes().Last();
	const FUnpackedLeafType Leaf = InnermostType.AsLeaf();
	const ESizeType NumType = RangeSchema.Type.AsRange().MaxSize;

	FTypedRange TypedRange;
	switch (Leaf.Type)
	{
		case ELeafType::Bool:
			TypedRange = ParseLeafRangeValues<bool>(Scratch, NumType, RangeTokens);
			break;
		case ELeafType::IntS:
			switch (Leaf.Width)
			{
				case ELeafWidth::B8:  TypedRange = ParseLeafRangeValues<int8 >(Scratch, NumType, RangeTokens); break;
				case ELeafWidth::B16: TypedRange = ParseLeafRangeValues<int16>(Scratch, NumType, RangeTokens); break;
				case ELeafWidth::B32: TypedRange = ParseLeafRangeValues<int32>(Scratch, NumType, RangeTokens); break;
				case ELeafWidth::B64: TypedRange = ParseLeafRangeValues<int64>(Scratch, NumType, RangeTokens); break;
			}
			break;
		case ELeafType::IntU:
			switch (Leaf.Width)
			{
				case ELeafWidth::B8:  TypedRange = ParseLeafRangeValues<uint8 >(Scratch, NumType, RangeTokens); break;
				case ELeafWidth::B16: TypedRange = ParseLeafRangeValues<uint16>(Scratch, NumType, RangeTokens); break;
				case ELeafWidth::B32: TypedRange = ParseLeafRangeValues<uint32>(Scratch, NumType, RangeTokens); break;
				case ELeafWidth::B64: TypedRange = ParseLeafRangeValues<uint64>(Scratch, NumType, RangeTokens); break;
			}
			break;
		case ELeafType::Float:
			if (Leaf.Width == ELeafWidth::B32)
			{
				TypedRange = ParseLeafRangeValues<float>(Scratch, NumType, RangeTokens);
			}
			else
			{
				check(Leaf.Width == ELeafWidth::B64);
				TypedRange = ParseLeafRangeValues<double>(Scratch, NumType, RangeTokens);
			}
			break;
		case ELeafType::Hex:
			switch (Leaf.Width)
			{
				case ELeafWidth::B8:	TypedRange = ParseHexRange<uint8 >(Scratch, NumType, RangeTokens); break;
				case ELeafWidth::B16:	TypedRange = ParseHexRange<uint16>(Scratch, NumType, RangeTokens); break;
				case ELeafWidth::B32:	TypedRange = ParseHexRange<uint32>(Scratch, NumType, RangeTokens); break;
				case ELeafWidth::B64:	TypedRange = ParseHexRange<uint64>(Scratch, NumType, RangeTokens); break;
			}
			break;
		case ELeafType::Enum:
			switch (Leaf.Width)
			{
				case ELeafWidth::B8:	TypedRange = ParseEnumRange<uint8 >(Scratch, ResolveEnumSchema(Schemas, Enum.Get()), Names, NumType, RangeTokens); break;
				case ELeafWidth::B16:	TypedRange = ParseEnumRange<uint16>(Scratch, ResolveEnumSchema(Schemas, Enum.Get()), Names, NumType, RangeTokens); break;
				case ELeafWidth::B32:	TypedRange = ParseEnumRange<uint32>(Scratch, ResolveEnumSchema(Schemas, Enum.Get()), Names, NumType, RangeTokens); break;
				case ELeafWidth::B64:	TypedRange = ParseEnumRange<uint64>(Scratch, ResolveEnumSchema(Schemas, Enum.Get()), Names, NumType, RangeTokens); break;
			}
			break;
		case ELeafType::Unicode:
			checkf(!RangeTokens.HasMore(), TEXT("Should have been handled by PrintUnicodeRangeAsLeafValue/ParseUnicodeLeafValueAsRange"));
			TypedRange = ParseUnicodeLeafValueAsRange(NumType, Leaf, {});
			break;
	}

	return TypedRange;
}

FTypedRange FMemberParser::ParseStructs(const FMemberSchema& RangeSchema, FScopedTokenReader RangeTokens)
{
	check(RangeSchema.NumInnerRanges == 1);
	ESizeType NumType = RangeSchema.Type.AsRange().MaxSize;
	const FStructType StructType = RangeSchema.InnerRangeType.AsStruct();
	check(!StructType.IsSuper);
	check(RangeSchema.InnerSchema || StructType.IsDynamic);

	TArray<FBuiltStruct*, TInlineAllocator64<64>> Structs;
	FOptionalDeclId Schema = ToOptionalDeclId(RangeSchema.InnerSchema);

	if (StructType.IsDynamic)
	{
		check(!Schema);
		while (RangeTokens.HasMore())
		{
			FToken Token = RangeTokens.GrabToken(EToken::BeginStruct);
			FDeclId DynamicSchema = ParseDynamicStructType();
			checkf(!Schema || Schema == DynamicSchema, TEXT("Heterogeneous struct ranges have not been implemented yet"));
			Schema = DynamicSchema;

			FBuiltStruct* Struct = ParseMembersInternal(
				DynamicStructType.AsStruct(), Schema.Get(), { RangeTokens.GetTokens(), Token });
			Structs.Add(Struct);
			Token = RangeTokens.GrabToken(EToken::EndStruct);
		}
	}
	else
	{
		check(Schema);
		while (RangeTokens.HasMore())
		{
			FToken Token = RangeTokens.GrabToken(EToken::BeginStruct);
			FBuiltStruct* Struct = ParseMembersInternal(StructType, Schema.Get(), { RangeTokens.GetTokens(), Token });
			Structs.Add(Struct);
			Token = RangeTokens.GrabToken(EToken::EndStruct);
		}
	}

	FMemberSchema BuildRangeSchema = Schema ?
		MakeStructRangeSchema(NumType, Schema.Get()) :
		MakeDynamicStructRangeSchema(NumType);

	FTypedRange TypedRange { BuildRangeSchema };
	if (int64 Num = Structs.Num())
	{
		check(Schema);
		TypedRange.Values = FBuiltRange::Create(Scratch, Structs.Num(), sizeof(FBuiltStruct*));
		FMemory::Memcpy(TypedRange.Values->Data, Structs.GetData(), Structs.NumBytes());
	}		

	return TypedRange;
}

FTypedRange FMemberParser::ParseRanges(const FMemberSchema& RangeSchema, FScopedTokenReader RangeTokens)
{
	check(RangeSchema.NumInnerRanges > 1);
	check(RangeSchema.NestedRangeTypes);

	const ESizeType NumType = RangeSchema.Type.AsRange().MaxSize;
	const bool IsDynamic = RangeSchema.GetInnermostType() == DynamicStructType;
	const FMemberSchema InnerRangeSchema
	{
		.Type = RangeSchema.NestedRangeTypes[0],
		.InnerRangeType = RangeSchema.NestedRangeTypes[1],
		.NumInnerRanges = IntCastChecked<uint16>(RangeSchema.NumInnerRanges - 1),
		.InnerSchema = RangeSchema.InnerSchema,
		.NestedRangeTypes = RangeSchema.NumInnerRanges > 2 ? RangeSchema.NestedRangeTypes + 1 : nullptr,
	};

	FMemberSchema BuiltInnerRangeSchema = InnerRangeSchema;

	TArray64<FBuiltRange*> Ranges;
	while (RangeTokens.HasMore())
	{
		FTypedRange TypedRange = ParseRangeInternal(InnerRangeSchema);
		Ranges.Add(TypedRange.Values);
		if (IsDynamic)
		{
			checkf(!BuiltInnerRangeSchema.InnerSchema ||
				BuiltInnerRangeSchema.InnerSchema.Get() == TypedRange.Schema.InnerSchema,
				TEXT("Heterogeneous struct ranges have not been implemented yet"));
			BuiltInnerRangeSchema = TypedRange.Schema;
		}
	}

	FBuiltRange* Out = nullptr;
	if (int64 Num = Ranges.Num())
	{
		Out = FBuiltRange::Create(Scratch, Num, sizeof(FBuiltRange*));
		FMemory::Memcpy(Out->Data, Ranges.GetData(), Ranges.NumBytes());
	}

	return { MakeNestedRangeSchema(Scratch, NumType, BuiltInnerRangeSchema), Out };
}

static bool IsUnicodeString(const FMemberSchema& MemberSchema)
{
	return MemberSchema.Type.IsRange() &&
		MemberSchema.NumInnerRanges == 1 &&
		MemberSchema.InnerRangeType.AsLeaf().Type == ELeafType::Unicode;
}

FTypedRange FMemberParser::ParseRangeInternal(const FMemberSchema& RangeSchema)
{
	FTypedRange TypedRange{ };

	FToken Token = MemberTokens.GrabToken();

	if (Token.Token == EToken::Leaf)
	{
		check(IsUnicodeString(RangeSchema));
		FUtf8StringView String = Token.Value();
		FRangeType RangeType = RangeSchema.Type.AsRange();
		FUnpackedLeafType Leaf = RangeSchema.InnerRangeType.AsLeaf();
		return ParseUnicodeLeafValueAsRange(RangeType.MaxSize, Leaf, String);
	}

	check(Token.Token == EToken::BeginRange);
	{
		FScopedTokenReader RangeTokens(MemberTokens.GetTokens(), Token);
		switch (RangeSchema.InnerRangeType.GetKind())
		{
			case EMemberKind::Leaf:		TypedRange = ParseLeaves(RangeSchema, RangeTokens);		break;
			case EMemberKind::Struct:	TypedRange = ParseStructs(RangeSchema, RangeTokens);	break;
			case EMemberKind::Range:	TypedRange = ParseRanges(RangeSchema, RangeTokens);		break;
		}
		check(!RangeTokens.HasMore());
	}
	Token = MemberTokens.GrabToken(EToken::EndRange);

	return TypedRange;
}

FBuiltStruct* FMemberParser::ParseMembersInternal(
	FStructType StructType,
	FDeclId Id,
	FScopedTokenReader InnerTokens)
{
	const FParsedStructSchema& ParsedSchema = ResolveStructSchema(Schemas, Id);
	FMemberParser MemberParser(Scratch, ParsedSchema, InnerTokens, Names, Types, Schemas);
	return MemberParser.ParseMembers(Id);
}

static FUtf8StringView ParseString(TArray<UTF8CHAR>& Out, FUtf8StringView String)
{
	FUtf8StringView Verbatim = FAsciiSet::FindPrefixWithout(String, BackslashSet);
	if (Verbatim.Len() == String.Len())
	{
		return String;
	}

	Out.Reserve(String.Len());
	while (!String.IsEmpty())
	{
		Out.Append(MakeArrayView(Verbatim.GetData(), Verbatim.Len()));
		String.RightChopInline(Verbatim.Len());
		while (TOptional<uint32> Codepoint = GrabEscapedCodepoint(String))
		{
			checkf(Codepoint.GetValue() <= 127, TEXT("Unexpected codepoint: %u"), Codepoint.GetValue());
			Out.Add(static_cast<UTF8CHAR>(Codepoint.GetValue()));
		}
		Verbatim = FAsciiSet::FindPrefixWithout(String, BackslashSet);
	}
	return { Out.GetData(), Out.Num() };
}

FTypedRange FMemberParser::ParseUnicodeLeafValueAsRange(
	ESizeType NumType,
	FUnpackedLeafType Leaf,
	FUtf8StringView String)
{
	TArray<UTF8CHAR> Buffer;
	FUtf8StringView Parsed = ParseString(Buffer, String);
	if (Leaf.Width == ELeafWidth::B8)
	{
		const char8_t* Src = reinterpret_cast<const char8_t*>(Parsed.GetData());
		return BuildLeafRange(Scratch, NumType, MakeArrayView(Src, Parsed.Len()));
	}
	else if (Leaf.Width == ELeafWidth::B16)
	{
		const int32 DstLen = FPlatformString::ConvertedLength<UTF16CHAR>(Parsed.GetData(), Parsed.Len());
		TArray<UTF16CHAR, TInlineAllocator<1024>> Buf;
		Buf.Reserve(DstLen);
		UTF16CHAR* Dst = Buf.GetData();
		const UTF16CHAR* DstEnd = FPlatformString::Convert(Dst, DstLen, Parsed.GetData(), Parsed.Len());
		check(DstEnd);
		check(DstEnd - Dst == DstLen);
		return BuildLeafRange(Scratch, NumType, MakeArrayView(reinterpret_cast<const char16_t*>(Dst), DstLen));
	}
	else // ELeafWidth::B32
	{
		check(Leaf.Width == ELeafWidth::B32);
		// PP-TEXT: StringCast<UTF32CHAR>/ToUtf32() missing in ParseUnicodeLeafValueAsRange()
		// auto Utf32 = StringCast<UTF32CHAR>(Parsed.GetData(), Parsed.Len());
		// const char32_t* Src = reinterpret_cast<const char32_t*>(Utf32.Get());
		// return BuildLeafRange(Scratch, NumType, MakeArrayView(Src, Utf32.Length()));
		checkf(false, TEXT("StringCast<UTF32CHAR>/ToUtf32() not implemented"));
		return FTypedRange{ };
	}
}

///////////////////////////////////////////////////////////////////////////////

void FBatchParser::Parse(TArray64<uint8>& Out)
{
	DbgVis::FIdScope _(Names, "Utf8View");
	Tokenize();
	ParseAll();
	Write(Out);
}

void FBatchParser::Tokenize()
{
	while (Tokenizer.HasMore())
	{
		TOptional<FToken> Token = Tokenizer.GrabToken();
		if (!Token)
		{
			checkf(false, TEXT("%s"), *Print(Tokenizer.GetLastError()));
			continue;
		}

		if (Token->Depth == 1 && Token->Token == EToken::BeginStruct)
		{
			if (Token->Value() == GLiterals.Structs)
			{
				StructsIdx = Tokens.Num();
			}
			else if (Token->Value() == GLiterals.Enums)
			{
				EnumsIdx = Tokens.Num();
			}
		}

		else if (Token->Depth == 1 && Token->Token == EToken::BeginRange)
		{
			if (Token->Value() == GLiterals.Objects)
			{
				ObjectsIdx = Tokens.Num();
			}
		}

		Tokens.Add(*Token);
	}

	checkf(EnumsIdx >= 0, TEXT("No '%.*hs' section found"), GLiterals.Structs.Len(), GLiterals.Structs.GetData());
	checkf(StructsIdx >= 0, TEXT("No '%.*hs' section found"), GLiterals.Enums.Len(), GLiterals.Enums.GetData());
	checkf(ObjectsIdx >= 0, TEXT("No '%.*hs' section found"), GLiterals.Objects.Len(), GLiterals.Objects.GetData());
}

void FBatchParser::ParseAll()
{
	TConstArrayView64<FToken> TokensView(Tokens);
	ParseEnumSchemas(TokensView.RightChop(EnumsIdx));
	ParseStructSchemas(TokensView.RightChop(StructsIdx));
	ParseObjects(TokensView.RightChop(ObjectsIdx));
}

void FBatchParser::ParseStructSchemas(TConstArrayView64<FToken> TokensView)
{
	FTokenReader TokenIt(TokensView);
	verify(TokenIt.GrabToken().Value() == GLiterals.Structs);

	FStructSchemaParser Parser(TokenIt, Names, Types.Structs, Schemas);
	while (Parser.HasMore())
	{
		FParsedStructSchema ParsedStructSchema;
		if (Parser.ParseStructSchema(ParsedStructSchema))
		{
			Schemas.Structs.SetNum(Types.Structs.Num());
			Schemas.Structs[ParsedStructSchema.Id.Idx] = MoveTemp(ParsedStructSchema);
		}
	}
}

void FBatchParser::ParseEnumSchemas(TConstArrayView64<FToken> TokensView)
{
	FTokenReader TokenIt(TokensView);
	verify(TokenIt.GrabToken().Value() == GLiterals.Enums);

	FEnumSchemaParser Parser(TokenIt, Names, Types.Enums);
	while (Parser.HasMore())
	{
		FParsedEnumSchema ParsedEnumSchema;
		if (Parser.ParseEnumSchema(ParsedEnumSchema))
		{
			check(ParsedEnumSchema.Id.Idx == Schemas.Enums.Num());
			Schemas.Enums.Emplace(MoveTemp(ParsedEnumSchema));
		}
	}
}

void FBatchParser::ParseObjects(TConstArrayView64<FToken> TokensView)
{
	FTokenReader TokenIt(TokensView);
	verify(TokenIt.GrabToken().Value() == GLiterals.Objects);

	FScopedTokenReader ScopedReader(TokenIt);
	while (ScopedReader.HasMore())
	{
		// Each item in the "Objects" range is a struct with one leaf, where the
		// leaf's key is the object type and it's value a struct of the object;
		//
		//   [ {"<type>": { ... }}, {"<type>": ...etc
		//
		// So we've two structs and depths to fetch.

		FToken Token;
		if (!GrabToNextBeginStruct(Token, ScopedReader))
		{
			break;
		}

		FScopedTokenReader StructReader(TokenIt);
		if (!GrabToNextBeginStruct(Token, StructReader))
		{
			break;
		}

		TOptional<FTypeTokens> TypeTokens = TokenizeType(Token.Value());
		if (!TypeTokens)
		{
			checkf(false, TEXT("Invalid object struct schema '%s'"), *Print(Token));
			continue;
		}

		const FType Type = MakeType(TypeTokens.GetValue(), Names);
		const FOptionalDeclId StructId = Names.GetStructId(Type);

		checkf(StructId, TEXT("Object struct schema '%s' not found"), *Print(Token));
		const FParsedStructSchema& ParsedSchema = ResolveStructSchema(Schemas, StructId.Get());

		{
			FScopedTokenReader MemberTokens(TokenIt, Token);
			FMemberParser MemberParser(Scratch, ParsedSchema, MemberTokens, Names, Types, Schemas);
			FBuiltStruct* BuiltStruct = MemberParser.ParseMembers(StructId.Get());
			check(!MemberTokens.HasMore());
			Objects.Emplace(StructId.Get(), BuiltStruct);
		}

		Token = TokenIt.GrabToken(EToken::EndStruct);
	}
}

FDeclId FParseDeclarations::Lower(FBindId Id) const
{
	checkf(false, TEXT("All struct ids should be declared, nothing is bound with different names for text"));
	return LowerCast(Id);
}

const FStructDeclaration* FParseDeclarations::Find(FStructId Id) const
{
	if (Id.Idx < (uint32)Structs.Num())
	{
		return Structs[Id.Idx];
	}
	return nullptr;
}

void FBatchParser::Write(TArray64<uint8>& Out)
{
	FSchemasBuilder SchemaBuilders(Names, Types, Scratch, ESchemaFormat::StableNames);
	for (const TPair<FStructId, FBuiltStruct*>& Object : Objects)
	{
		SchemaBuilders.NoteStructAndMembers(Object.Key, *Object.Value);
	}
	FBuiltSchemas BuiltSchemas = SchemaBuilders.Build(); 

	FWriter Writer(Names, Types, BuiltSchemas, ESchemaFormat::StableNames);
	TArray64<uint8> Tmp;

	// Write schemas
	Writer.WriteSchemas(Tmp);
	WriteInt(Out, IntCastChecked<uint32>(Tmp.Num()));
	WriteArray(Out, Tmp);
	Tmp.Reset();

	// Write objects
	for (const TPair<FStructId, FBuiltStruct*>& Object : Objects)
	{
		WriteInt(Tmp, Writer.GetWriteId(Object.Key).Get().Idx);
		Writer.WriteMembers(Tmp, Object.Key, *Object.Value);
		WriteSkippableSlice(Out, Tmp);
		Tmp.Reset();
	}

	// Write object terminator
	WriteSkippableSlice(Out, TConstArrayView64<uint8>());
}

///////////////////////////////////////////////////////////////////////////////

void ParseJsonBatch(TArray64<uint8>& OutBinary, FUtf8StringView Json)
{
	FScratchAllocator Scratch;
	FJsonTokenizer JsonScanner(Json);
	FBatchParser BatchParser(JsonScanner, Scratch);
	BatchParser.Parse(OutBinary);
}

void ParseYamlBatch(TArray64<uint8>& OutBinary, FUtf8StringView Yaml)
{
	FScratchAllocator Scratch;
	FYamlTokenizer YamlScanner(Yaml);
	FBatchParser BatchParser(YamlScanner, Scratch);
	BatchParser.Parse(OutBinary);
}

} // namespace PlainProps
