// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsPrint.h"
#include "PlainPropsBuild.h"
#include "PlainPropsDiff.h"
#include "PlainPropsIndex.h"
#include "Containers/StringConv.h"
#include "Containers/Utf8String.h"
#include "PlainPropsInternalFormat.h"
#include "PlainPropsInternalPrint.h"
#include "PlainPropsInternalRead.h"
#include "PlainPropsInternalText.h"
#include "Misc/AsciiSet.h"
#include "Misc/StringBuilder.h"
#include <charconv>

namespace PlainProps
{

static constexpr bool PrintWithComments = true;

//////////////////////////////////////////////////////////////////////////

const FLiterals GLiterals;

FAnsiStringView ToString(ESizeType Width)
{
	return GLiterals.Ranges[(uint8)Width];
}

FAnsiStringView ToString(FRangeType Range)
{
	return GLiterals.RangeTypes[(uint8)Range.MaxSize];
}

FAnsiStringView ToString(EMemberKind Kind)
{
	return GLiterals.Kinds[(uint8)Kind];
}

FAnsiStringView ToString(FUnpackedLeafType Leaf)
{
	return GLiterals.Leaves[(uint8)Leaf.Type][(uint8)Leaf.Width];
}

FAnsiStringView ToString(ELeafWidth Width)
{
	return GLiterals.Widths[(uint8)Width];
}

FAnsiStringView ToString(FStructType Struct)
{
	if (Struct.IsSuper && Struct.IsDynamic)
	{
		return "DynamicSuperStruct";
	}
	else if (Struct.IsSuper)
	{
		return "SuperStruct";
	}
	else if (Struct.IsDynamic)
	{
		return "DynamicStruct";
	}
	return "Struct";
}

FAnsiStringView ToString(FMemberType Type)
{
	switch (Type.GetKind())
	{
	case EMemberKind::Leaf:		return ToString(Type.AsLeaf());
	case EMemberKind::Struct:	return ToString(Type.AsStruct());
	case EMemberKind::Range:	return ToString(Type.AsRange());
	}
	unimplemented();
	return "Unknown";
}

FAnsiStringView ToString(ESchemaFormat Format)
{
	switch (Format)
	{
		case ESchemaFormat::InMemoryNames:	return "InMemoryNames";
		case ESchemaFormat::StableNames:	return "StableNames";
	}
	return "Unknown";
}

//////////////////////////////////////////////////////////////////////////

void FIdIndexerBase::InitParameterNames()
{
	for (uint32 T = 0; T < 8; ++T)
	{
		for (uint32 W = 0; W < 4; ++W)
		{
			Leaves[T][W] = InitParameterName(GLiterals.Leaves[T][W]);
		}
	}

	for (uint32 S = 0; S < 9; ++S)
	{
		Ranges[S] = InitParameterName(GLiterals.Ranges[S]);
	}
}

///////////////////////////////////////////////////////////////////////////////
// Escape the quotation mark (U+0022), backslash (U+005C),
// and control characters U+0000 to U+001F (JSON Standard ECMA-404)
constexpr FAsciiSet EscapeSet("\\\""
	"\x00\x01\x02\x03\x04\x05\x06\x07\x08\x09\x0a\x0b\x0c\x0d\x0e\x0f"
	"\x10\x11\x12\x13\x14\x15\x16\x17\x18\x19\x1a\x1b\x1c\x1d\x1e\x1f");

static inline void EscapeChar(FUtf8Builder& Out, UTF8CHAR Char)
{
	switch (Char)
	{
	case '\"': Out.Append("\\\"");	break;
	case '\\': Out.Append("\\\\");	break;
	case '\b': Out.Append("\\b");	break;
	case '\f': Out.Append("\\f");	break;
	case '\n': Out.Append("\\n");	break;
	case '\r': Out.Append("\\r");	break;
	case '\t': Out.Append("\\t");	break;
	default:
		Out.Appendf(UTF8TEXT("\\u%04x"), uint32(Char));
		break;
	}
}

template<Enumeration T>
void Print(FUtf8Builder& Out, T Value)
{
	Out.Append(ToString(Value));
}

template<Arithmetic T> requires (std::is_integral_v<T>)
void Print(FUtf8Builder& Out, T Value)
{
	constexpr size_t Size = std::numeric_limits<T>::digits10 + 3;
	char Buf[Size];
	std::to_chars_result Result = std::to_chars(Buf, Buf + Size, Value);
	check(Result.ec == std::errc());
	Out.Append(Buf, Result.ptr - Buf);
}

template<UnsignedIntegral T>
struct FPrintFormatHex
{
	explicit FPrintFormatHex(T InValue) : Value(InValue) {}
	uint64 Value;
};

template<UnsignedIntegral T>
void Print(FUtf8Builder& Out, FPrintFormatHex<T> Hex)
{
	constexpr size_t Size = sizeof(T) * 2;
	char Buf[Size];
	std::to_chars_result Result = std::to_chars(Buf, Buf + Size, Hex.Value, 16);
	check(Result.ec == std::errc());
	Out.Append(Buf, Result.ptr - Buf);
}

template<Arithmetic T> requires (std::is_floating_point_v<T>)
void Print(FUtf8Builder& Out, T Value)
{
	constexpr size_t Size = 32;
	char Buf[Size];
	std::to_chars_result Result = std::to_chars(Buf, Buf + Size, Value, std::chars_format::general);
	check(Result.ec == std::errc());
	Out.Append(Buf, Result.ptr - Buf);
}

template<>
void Print(FUtf8Builder& Out, bool Value)
{
	Out.Append(Value ? GLiterals.True : GLiterals.False);
}

template<>
void Print(FUtf8Builder& Out, char8_t Value)
{
	UTF8CHAR Char = static_cast<UTF8CHAR>(Value);
	if (EscapeSet.Contains(Char))
	{
		EscapeChar(Out, Char);
	}
	else
	{
		Out.AppendChar(Char);
	}
}

template<>
void Print(FUtf8Builder& Out, char16_t Value)
{
	if (Value <= 127)
	{
		Print(Out, static_cast<char8_t>(Value));
	}
	else
	{
		Out.Append(FUtf8StringView(StringCast<UTF8CHAR>(reinterpret_cast<UTF16CHAR*>(&Value), 1)));
	}
}

template<>
void Print(FUtf8Builder& Out, char32_t Value)
{
	if (Value <= 127)
	{
		Print(Out, static_cast<char8_t>(Value));
	}
	else
	{
		Out.Append(FUtf8StringView(StringCast<UTF8CHAR>(reinterpret_cast<UTF32CHAR*>(&Value), 1)));
	}
}

static void PrintQuotedString(FUtf8Builder& Out, FUtf8StringView Value, UTF8CHAR VerbatimQuote='\'')
{
	FUtf8StringView Verbatim = FAsciiSet::FindPrefixWithout(Value, EscapeSet | "'");
	if (Verbatim.Len() == Value.Len())
	{
		Out << VerbatimQuote << Value << VerbatimQuote;
		return;
	}

	Out << '\"';
	while (!Value.IsEmpty())
	{
		Out << Verbatim;
		Value.RightChopInline(Verbatim.Len());
		FUtf8StringView Escape = FAsciiSet::FindPrefixWith(Value, EscapeSet);
		for (UTF8CHAR Char : Escape)
		{
			EscapeChar(Out, Char);
		}
		Value.RightChopInline(Escape.Len());
		Verbatim = FAsciiSet::FindPrefixWithout(Value, EscapeSet);
	}
	Out << '\"';
}


///////////////////////////////////////////////////////////////////////////////

static void PrintRangeType(FUtf8Builder& Out, FRangeType Type)
{
	Out.AppendChar('(');
	Out.Append(ToString(Type.MaxSize));
	Out.AppendChar(')');
}

inline void PrintStructFlags(FUtf8Builder& Out, FStructType StructType)
{
	if (StructType.IsDynamic)
	{
		Out.Append(GLiterals.Dynamic);
	}
}

void PrintSchema(FUtf8Builder& Out, const FBatchIds& Ids, FStructType StructType, FStructSchemaId Id)
{
	PrintStructFlags(Out, StructType);
	Ids.AppendString(Out, Id);
}

template<typename IdsType, typename OptionalStructId>
void PrintSchema(FUtf8Builder& Out, const IdsType& Ids, FStructType StructType, OptionalStructId Id)
{
	PrintStructFlags(Out, StructType);
	if (Id)
	{
		Ids.AppendString(Out, Id.Get());
	}
}

template<typename IdsType, typename OptionalEnumId>
void PrintSchema(FUtf8Builder& Out, const IdsType& Ids, FUnpackedLeafType Leaf, OptionalEnumId Id)
{
	if (Id)
	{
		Ids.AppendString(Out, Id.Get());
	}
	else
	{
		Out.Append(ToString(Leaf));
	}
}

template<typename IdsType, typename OptionalId>
void PrintInnermostSchema(FUtf8Builder& Out, const IdsType& Ids, FMemberType InnermostType, OptionalId InnerSchema)
{
	if (InnermostType.IsStruct())
	{
		PrintSchema(Out, Ids, InnermostType.AsStruct(), ToOptionalStruct(InnerSchema));
	}
	else
	{
		PrintSchema(Out, Ids, InnermostType.AsLeaf(), ToOptionalEnum(InnerSchema));
	}
}

static void PrintSchema(FUtf8Builder& Out, const FBatchIds& Ids, FRangeType Type, FRangeSchema Schema)
{
	PrintInnermostSchema(Out, Ids, GetInnermostType(Schema), Schema.InnermostSchema);

	PrintRangeType(Out, Type);
	FMemberType Inner = Schema.ItemType;
	for (const FMemberType* It = Schema.NestedItemTypes; Inner.IsRange(); Inner = *It++)
	{
		PrintRangeType(Out, Inner.AsRange());
	}
}

void PrintMemberSchema(FUtf8Builder& Out, const FIds& Ids, FMemberSchema Member)
{
	PrintInnermostSchema(Out, Ids, Member.GetInnermostType(), Member.InnerSchema);

	if (Member.Type.IsRange())
	{
		PrintRangeType(Out, Member.Type.AsRange());
		check(!Member.GetInnerRangeTypes().Last().IsRange());
		for (FMemberType Inner : Member.GetInnerRangeTypes().LeftChop(1))
		{
			PrintRangeType(Out, Inner.AsRange());
		}
	}
}

///////////////////////////////////////////////////////////////////////////////
template <UnsignedIntegral T, bool bFlagIndexing>
static FNameId GetEnumConstantId(const FEnumSchema& Schema, T Value)
{
	using ValueType = decltype(Value);

	TConstArrayView<FNameId> Names = MakeConstArrayView(Schema.Footer, Schema.Num);

	if (Schema.ExplicitConstants)
	{
		for (uint32 Idx = 0; ValueType Constant : GetConstants<ValueType>(Schema))
		{
			if (Constant == Value)
			{
				return Names[Idx];
			}
			++Idx;
		}
	}
	else
	{
		if constexpr (bFlagIndexing)
		{
			Value = 63 - FMath::CountLeadingZeros64(uint64(Value));
		}

		if (Value < ValueType(Names.Num()))
		{
			return Names[Value];
		}
	}

	return {};
}

template <UnsignedIntegral T>
static void PrintEnumValue(
	FUtf8StringBuilderBase& Buffer,
	T Value,
	const FEnumSchema& Schema,
	const FBatchIds& Ids)
{
	if (Schema.FlagMode == 0)
	{
		FNameId ConstantId = GetEnumConstantId<T, false>(Schema, Value);
		checkf(ConstantId != FNameId(), TEXT("Unknown enum constant value '%lld'"), int64(Value));
		Ids.AppendString(Buffer, ConstantId);
		return;
	}

	if (Value == 0)
	{
		if (FNameId ConstantId = GetEnumConstantId<T, true>(Schema, Value); ConstantId != FNameId())
		{
			Ids.AppendString(Buffer, ConstantId);
		}
		else
		{
			Buffer << GLiterals.UnsetFlagEnum;
		}
		return;
	}

	using ValueType = decltype(Value);
	FUtf8StringView Separator;
	do
	{
		ValueType Flag = Value & (~Value + 1);
		Value ^= Flag;
		FNameId ConstantId = GetEnumConstantId<T, true>(Schema, Flag);
		checkf(ConstantId != FNameId(), TEXT("Unknown enum flag value '%llx'"), uint64(Flag));
		Buffer << Separator;
		Ids.AppendString(Buffer, ConstantId);
		Separator = " | ";
	}
	while (Value != 0);
}



///////////////////////////////////////////////////////////////////////////////

class IMarkupBuilder
{
public:
	virtual void PushDocument() = 0;
	virtual void PopDocument() = 0;

	virtual void PushStruct() = 0;
	virtual void PopStruct() = 0;
	virtual void WriteId(FUtf8StringView Id) = 0;
	virtual void WriteValue(FUtf8StringView Value, bool bEscape=false) = 0;

	virtual void PushRange() = 0;
	virtual void PopRange() = 0;
	virtual void WriteItem(FUtf8StringView Value) = 0;

	virtual void WriteComment(FUtf8StringView Comment) = 0;
};



///////////////////////////////////////////////////////////////////////////////

class FMarkupBuilder
	: public IMarkupBuilder
{
public:
	virtual		~FMarkupBuilder()				= default;
	void		BeginDocument()					{ PushDocument(); }
	void		EndDocument()					{ PopDocument(); }
	void		BeginStruct()					{ PushStruct(); }
	void		BeginStruct(FUtf8StringView Id)	{ WriteId(Id); BeginStruct(); }
	void		EndStruct()						{ PopStruct(); }
	void		BeginRange()					{ PushRange(); }
	void		BeginRange(FUtf8StringView Id)	{ WriteId(Id); BeginRange(); }
	void		EndRange()						{ PopRange(); }
	void		AddComment(FUtf8StringView Str) { WriteComment(Str); }

	// split calls for id and value pairs
	void						AddId(FUtf8StringView Id) { WriteId(Id); }
	template<typename T> void	AddValue(T Value);
	void						AddValue(FUtf8StringView Value);
	void						AddValue(const auto*) = delete;

	// range items
	template<typename T> void	AddItem(T Value);
	void						AddItem(FUtf8StringView Value);
	void						AddItem(const auto*) = delete;

	// id value pairs
	template<typename T>
	void AddLeaf(FUtf8StringView Id, T Value)
	{
		WriteId(Id);
		AddValue(Value);
	}
};

template<typename T>
void FMarkupBuilder::AddValue(T Value)
{
	TUtf8StringBuilder<256> Buffer;
	Print(Buffer, Value);
	WriteValue(Buffer);
}

void FMarkupBuilder::AddValue(FUtf8StringView Value)
{
	WriteValue(Value, true);
}

template<typename T>
void FMarkupBuilder::AddItem(T Value)
{
	TUtf8StringBuilder<256> Buffer;
	Print(Buffer, Value);
	WriteItem(Buffer);
}

void FMarkupBuilder::AddItem(FUtf8StringView Value)
{
	WriteItem(Value);
}

///////////////////////////////////////////////////////////////////////////////

struct FMemberSchemaView
{
	using FMemberTypeRange = TConstArrayView<FMemberType>;

	FMemberType				Type;
	FSchemaBatchId			Batch;
	FOptionalSchemaId		InnerSchema;
	FMemberTypeRange		InnerRangeTypes;

	FMemberType GetInnermostType() const
	{
		return InnerRangeTypes.Num() > 0 ? InnerRangeTypes.Last() : Type;
	}

	FRangeSchema AsRangeSchema() const
	{
		check(Type.IsRange());
		return { InnerRangeTypes[0], Batch, InnerSchema, InnerRangeTypes.Num() > 1 ? &InnerRangeTypes[1] : nullptr };
	}
};

///////////////////////////////////////////////////////////////////////////////

class FStructSchemaReader
{
public:
	FStructSchemaReader(const FStructSchema& Schema, FSchemaBatchId InBatch);

	FType					GetStruct() const		{ return Struct; }
	bool					IsDense() const			{ return bIsDense; }
	bool					HasSuper() const		{ return bHasSuper; }
	uint16					GetVersion() const		{ return Version; }

	bool					HasMore() const			{ return MemberIdx < NumMembers; }

	FOptionalMemberId		PeekName() const;		// @pre HasMore()
	EMemberKind				PeekKind() const;		// @pre HasMore()
	FMemberType				PeekType() const;		// @pre HasMore()

	FStructSchemaHandle		GetSuper() const;		// @pre HasSuper()
	FMemberSchemaView 		GrabMember();			// @pre HasMore()

private:
	const FMemberType*		Footer;
	const FSchemaBatchId	Batch;					// Needed to resolve schemas
	const FType				Struct;
	const bool				bIsDense : 1;
	const bool				bHasSuper : 1;
	const bool				bUsesSuper : 1;
	const uint16			Version;
	const uint32			NumMembers;
	const uint32			NumRangeTypes;			// Number of ranges and nested ranges
	const uint32			NumInnerSchemas;		// Number of static structs and enums

	uint32					MemberIdx = 0;
	uint32					RangeTypeIdx = 0;		// Types of [nested] ranges
	uint32					InnerSchemaIdx = 0;		// Types of static structs and enums

	void					AdvanceToNextMember()	{ ++MemberIdx; }

	using FMemberTypeRange = TConstArrayView<FMemberType>;

	FMemberTypeRange		GrabRangeTypes();
	FSchemaId				GrabInnerSchema();
	FOptionalSchemaId		GrabLeafSchema(FLeafType Leaf);
	FOptionalSchemaId		GrabStructSchema(FStructType Struct);
	FOptionalSchemaId		GrabRangeSchema(FMemberType InnermostType);

	const FMemberType*		GetMemberTypes() const;
	const FMemberType*		GetRangeTypes() const;
	const FSchemaId*		GetInnerSchemas() const;
	const FMemberId*		GetMemberNames() const;
};

///////////////////////////////////////////////////////////////////////////////

class FJsonBuilder final
	: public FMarkupBuilder
{
public:
					FJsonBuilder(FUtf8Builder& InBuilder);
					~FJsonBuilder();
private:
	enum class EAction { Open, Add, Close };

	struct FScope
	{
		uint8		bIsStruct : 1;
		uint8		bHasItems : 1;
		uint8		KeyValueTickTock : 1;
		uint8		_Unused : 5;
	};

	using FScopes = TArray<FScope, TInlineAllocator<32>>;

	virtual void	PushDocument() override;
	virtual void	PopDocument() override;
	virtual void	PushStruct() override;
	virtual void	PopStruct() override;
	virtual void	PushRange() override;
	virtual void	PopRange() override;
	virtual void	WriteId(FUtf8StringView Id) override;
	virtual void	WriteValue(FUtf8StringView Value, bool bEscaped=false) override;
	virtual void	WriteItem(FUtf8StringView Value) override;
	virtual void	WriteComment(FUtf8StringView Comment) override;
	void			Prologue(EAction Action);
	FScopes			Scopes;
	FUtf8Builder&	Builder;
};

FJsonBuilder::FJsonBuilder(FUtf8Builder& InBuilder)
: Builder(InBuilder)
{
}

FJsonBuilder::~FJsonBuilder()
{
	check(Scopes.IsEmpty());
}

void FJsonBuilder::Prologue(EAction Action)
{
	if (Scopes.IsEmpty())
	{
		return;
	}

	FScope& Scope = Scopes.Last();

	bool bHadItems = Scope.bHasItems;
	Scope.bHasItems = true;

	bool bIndent = true;

	switch (Action)
	{
	case EAction::Close:
		Scopes.Pop(EAllowShrinking::No);
		if (!bHadItems)
		{
			return;
		}
		Builder << '\n';
		break;

	case EAction::Open:
		if (!Scope.bIsStruct && bHadItems)
		{
			Builder << ",";
		}
		bIndent = ((Scope.KeyValueTickTock & 1) == 0);
		if (bIndent)
		{
			Builder << "\n";
		}
		else
		Scope.KeyValueTickTock += (Scope.bIsStruct == true);
		break;

	case EAction::Add:
		bIndent = ((Scope.KeyValueTickTock & 1) == 0);
		if (bIndent)
		{
			Builder << (bHadItems ? ",\n" : "\n");
		}
		Scope.KeyValueTickTock += (Scope.bIsStruct == true);
		break;
	}

	if (!bIndent)
	{
		return;
	}

	int32 IndentNum = Scopes.Num() << 1;
	for (const FUtf8StringView Spaces("                                    ");;)
	{
		FUtf8StringView Trimmed = Spaces.Left(IndentNum);
		Builder << Trimmed;
		if (IndentNum -= Trimmed.Len(); IndentNum <= 0)
		{
			break;
		}
	}
}

void FJsonBuilder::PushDocument()
{
	PushStruct();
}

void FJsonBuilder::PopDocument()
{
	PopStruct();
	Builder << '\n';
}

void FJsonBuilder::PushStruct()
{
	Prologue(EAction::Open);

	Builder << '{';

	Scopes.Emplace(FScope{
		.bIsStruct = true,
	});
}

void FJsonBuilder::PopStruct()
{
	Prologue(EAction::Close);
	Builder << '}';
}

void FJsonBuilder::PushRange()
{
	Prologue(EAction::Open);
	Builder << '[';
	Scopes.Emplace(FScope{});
}

void FJsonBuilder::PopRange()
{
	Prologue(EAction::Close);
	Builder << ']';
}

void FJsonBuilder::WriteId(FUtf8StringView Id)
{
	Prologue(EAction::Add);

	PrintQuotedString(Builder, Id, '\"');
	Builder << " : ";
}

void FJsonBuilder::WriteValue(FUtf8StringView Value, bool bEscaped)
{
	Prologue(EAction::Add);

	if (bEscaped)
	{
		PrintQuotedString(Builder, Value, '\"');
		return;
	}

	Builder << '\"';
	Builder << Value;
	Builder << '\"';
}

void FJsonBuilder::WriteItem(FUtf8StringView Value)
{
	Prologue(EAction::Add);
	PrintQuotedString(Builder, Value, '\"');
}

void FJsonBuilder::WriteComment(FUtf8StringView Comment)
{
}



///////////////////////////////////////////////////////////////////////////////

class FYamlBuilder final
	: public FMarkupBuilder
{
public:
	FYamlBuilder(FUtf8Builder& InStringBuilder);
	~FYamlBuilder();

private:
	virtual void PushDocument() override;
	virtual void PopDocument() override;
	virtual void PushStruct() override;
	virtual void PopStruct() override;
	virtual void PushRange() override;
	virtual void PopRange() override;
	virtual void WriteId(FUtf8StringView Id) override;
	virtual void WriteValue(FUtf8StringView Value, bool bEscaped=false) override;
	virtual void WriteItem(FUtf8StringView Value) override;
	virtual void WriteComment(FUtf8StringView Comment) override;
	void AppendNewLine();
	void PopScope();

	struct FScopeInfo
	{
		uint8 HasItems : 1;
		uint8 IsInStruct : 1;
		uint8 _Unused : 6;
	};

	FUtf8Builder& Text;
	TArray<FScopeInfo, TInlineAllocator<32>> Scope;
	uint32 LineStart = 0;
};

///////////////////////////////////////////////////////////////////////////////

class FMemberPrinter
{
public:
	FMemberPrinter(FMarkupBuilder& InMarkupBuilder, const FBatchIds& InIds)
	: MarkupBuilder(InMarkupBuilder)
	, Ids(InIds)
	{}

	void PrintMembers(FStructView StructView);

private:
	void PrintLeaf(FMemberId Id, FLeafView LeafView);
	template <UnsignedIntegral T>
	void PrintLeafEnum(FMemberId Id, FLeafView LeafView);
	void PrintStruct(FOptionalMemberId Id, FStructType StructType, FStructView StructView);
	void PrintRange(FMemberId Id, FRangeType RangeType, const FRangeView& RangeView);

	void PrintLeaves(FUnpackedLeafType Leaf, const FLeafRangeView& LeafRange);
	template <UnsignedIntegral T>
	void PrintLeavesEnum(FUnpackedLeafType Leaf, const FLeafRangeView& LeafRange);
	void PrintStructs(FStructType StructType, const FStructRangeView& StructRange);
	void PrintRanges(FRangeType RangeType, const FNestedRangeView& NestedRange);

	void PrintMembersInternal(FStructType StructType, FStructView StructView, const FStructSchema& Schema);
	void PrintRangeInternal(FRangeType RangeType, const FRangeView& RangeView);

	bool IsUnicodeString(const FRangeView& RangeView);
	void PrintUnicodeLeafValue(FLeafView LeafView);
	void PrintUnicodeRangeAsLeaf(FOptionalMemberId Id, FRangeType RangeType, const FRangeView& RangeView);

	template<typename MemberType, typename SchemaType>
	void PrintSchemaComment(MemberType Type, SchemaType Schema)
	{
		if constexpr (PrintWithComments)
		{
			PrintSchema(/* out */ Tmp, Ids, Type, Schema);
			MarkupBuilder.AddComment(Tmp);
			Tmp.Reset();
		}
	}

	FMarkupBuilder& MarkupBuilder;
	const FBatchIds& Ids;
	TUtf8StringBuilder<256> Tmp;
};

///////////////////////////////////////////////////////////////////////////////

static void PrintBatch(FMarkupBuilder& MarkupBuilder, FBatchPrinter& Printer, TConstArrayView<FStructView> Objects)
{
	MarkupBuilder.BeginDocument();
	Printer.PrintSchemas();
	Printer.PrintObjects(Objects);
	MarkupBuilder.EndDocument();
}

void PrintYamlBatch(FUtf8Builder& Out, const FBatchIds& Ids, TConstArrayView<FStructView> Objects)
{
	FYamlBuilder YamlBuilder(Out);
	FBatchPrinter Printer(YamlBuilder, Ids);
	PrintBatch(YamlBuilder, Printer, Objects);
}

void PrintJsonBatch(FUtf8Builder& Out, const FBatchIds& Ids, TConstArrayView<FStructView> Objects)
{
	FJsonBuilder JsonBuilder(Out);
	FBatchPrinter Printer(JsonBuilder, Ids);
	PrintBatch(JsonBuilder, Printer, Objects);
}

///////////////////////////////////////////////////////////////////////////////

FStructSchemaReader::FStructSchemaReader(const FStructSchema& Schema, FSchemaBatchId InBatch)
: Footer(Schema.Footer)
, Batch(InBatch)
, Struct(Schema.Type)
, bIsDense(Schema.IsDense)
, bHasSuper(Schema.Inheritance != ESuper::No)
, bUsesSuper(UsesSuper(Schema.Inheritance))
, Version(Schema.Version)
, NumMembers(Schema.NumMembers)
, NumRangeTypes(Schema.NumRangeTypes)
, NumInnerSchemas(Schema.NumInnerSchemas)
, InnerSchemaIdx(SkipDeclaredSuperSchema(Schema.Inheritance))
{
	check(InnerSchemaIdx <= NumInnerSchemas);
	checkf(NumRangeTypes != 0xFFFFu, TEXT("GrabRangeTypes() doesn't check for wrap-around"));
}

FOptionalMemberId FStructSchemaReader::PeekName() const
{
	int32 MemberNameIdx = MemberIdx - bUsesSuper;
	return MemberNameIdx >= 0 ? ToOptional(GetMemberNames()[MemberNameIdx]) : NoId;
}

EMemberKind FStructSchemaReader::PeekKind() const
{
	return PeekType().GetKind();
}

FMemberType	FStructSchemaReader::PeekType() const
{
	check(HasMore());
	return GetMemberTypes()[MemberIdx];
}

FStructSchemaHandle FStructSchemaReader::GetSuper() const
{
	check(HasSuper());
	check(NumInnerSchemas > 0);
	return { static_cast<FStructSchemaId>(GetInnerSchemas()[0]), Batch };
}

FMemberSchemaView FStructSchemaReader::GrabMember()
{
	check(HasMore());
	FMemberType Type = PeekType();
	FMemberSchemaView Out{ Type, Batch };

	switch (PeekKind())
	{
		case EMemberKind::Leaf:
			Out.InnerSchema = GrabLeafSchema(Type.AsLeaf());
			break;
		case EMemberKind::Struct:
			Out.InnerSchema = GrabStructSchema(Type.AsStruct());
			break;
		case EMemberKind::Range:
			Out.InnerRangeTypes = GrabRangeTypes();
			Out.InnerSchema = GrabRangeSchema(Out.InnerRangeTypes.Last());
			break;
	}

	AdvanceToNextMember();

	return Out;
}

FStructSchemaReader::FMemberTypeRange FStructSchemaReader::GrabRangeTypes()
{
	return GrabInnerRangeTypes(MakeArrayView(GetRangeTypes(), NumRangeTypes), /* in-out */ RangeTypeIdx);
}

FSchemaId FStructSchemaReader::GrabInnerSchema()
{
	check(InnerSchemaIdx < NumInnerSchemas);
	uint32 Idx = InnerSchemaIdx++;
	return GetInnerSchemas()[Idx];
}

FOptionalSchemaId FStructSchemaReader::GrabLeafSchema(FLeafType Member)
{
	return Member.Type == ELeafType::Enum ? ToOptional(GrabInnerSchema()) : NoId;
}

FOptionalSchemaId FStructSchemaReader::GrabStructSchema(FStructType Member)
{
	return Member.IsDynamic ? NoId : ToOptional(GrabInnerSchema());
}

FOptionalSchemaId FStructSchemaReader::GrabRangeSchema(FMemberType InnermostType)
{
	check(!InnermostType.IsRange());
	return InnermostType.IsStruct() ?
		GrabStructSchema(InnermostType.AsStruct()) :
		GrabLeafSchema(InnermostType.AsLeaf());
}

const FMemberType* FStructSchemaReader::GetMemberTypes() const
{
	return FStructSchema::GetMemberTypes(Footer);
}

const FMemberType* FStructSchemaReader::GetRangeTypes() const
{
	return FStructSchema::GetRangeTypes(Footer, NumMembers);
}

const FSchemaId* FStructSchemaReader::GetInnerSchemas() const
{
	return FStructSchema::GetInnerSchemas(Footer, NumMembers, NumRangeTypes, NumMembers - bUsesSuper);
}

const FMemberId* FStructSchemaReader::GetMemberNames() const
{
	return FStructSchema::GetMemberNames(Footer, NumMembers, NumRangeTypes);
}

///////////////////////////////////////////////////////////////////////////////

void FMarkupBuilderDeleter::operator()(FMarkupBuilder* MarkupBuilder) const
{
	delete MarkupBuilder;
}

FYamlBuilderPtr MakeYamlBuilder(FUtf8Builder& StringBuilder)
{
	return FYamlBuilderPtr(new FYamlBuilder(StringBuilder));
}

///////////////////////////////////////////////////////////////////////////////

FYamlBuilder::FYamlBuilder(FUtf8Builder& InStringBuilder)
: Text(InStringBuilder)
{
	Scope.Emplace(FScopeInfo{});
}

FYamlBuilder::~FYamlBuilder()
{
	Scope.Pop(EAllowShrinking::No);
	check(Scope.IsEmpty());
}

void FYamlBuilder::PushDocument()
{
	Text << "---\n";
	Scope.Emplace(FScopeInfo{
		.IsInStruct = true,
	});
}

void FYamlBuilder::PopDocument()
{
	Text << "...";
	Scope.Pop(EAllowShrinking::No);
}

void FYamlBuilder::PushStruct()
{
	Scope.Last().HasItems = true;
	bool bNeedNewLine = Scope.Last().IsInStruct;

	Scope.Emplace(FScopeInfo{
		.IsInStruct = true,
	});

	// Current parser needs struct-type range items to be on a new line.
	bNeedNewLine = true;

	LineStart = Text.Len();
	if (bNeedNewLine)
	{
		AppendNewLine();
	}
}

void FYamlBuilder::PopStruct()
{
	PopScope();
}

void FYamlBuilder::PushRange()
{
	Scope.Last().HasItems = true;
	Scope.Emplace(FScopeInfo{
		.IsInStruct = false,
	});

	AppendNewLine();
}

void FYamlBuilder::PopRange()
{
	PopScope();
}

void FYamlBuilder::WriteId(FUtf8StringView Id)
{
	PrintQuotedString(Text, Id);
	Text.AppendChar(':');

	Scope.Last().HasItems = true;
}

void FYamlBuilder::WriteValue(FUtf8StringView Value, bool bEscaped)
{
	if (bEscaped)
	{
		Text << ' ';
		PrintQuotedString(Text, Value);
	}
	else
	{
		Text << " \'";
		Text << Value;
		Text.AppendChar('\'');
	}

	Scope.Last().HasItems = true;

	AppendNewLine();
}

void FYamlBuilder::WriteItem(FUtf8StringView Value)
{
	check(Scope.Last().IsInStruct == false);
	WriteValue(Value, true);
}

void FYamlBuilder::WriteComment(FUtf8StringView Comment)
{
	Text.RemoveSuffix(Text.Len() - LineStart);
	Text << " #" << Comment;

	AppendNewLine();
}

void FYamlBuilder::AppendNewLine()
{
	LineStart = Text.Len();

	Text << '\n';

	int32 IndentNum = (Scope.Num() - 2) << 1;
	Text.Reserve(LineStart + IndentNum);
	for (const FUtf8StringView Spaces("                                    ");;)
	{
		FUtf8StringView Trimmed = Spaces.Left(IndentNum);
		Text << Trimmed;
		if (IndentNum -= Trimmed.Len(); IndentNum <= 0)
		{
			break;
		}
	}

	if (Scope.Last().IsInStruct == false)
	{
		Text << "- ";
	}
}

void FYamlBuilder::PopScope()
{
	FScopeInfo Last = Scope.Last();
	Scope.Pop(EAllowShrinking::No);

	Text.RemoveSuffix(Text.Len() - LineStart);

	if (!Last.HasItems)
	{
		Text << (Last.IsInStruct ? " {}" : " []");
	}

	AppendNewLine();
}


///////////////////////////////////////////////////////////////////////////////

FBatchPrinter::FBatchPrinter(FMarkupBuilder& InMarkupBuilder, const FBatchIds& InIds)
: MarkupBuilder(InMarkupBuilder)
, Ids(InIds)
{}

FBatchPrinter::~FBatchPrinter()
{}

void FBatchPrinter::PrintSchemas()
{
	MarkupBuilder.BeginStruct(GLiterals.Structs);
	for (const FStructSchema& Struct : GetStructSchemas(Ids.GetSchemas()))
	{
		PrintStructSchema(Struct, Ids.GetBatchId());
	}
	MarkupBuilder.EndStruct();

	MarkupBuilder.BeginStruct(GLiterals.Enums);
	for (const FEnumSchema& EnumSchema : GetEnumSchemas(Ids.GetSchemas()))
	{
		PrintEnumSchema(EnumSchema);
	}
	MarkupBuilder.EndStruct();
}

void FBatchPrinter::PrintObjects(TConstArrayView<FStructView> Objects)
{
	MarkupBuilder.BeginRange(GLiterals.Objects);
	for (FStructView Object : Objects)
	{
		FMemberPrinter(MarkupBuilder, Ids).PrintMembers(Object);
	}
	MarkupBuilder.EndRange();
}

static void PrintMemberSchema(FUtf8Builder& Out, const FBatchIds& Ids, const FMemberSchemaView& Schema)
{
	switch (Schema.Type.GetKind())
	{
	case EMemberKind::Leaf:		PrintSchema(Out, Ids, Schema.Type.AsLeaf(), ToOptionalEnum(Schema.InnerSchema)); break;
	case EMemberKind::Range:	PrintSchema(Out, Ids, Schema.Type.AsRange(), Schema.AsRangeSchema()); break;
	case EMemberKind::Struct:	PrintSchema(Out, Ids, Schema.Type.AsStruct(), ToOptionalStruct(Schema.InnerSchema)); break;
	}
}

template <int32 BufferSize>
class FPrintId
{
	TUtf8StringBuilder<BufferSize>		Buffer;
public:
	template <typename T>
	FPrintId(const FBatchIds& Ids, T Id) { Ids.AppendString(Buffer, Id); }
	FUtf8StringView operator*() const	{ return Buffer.ToView(); }
};

void FBatchPrinter::PrintStructSchema(const FStructSchema& Struct, FSchemaBatchId BatchId)
{
	FStructSchemaReader Reader(Struct, BatchId);

	MarkupBuilder.BeginStruct(*FPrintId<128>(Ids, Reader.GetStruct()));

	if (uint16 Version = Reader.GetVersion())
	{
		MarkupBuilder.AddLeaf(GLiterals.Version, Version);
	}

	if (Reader.HasSuper())
	{
		const FStructSchema& SuperSchema = Reader.GetSuper().Resolve();
		MarkupBuilder.AddLeaf(GLiterals.DeclaredSuper, *FPrintId<128>(Ids, SuperSchema.Type));
	}

	MarkupBuilder.BeginStruct(GLiterals.Members);
	TUtf8StringBuilder<256> Buf;
	while (Reader.HasMore())
	{
		Ids.AppendString(Buf, Reader.PeekName());
		MarkupBuilder.AddId(Buf);
		Buf.Reset();

		PrintMemberSchema(Buf, Ids, Reader.GrabMember());
		MarkupBuilder.AddValue(FUtf8StringView(Buf));
		Buf.Reset();
	}
	MarkupBuilder.EndStruct();

	MarkupBuilder.EndStruct();
}

void FBatchPrinter::PrintEnumSchema(const FEnumSchema& Enum)
{
	MarkupBuilder.BeginStruct(*FPrintId<128>(Ids, Enum.Type));
	MarkupBuilder.AddLeaf(GLiterals.FlagMode, !!Enum.FlagMode);
	MarkupBuilder.AddLeaf(GLiterals.Width, Enum.Width);

	MarkupBuilder.BeginStruct(GLiterals.Constants);
	TConstArrayView<FNameId> EnumNames = MakeConstArrayView(Enum.Footer, Enum.Num);
	switch (Enum.Width)
	{
		case ELeafWidth::B8:
			PrintEnumConstants(EnumNames, GetConstants<uint8>(Enum), Enum.FlagMode);
			break;
		case ELeafWidth::B16:
			PrintEnumConstants(EnumNames, GetConstants<uint16>(Enum), Enum.FlagMode);
			break;
		case ELeafWidth::B32:
			PrintEnumConstants(EnumNames, GetConstants<uint32>(Enum), Enum.FlagMode);
			break;
		case ELeafWidth::B64:
			PrintEnumConstants(EnumNames, GetConstants<uint64>(Enum), Enum.FlagMode);
			break;
	}
	MarkupBuilder.EndStruct();

	MarkupBuilder.EndStruct();
}

template<typename IntType>
void FBatchPrinter::PrintEnumConstants(
	TConstArrayView<FNameId> EnumNames,
	TConstArrayView<IntType> Constants,
	bool bFlagMode)
{
	uint16 NamesNum = IntCastChecked<uint16>(EnumNames.Num());
	if (Constants.Num() > 0)
	{
		check(EnumNames.Num() == Constants.Num());
		for (uint16 Idx = 0; Idx < NamesNum; ++Idx)
		{
			MarkupBuilder.AddLeaf(*FPrintId<128>(Ids, EnumNames[Idx]), (uint64)Constants[Idx]);
		}
	}
	else if (bFlagMode)
	{
		uint64 Value = 1;
		for (uint16 Idx = 0; Idx < NamesNum; ++Idx)
		{
			MarkupBuilder.AddLeaf(*FPrintId<128>(Ids, EnumNames[Idx]), Value);
			Value <<= 1;
		}
	}
	else
	{
		for (uint16 Idx = 0; Idx < NamesNum; ++Idx)
		{
			MarkupBuilder.AddLeaf(*FPrintId<128>(Ids, EnumNames[Idx]), (uint64)Idx);
		}
	}
}

///////////////////////////////////////////////////////////////////////////////

void FMemberPrinter::PrintMembers(FStructView StructView)
{
	const FStructSchema& Schema = StructView.Schema.Resolve();
	MarkupBuilder.BeginStruct();
	MarkupBuilder.BeginStruct(*FPrintId<128>(Ids, Schema.Type));
	PrintMembersInternal({ EMemberKind::Struct }, StructView, Schema);
	MarkupBuilder.EndStruct();
}

void FMemberPrinter::PrintLeaf(FMemberId Id, FLeafView LeafView)
{
	MarkupBuilder.AddId(*FPrintId<128>(Ids, Id));

	switch (LeafView.Leaf.Type)
	{
	case ELeafType::Bool:
		MarkupBuilder.AddValue(LeafView.AsBool());
		break;
	case ELeafType::IntS:
		switch (LeafView.Leaf.Width)
		{
			case ELeafWidth::B8:	MarkupBuilder.AddValue(LeafView.AsS8()); break;
			case ELeafWidth::B16:	MarkupBuilder.AddValue(LeafView.AsS16()); break;
			case ELeafWidth::B32:	MarkupBuilder.AddValue(LeafView.AsS32()); break;
			case ELeafWidth::B64:	MarkupBuilder.AddValue(LeafView.AsS64()); break;
		}
		break;
	case ELeafType::IntU:
		switch (LeafView.Leaf.Width)
		{
			case ELeafWidth::B8:	MarkupBuilder.AddValue(LeafView.AsU8()); break;
			case ELeafWidth::B16:	MarkupBuilder.AddValue(LeafView.AsU16()); break;
			case ELeafWidth::B32:	MarkupBuilder.AddValue(LeafView.AsU32()); break;
			case ELeafWidth::B64:	MarkupBuilder.AddValue(LeafView.AsU64()); break;
		}
		break;
	case ELeafType::Float:
		if (LeafView.Leaf.Width == ELeafWidth::B32)
		{
			MarkupBuilder.AddValue(LeafView.AsFloat());
		}
		else
		{
			check(LeafView.Leaf.Width == ELeafWidth::B64);
			MarkupBuilder.AddValue(LeafView.AsDouble());
		}
		break;
	case ELeafType::Hex:
		switch (LeafView.Leaf.Width)
		{
			case ELeafWidth::B8:	MarkupBuilder.AddValue(FPrintFormatHex<uint8 >{LeafView.AsHex8()});  break;
			case ELeafWidth::B16:	MarkupBuilder.AddValue(FPrintFormatHex<uint16>{LeafView.AsHex16()}); break;
			case ELeafWidth::B32:	MarkupBuilder.AddValue(FPrintFormatHex<uint32>{LeafView.AsHex32()}); break;
			case ELeafWidth::B64:	MarkupBuilder.AddValue(FPrintFormatHex<uint64>{LeafView.AsHex64()}); break;
		}
		break;
	case ELeafType::Enum:
		switch (LeafView.Leaf.Width)
		{
			case ELeafWidth::B8:	PrintLeafEnum<uint8 >(Id, LeafView); break;
			case ELeafWidth::B16:	PrintLeafEnum<uint16>(Id, LeafView); break;
			case ELeafWidth::B32:	PrintLeafEnum<uint32>(Id, LeafView); break;
			case ELeafWidth::B64:	PrintLeafEnum<uint64>(Id, LeafView); break;
		}
		break;
	case ELeafType::Unicode:
		switch (LeafView.Leaf.Width)
		{
			case ELeafWidth::B8:	MarkupBuilder.AddValue(LeafView.AsChar8());	 break;
			case ELeafWidth::B16:	MarkupBuilder.AddValue(LeafView.AsChar16()); break;
			case ELeafWidth::B32:	MarkupBuilder.AddValue(LeafView.AsChar32()); break;
			case ELeafWidth::B64:	check(false);								 break;
		};
	}

	PrintSchemaComment(LeafView.Leaf, LeafView.Enum);
}

template <UnsignedIntegral T>
void FMemberPrinter::PrintLeafEnum(FMemberId Id, FLeafView LeafView)
{
	check(LeafView.Leaf.Type == ELeafType::Enum);

	T Value = LeafView.AsUnderlyingValue<T>();

	FEnumSchemaId EnumSchemaId = LeafView.Enum.Get();
	const FSchemaBatch& SchemaBatch = Ids.GetSchemas();
	const FEnumSchema& EnumSchema = ResolveEnumSchema(SchemaBatch, EnumSchemaId);

	TUtf8StringBuilder<64> Buffer;
	PrintEnumValue(Buffer, Value, EnumSchema, Ids);

	MarkupBuilder.AddValue(FUtf8StringView(Buffer));
}

void FMemberPrinter::PrintStruct(FOptionalMemberId MemberId, FStructType StructType, FStructView StructView)
{
	MarkupBuilder.BeginStruct(*FPrintId<128>(Ids, MemberId));
	PrintMembersInternal(StructType, StructView, StructView.Schema.Resolve());
}

void FMemberPrinter::PrintRange(FMemberId Id, FRangeType RangeType, const FRangeView& RangeView)
{
	if (IsUnicodeString(RangeView))
	{
		PrintUnicodeRangeAsLeaf(Id, RangeType, RangeView);
	}
	else
	{
		MarkupBuilder.BeginRange(*FPrintId<128>(Ids, Id));
		PrintRangeInternal(RangeType, RangeView);
	}
}

void FMemberPrinter::PrintLeaves(FUnpackedLeafType Leaf, const FLeafRangeView& LeafRange)
{
	switch (Leaf.Type)
	{
	case ELeafType::Bool:
		for (bool b : LeafRange.AsBools()) { MarkupBuilder.AddItem(b); } break;
	case ELeafType::IntS:
		switch (Leaf.Width)
		{
			case ELeafWidth::B8:  for (const int8&  I : LeafRange.AsS8s())  { MarkupBuilder.AddItem(I); } break;
			case ELeafWidth::B16: for (const int16& I : LeafRange.AsS16s()) { MarkupBuilder.AddItem(I); } break;
			case ELeafWidth::B32: for (const int32& I : LeafRange.AsS32s()) { MarkupBuilder.AddItem(I); } break;
			case ELeafWidth::B64: for (const int64& I : LeafRange.AsS64s()) { MarkupBuilder.AddItem(I); } break;
		}
		break;
	case ELeafType::IntU:
		switch (Leaf.Width)
		{
			case ELeafWidth::B8:  for (const uint8&  U : LeafRange.AsU8s())  { MarkupBuilder.AddItem(U); } break;
			case ELeafWidth::B16: for (const uint16& U : LeafRange.AsU16s()) { MarkupBuilder.AddItem(U); } break;
			case ELeafWidth::B32: for (const uint32& U : LeafRange.AsU32s()) { MarkupBuilder.AddItem(U); } break;
			case ELeafWidth::B64: for (const uint64& U : LeafRange.AsU64s()) { MarkupBuilder.AddItem(U); } break;
		}
		break;
	case ELeafType::Float:
		if (Leaf.Width == ELeafWidth::B32)
		{
			for (const float& f : LeafRange.AsFloats()) { MarkupBuilder.AddItem(f); }
		}
		else
		{
			check(Leaf.Width == ELeafWidth::B64);
			for (const double& d : LeafRange.AsDoubles()) { MarkupBuilder.AddItem(d); }
		}
		break;
	case ELeafType::Hex:
		switch (Leaf.Width)
		{
			case ELeafWidth::B8:  for (uint8  U : LeafRange.AsHex8s())  { MarkupBuilder.AddItem(FPrintFormatHex<uint8> {U}); } break;
			case ELeafWidth::B16: for (uint16 U : LeafRange.AsHex16s()) { MarkupBuilder.AddItem(FPrintFormatHex<uint16>{U}); } break;
			case ELeafWidth::B32: for (uint32 U : LeafRange.AsHex32s()) { MarkupBuilder.AddItem(FPrintFormatHex<uint32>{U}); } break;
			case ELeafWidth::B64: for (uint64 U : LeafRange.AsHex64s()) { MarkupBuilder.AddItem(FPrintFormatHex<uint64>{U}); } break;
		}
		break;
	case ELeafType::Enum:
		switch (Leaf.Width)
		{
			case ELeafWidth::B8:  PrintLeavesEnum<uint8 >(Leaf, LeafRange); break;
			case ELeafWidth::B16: PrintLeavesEnum<uint16>(Leaf, LeafRange); break;
			case ELeafWidth::B32: PrintLeavesEnum<uint32>(Leaf, LeafRange); break;
			case ELeafWidth::B64: PrintLeavesEnum<uint64>(Leaf, LeafRange); break;
		}
		break;
	case ELeafType::Unicode:
		checkf(LeafRange.Num() == 0, TEXT("Should have been handled by PrintUnicodeRangeAsLeaf"));
		break;
	}
}

template <UnsignedIntegral T>
void FMemberPrinter::PrintLeavesEnum(FUnpackedLeafType Leaf, const FLeafRangeView& LeafRange)
{
	FEnumSchemaId EnumSchemaId = GetEnumSchemaId(LeafRange).Get();
	const FSchemaBatch& SchemaBatch = Ids.GetSchemas();
	const FEnumSchema& EnumSchema = ResolveEnumSchema(SchemaBatch, EnumSchemaId);

	for (const T Value: LeafRange.AsUnderlyingValues<T>())
	{
		TUtf8StringBuilder<128> Buffer;
		PrintEnumValue(Buffer, Value, EnumSchema, Ids);
		MarkupBuilder.AddItem(FUtf8StringView(Buffer));
	}
}

void FMemberPrinter::PrintStructs(FStructType StructType, const FStructRangeView& StructRange)
{
	for (FStructView StructView : StructRange)
	{
		MarkupBuilder.BeginStruct();
		PrintMembersInternal(StructType, StructView, StructView.Schema.Resolve());
	}
}

void FMemberPrinter::PrintRanges(FRangeType RangeType, const FNestedRangeView& NestedRange)
{
	for (FRangeView RangeView : NestedRange)
	{
		if (IsUnicodeString(RangeView))
		{
			PrintUnicodeRangeAsLeaf(NoId, RangeType, RangeView);
		}
		else
		{
			MarkupBuilder.BeginRange();
			PrintRangeInternal(RangeType, RangeView);
		}
	}
}

void FMemberPrinter::PrintMembersInternal(FStructType StructType, FStructView StructView, const FStructSchema& Schema)
{
	FMemberReader It(Schema, StructView.Values, StructView.Schema.Batch);

	const bool HasMembers = StructType.IsDynamic || It.HasMore();
	if (HasMembers)
	{
		PrintSchemaComment(StructType, StructView.Schema.Id);
	}

	if (StructType.IsDynamic)
	{
		MarkupBuilder.AddLeaf(GLiterals.Dynamic, *FPrintId<128>(Ids, Schema.Type));
	}
	while (It.HasMore())
	{
		FOptionalMemberId Id = It.PeekName();
		FMemberType Type = It.PeekType();
		switch (Type.GetKind())
		{
			case EMemberKind::Leaf:
				PrintLeaf(Id.Get(), It.GrabLeaf());
				break;
			case EMemberKind::Struct:
				PrintStruct(Id, Type.AsStruct(), It.GrabStruct());
				break;
			case EMemberKind::Range:
				PrintRange(Id.Get(), Type.AsRange(), It.GrabRange());
				break;
		}
	}
	MarkupBuilder.EndStruct();

	if (!HasMembers)
	{
		PrintSchemaComment(StructType, StructView.Schema.Id);
	}
}

void FMemberPrinter::PrintRangeInternal(FRangeType RangeType, const FRangeView& RangeView)
{
	FRangeSchema Schema = GetSchema(RangeView);
	if (RangeView.Num() > 0)
	{
		PrintSchemaComment(RangeType, Schema);
	}

	switch (Schema.ItemType.GetKind())
	{
		case EMemberKind::Leaf:
			PrintLeaves(Schema.ItemType.AsLeaf(), RangeView.AsLeaves());
			break;
		case EMemberKind::Struct:
			PrintStructs(Schema.ItemType.AsStruct(), RangeView.AsStructs());
			break;
		case EMemberKind::Range:
			PrintRanges(Schema.ItemType.AsRange(), RangeView.AsRanges());
			break;
	}
	MarkupBuilder.EndRange();

	if (RangeView.Num() == 0)
	{
		PrintSchemaComment(RangeType, Schema);
	}
}

bool FMemberPrinter::IsUnicodeString(const FRangeView& RangeView)
{
	FMemberType Type = GetSchema(RangeView).ItemType;
	return RangeView.Num() > 0 && Type.IsLeaf() && Type.AsLeaf().Type == ELeafType::Unicode;
}

template <typename CharType, typename RangeCharType>
static void AddUnicodeRangeLeaf(FMarkupBuilder& MarkupBuilder, FOptionalMemberId Id, TRangeView<RangeCharType> Range)
{
	check(Range.Num() > 0);
	const CharType* Src = reinterpret_cast<const CharType*>(Range.begin());
	const int32 SrcLen = IntCastChecked<int32>(Range.Num());
	const int32 DstLen = FPlatformString::ConvertedLength<UTF8CHAR>(Src, SrcLen);
	TArray<UTF8CHAR, TInlineAllocator<1024>> Buf;
	Buf.SetNumUninitialized(DstLen);
	UTF8CHAR* Dst = Buf.GetData();
	const UTF8CHAR* DstEnd = FPlatformString::Convert(Dst, DstLen, Src, SrcLen);
	check(DstEnd);
	check(DstEnd - Dst == DstLen);
	if (Id)
	{
		MarkupBuilder.AddValue(FUtf8StringView(Dst, DstLen));
	}
	else
	{
		MarkupBuilder.AddItem(FUtf8StringView(Dst, DstLen));
	}
}

void FMemberPrinter::PrintUnicodeRangeAsLeaf(FOptionalMemberId Id, FRangeType RangeType, const FRangeView& RangeView)
{
	check(IsUnicodeString(RangeView));

	if (Id)
	{
		MarkupBuilder.AddId(*FPrintId<128>(Ids, Id));
	}

	const FLeafRangeView LeafRange = RangeView.AsLeaves();
	const FUnpackedLeafType Leaf = GetSchema(RangeView).ItemType.AsLeaf();

	switch (Leaf.Width)
	{
		case ELeafWidth::B8:	AddUnicodeRangeLeaf<UTF8CHAR >(MarkupBuilder, Id, LeafRange.AsUtf8());	break;
		case ELeafWidth::B16:	AddUnicodeRangeLeaf<UTF16CHAR>(MarkupBuilder, Id, LeafRange.AsUtf16());	break;
		case ELeafWidth::B32:	AddUnicodeRangeLeaf<UTF32CHAR>(MarkupBuilder, Id, LeafRange.AsUtf32());	break;
		case ELeafWidth::B64:	check(false);															break;
	};

	PrintSchemaComment(FRangeType(RangeType), GetSchema(RangeView));
}

///////////////////////////////////////////////////////////////////////////////

void FIdsBase::AppendString(FUtf8Builder& Out, FMemberId Name) const
{
	AppendString(Out, Name.Id);
}

void FIdsBase::AppendString(FUtf8Builder& Out, FOptionalMemberId Name) const
{
	if (Name)
	{
		AppendString(Out, Name.Get().Id);
	}
	else
	{
		Out.Append(GLiterals.Super);
	}
}

void FIdsBase::AppendString(FUtf8Builder& Out, FScopeId Scope) const
{
	if (Scope.IsFlat())
	{
		AppendString(Out, Scope.AsFlat().Name);
	}
	else if (Scope)
	{
		FNestedScope Nested = Resolve(Scope.AsNested());
		AppendString(Out, Nested.Outer);
		Out.AppendChar('.');
		AppendString(Out, Nested.Inner.Name);
	}
}

void FIdsBase::AppendString(FUtf8Builder& Out, FTypenameId Typename) const
{
	if (Typename.IsConcrete())
	{
		AppendString(Out, Typename.AsConcrete().Id);
	}
	else
	{
		FParametricTypeView ParametricType = Resolve(Typename.AsParametric());
		TConstArrayView<FType> Parameters = ParametricType.GetParameters();

		if (ParametricType.Name)
		{
			AppendString(Out, ParametricType.Name.Get().Id);
		}

		Out.AppendChar(ParametricType.Name ? '<' : '[');
		for (FType Parameter : Parameters.LeftChop(1))
		{
			AppendString(Out, Parameter);
			Out.AppendChar(',');
		}
		if (Parameters.Num() > 0)
		{
			AppendString(Out, Parameters.Last());
		}
		Out.AppendChar(ParametricType.Name ? '>' : ']');
	}
}

void FIdsBase::AppendString(FUtf8Builder& Out, FType Type) const
{
	if (Type.Scope)
	{
		AppendString(Out, Type.Scope);
		Out.AppendChar('.');
	}
	AppendString(Out, Type.Name);
}

void FIds::AppendString(FUtf8Builder& Out, FEnumId Name) const
{
	AppendString(Out, Resolve(Name));
}

void FIds::AppendString(FUtf8Builder& Out, FStructId Name) const
{
	AppendString(Out, Resolve(Name));
}

void FBatchIds::AppendString(FUtf8Builder& Out, FEnumSchemaId Name) const
{
	AppendString(Out, Resolve(Name));
}

void FBatchIds::AppendString(FUtf8Builder& Out, FStructSchemaId Name) const
{
	AppendString(Out, Resolve(Name));
}

////////////////////////////////////////////////////////////////////////////////

FString FDebugIds::Print(FNameId Name) const
{
	TUtf8StringBuilder<128> Out;
	if (Name.Idx < Ids.NumNames())
	{
		Ids.AppendString(Out, Name);
	}
	else
	{
		Out << GLiterals.Oob;
	}
	return FString(FStringView(StringCast<TCHAR>(Out.GetData(), Out.Len())));
}

FString FDebugIds::Print(FMemberId Name) const
{
	TUtf8StringBuilder<128> Out;
	if (Name.Id.Idx < Ids.NumNames())
	{
		Ids.AppendString(Out, Name.Id);
	}
	else
	{
		Out << GLiterals.Oob;
	}
	return FString(FStringView(StringCast<TCHAR>(Out.GetData(), Out.Len())));
}

FString FDebugIds::Print(FOptionalMemberId Name) const
{
	TUtf8StringBuilder<128> Out;
	if (!Name || Name.Get().Id.Idx < Ids.NumNames())
	{
		Ids.AppendString(Out, Name);
	}
	else
	{
		Out << GLiterals.Oob;
	}
	return FString(FStringView(StringCast<TCHAR>(Out.GetData(), Out.Len())));
}

static const bool IsValidScope(FScopeId Scope, const FIds& Ids)
{
	if (Scope.IsFlat())
	{
		return Scope.AsFlat().Name.Idx < Ids.NumNames();
	}
	else if (Scope)
	{
		return Scope.AsNested().Idx < Ids.NumNestedScopes();
	}
	return !Scope; // Unscoped
}

FString FDebugIds::Print(FScopeId Scope) const
{
	TUtf8StringBuilder<128> Out;
	if (IsValidScope(Scope, Ids))
	{
		Ids.AppendString(Out, Scope);
	}
	else
	{
		Out << GLiterals.Oob;
	}
	return FString(FStringView(StringCast<TCHAR>(Out.GetData(), Out.Len())));
}

static const bool IsValidTypename(FTypenameId Typename, const FIds& Ids)
{
	if (Typename.IsConcrete())
	{
		return Typename.AsConcrete().Id.Idx < Ids.NumNames();
	}
	return Typename.AsParametric().Idx < Ids.NumNames();
}

FString FDebugIds::Print(FTypenameId Typename) const
{
	TUtf8StringBuilder<128> Out;
	if (IsValidTypename(Typename, Ids))
	{
		Ids.AppendString(Out, Typename);
	}
	else
	{
		Out << GLiterals.Oob;
	}
	return FString(FStringView(StringCast<TCHAR>(Out.GetData(), Out.Len())));
}

FString FDebugIds::Print(FConcreteTypenameId Typename) const
{
	return Print(FTypenameId(Typename));
}

FString FDebugIds::Print(FParametricTypeId Typename) const
{
	return Print(FTypenameId(Typename));
}

FString FDebugIds::Print(FType Type) const
{
	TUtf8StringBuilder<128> Out;
	if (IsValidScope(Type.Scope, Ids) && IsValidTypename(Type.Name, Ids))
	{
		Ids.AppendString(Out, Type);
	}
	else
	{
		Out << GLiterals.Oob;
	}
	return FString(FStringView(StringCast<TCHAR>(Out.GetData(), Out.Len())));
}

FString FDebugIds::Print(FEnumId Name) const
{
	TUtf8StringBuilder<128> Out;
	if (Name.Idx < Ids.NumEnums())
	{
		Ids.AppendString(Out, Name);
	}
	else
	{
		Out << GLiterals.Oob;
	}
	return FString(FStringView(StringCast<TCHAR>(Out.GetData(), Out.Len())));
}

FString FDebugIds::Print(FStructId Name) const
{
	TUtf8StringBuilder<128> Out;
	if (Name.Idx < Ids.NumStructs())
	{
		Ids.AppendString(Out, Name);
	}
	else
	{
		Out << GLiterals.Oob;
	}
	return FString(FStringView(StringCast<TCHAR>(Out.GetData(), Out.Len())));
}

FString FDebugIds::Print(const FMemberSchema& Schema) const
{
	TUtf8StringBuilder<128> Out;
	PrintMemberSchema(Out, Ids, Schema);
	return FString(FStringView(StringCast<TCHAR>(Out.GetData(), Out.Len())));
}

///////////////////////////////////////////////////////////////////////////////

void PrintDiff(FUtf8Builder& Out, const FIds& Ids, const FDiffPath& Diff)
{
	check(Diff.Num());

	for (FDiffNode Node : Diff)
	{
		Ids.AppendString(Out, Node.Name);
		Out << '.';
	}
	Out.RemoveSuffix(1);
	Out << ' ';
	Out << '(';
	for (FDiffNode Node : Diff)
	{
		if (Node.Type.IsStruct())
		{
			Ids.AppendString(Out, Ids.Resolve(Node.Meta.Struct).Name);
		}
		else if (Node.Type.IsRange())
		{
			Ids.AppendString(Out, FTypenameId(Node.Meta.Range.GetBindName()));
		}
		else if (FOptionalEnumId Enum = Node.Meta.Leaf)
		{
			Ids.AppendString(Out, Ids.Resolve(Enum.Get()).Name);
		}
		else
		{
			Out << ToString(ToLeafType(Node.Type.AsLeaf()));
		}
		Out << ' ';
	}
	Out.RemoveSuffix(1);
	Out << ')';
}

void PrintDiffs(FUtf8Builder& Out, const FIds& Ids, TConstArrayView<FDiffPath> Diffs)
{
	check(Diffs.Num());

	for (const FDiffPath& Diff : Diffs)
	{
		PrintDiff(Out, Ids, Diff);
	}
}

void PrintDiff(FUtf8Builder& Out, const FBatchIds& Ids, const FReadDiffPath& Diff)
{
	check(Diff.Num());

	bool bWasName = false;
	// print type name for the outermost struct
	if (Diff.Last().Struct)
	{
		Ids.AppendString(Out, Ids.Resolve(Diff.Last().Struct.Get()).Name);
		bWasName = true;
	}
	// print struct members path with range indices
	for (FReadDiffNode Node : ReverseIterate(Diff))
	{
		if (Node.Name || Node.RangeIdx == ~uint64(0))
		{
			if (bWasName)
			{
				Out << '.';
			}
			if (!Node.Name)
			{
				Out << GLiterals.Super;
			}
			else
			{
				Ids.AppendString(Out, Node.Name);
			}
			bWasName = true;
		}
		else if (Node.Type.IsRange())
		{
			Out << "[" << Node.RangeIdx << "]";
			bWasName = false;
		}
	}
}

} // namespace PlainProps
