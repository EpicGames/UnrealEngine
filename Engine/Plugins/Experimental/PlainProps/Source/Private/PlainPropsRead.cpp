// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsRead.h"
#include "PlainPropsInternalRead.h"
#include "PlainPropsVisualize.h"
#include "Misc/Optional.h"

#include <atomic>
#include <type_traits>

namespace PlainProps
{

static_assert(sizeof(FMemberType) == 1);

void FSchemaBatch::ValidateBounds(uint64 NumBytes) const
{
	static constexpr uint32 Alignment = FMath::Max(alignof(FStructSchema), alignof(FEnumSchema));
	check(IsAligned(this, Alignment));
	check(sizeof(FSchemaBatch) + NumSchemas * sizeof(uint32) <= NestedScopesOffset);
	check(NestedScopesOffset + NumNestedScopes * sizeof(FNestedScope) + NumParametricTypes * sizeof(FParametricType) <= NumBytes);

	for (uint32 SchemaOffset : GetSchemaOffsets())
	{
		check(SchemaOffset < NestedScopesOffset);
		check(IsAligned(SchemaOffset, Alignment));
	}
		
	uint32 NumParameters = 0;
	for (FParametricType ParametricType : GetParametricTypes())
	{
		check(ParametricType.Parameters.Idx == NumParameters);
		check(ParametricType.Parameters.NumParameters > 0);
		NumParameters += ParametricType.Parameters.NumParameters;
	}

	const void* ExpectedEnd = GetFirstParameter() + NumParameters;
	const void* ActualEnd = reinterpret_cast<const uint8*>(this) + NumBytes;
	check(ExpectedEnd == ActualEnd);
}


class FReadSchemaRegistry
{
	static constexpr uint32 Capacity = 1 << 16;
	std::atomic<const FSchemaBatch*> Slots[Capacity]{};
	std::atomic<uint32> Counter{};

public:
	FReadSchemaRegistry()
	{
		DbgVis::AssignReadSchemasDebuggingState(reinterpret_cast<DbgVis::FSchemaBatch**>(Slots));
	}

	FSchemaBatchId Mount(const FSchemaBatch* Batch)
	{
		const uint32 Count = Counter.fetch_add(1, std::memory_order_relaxed);
		for (uint32 BaseIdx = 0; BaseIdx < FReadSchemaRegistry::Capacity; ++BaseIdx)
		{
			uint32 Idx = (Count + BaseIdx) % FReadSchemaRegistry::Capacity;
			const FSchemaBatch* Null = nullptr;
			if (Slots[Idx].load(std::memory_order_relaxed) == nullptr &&
				Slots[Idx].compare_exchange_strong(Null, Batch, std::memory_order_acq_rel))
			{
				FSchemaBatchId Out;
				Out.Idx = static_cast<uint16>(Idx);
				return Out;
			}
		}

		checkf(false, TEXT("Exceeded fixed limit of 65536 simulatenous read batches"));
		while (true);
	}

	const FSchemaBatch* Unmount(FSchemaBatchId Id)
	{
		const FSchemaBatch* Batch = &Get(Id);
		verify(Slots[Id.Idx].compare_exchange_strong(Batch, nullptr, std::memory_order_acq_rel));
		return Batch;
	}

	const FSchemaBatch& Get(FSchemaBatchId Id) const
	{
		const FSchemaBatch* Batch = Slots[Id.Idx].load(std::memory_order_relaxed);
		check(Batch);
		return *Batch;
	}
};
static FReadSchemaRegistry GReadSchemas;

const FSchemaBatch*	ValidateSchemas(FMemoryView Schemas)
{
	const FSchemaBatch* Batch = reinterpret_cast<const FSchemaBatch*>(Schemas.GetData());
	Batch->ValidateBounds(Schemas.GetSize());
	return Batch;
}

FSchemaBatchId MountReadSchemas(const FSchemaBatch* Batch)
{
	return GReadSchemas.Mount(Batch);
}

const FSchemaBatch* UnmountReadSchemas(FSchemaBatchId Id)
{
	return GReadSchemas.Unmount(Id);
}

uint32 NumStructSchemas(FSchemaBatchId Batch)
{
	return GReadSchemas.Get(Batch).NumStructSchemas;
}

const FSchemaBatch& GetReadSchemas(FSchemaBatchId Batch)
{
	return GReadSchemas.Get(Batch);
}

const FStructSchema& ResolveStructSchema(FSchemaBatchId Batch, FStructSchemaId Schema)
{
	return ResolveStructSchema(GReadSchemas.Get(Batch), Schema);
}

const FEnumSchema& ResolveEnumSchema(FSchemaBatchId Batch, FEnumSchemaId Schema)
{
	return ResolveEnumSchema(GReadSchemas.Get(Batch), Schema);
}

FNestedScope ResolveUntranslatedNestedScope(FSchemaBatchId Batch, FNestedScopeId Id)
{
	return ResolveNestedScope(GReadSchemas.Get(Batch), Id);
}

FParametricTypeView ResolveUntranslatedParametricType(FSchemaBatchId Batch, FParametricTypeId Id)
{
	return ResolveParametricType(GReadSchemas.Get(Batch), Id);
}

//////////////////////////////////////////////////////////////////////////

FLeafRangeView FRangeView::AsLeaves() const
{
	FUnpackedLeafType Leaf = Schema.ItemType.AsLeaf();
	FOptionalEnumSchemaId EnumId = static_cast<FOptionalEnumSchemaId>(Schema.InnermostSchema);
	const uint8* Data = static_cast<const uint8*>(Values.GetData());
	return {Leaf, Schema.Batch, EnumId, NumItems, Data };
}

FStructRangeView FRangeView::AsStructs() const
{
	check(IsStructRange());
	FStructSchemaId Id = static_cast<FStructSchemaId>(Schema.InnermostSchema.Get());
	return {NumItems, Values, {Id, Schema.Batch}};
}

FNestedRangeView FRangeView::AsRanges() const
{
	check(IsNestedRange());
	return {NumItems, Values, Schema};
}

//////////////////////////////////////////////////////////////////////////

FRangeView FNestedRangeIterator::operator*() const 
{
	FByteReader PeekBytes = ByteIt;
	FBitCacheReader PeekBits = BitIt;
	
	FRangeView Out;
	Out.Schema = { Schema.NestedItemTypes[0], Schema.Batch, Schema.InnermostSchema, /* Only valid for nested ranges */ Schema.NestedItemTypes + 1 };
	Out.NumItems = GrabRangeNum(Schema.ItemType.AsRange().MaxSize, /* in-out */ PeekBytes,  /* in-out */ PeekBits);
	Out.Values = GrabRangeValues(Out.NumItems, Schema.NestedItemTypes[0], /* in-out */ PeekBytes);

	return Out;
}

void FNestedRangeIterator::operator++()
{
	uint64 Num = GrabRangeNum(Schema.ItemType.AsRange().MaxSize, /* in-out */ ByteIt, /* in-out */ BitIt);
	(void)GrabRangeValues(Num, Schema.NestedItemTypes[0], /* in-out */ ByteIt);
}

//////////////////////////////////////////////////////////////////////////

const FMemberType*	FMemberReader::GetMemberTypes() const	{ return FStructSchema::GetMemberTypes(Footer); }
const FMemberType*	FMemberReader::GetRangeTypes() const	{ return FStructSchema::GetRangeTypes(Footer, NumMembers); }
const FSchemaId*	FMemberReader::GetInnerSchemas() const	{ return FStructSchema::GetInnerSchemas(Footer, NumMembers, NumRangeTypes, NumMembers - UsesSuper); }
const FMemberId*	FMemberReader::GetMemberNames() const	{ return FStructSchema::GetMemberNames(Footer, NumMembers, NumRangeTypes); }

FMemberReader::FMemberReader(const FStructSchema& Schema, FByteReader Values, FSchemaBatchId InBatch)
: Footer(Schema.Footer)
, Batch(InBatch)
, IsSparse(!Schema.IsDense)
, UsesSuper(PlainProps::UsesSuper(Schema.Inheritance))
, NumMembers(Schema.NumMembers)
, NumRangeTypes(Schema.NumRangeTypes)
, InnerSchemaIdx(SkipDeclaredSuperSchema(Schema.Inheritance))
, ValueIt(Values)
#if DO_CHECK
, NumInnerSchemas(Schema.NumInnerSchemas)
#endif
{
#if DO_CHECK
	check(InnerSchemaIdx <= NumInnerSchemas);
#endif
	checkf(NumRangeTypes != 0xFFFFu, TEXT("GrabRangeTypes() doesn't check for wrap-around"));

	if (IsSparse)
	{
		SkipMissingSparseMembers();
	}
}

FOptionalMemberId FMemberReader::PeekName() const
{
	int32 MemberNameIdx = MemberIdx - UsesSuper;
	return MemberNameIdx >= 0 ? ToOptional(GetMemberNames()[MemberNameIdx]) : NoId;
}

FMemberId FMemberReader::PeekNameUnchecked() const
{
	int32 MemberNameIdx = MemberIdx - UsesSuper;
	return GetMemberNames()[MemberNameIdx];
}

EMemberKind FMemberReader::PeekKind() const
{
	return PeekType().GetKind();
}

FMemberType	FMemberReader::PeekType() const
{
	check(HasMore());
	return GetMemberTypes()[MemberIdx];
}

void FMemberReader::AdvanceToNextMember()
{
	++MemberIdx;
	if (IsSparse)
	{
		SkipMissingSparseMembers();
	}
}

void FMemberReader::SkipMissingSparseMembers()
{
	// Change code in LoadMembers() too
	while (MemberIdx < NumMembers && GrabBit())
	{
		FMemberType Type = GetMemberTypes()[MemberIdx];
		FMemberType InnermostType = Type.IsRange() ? /* advances RangeTypeIdx */ GrabRangeTypes().Last() : Type;
		SkipSchema(InnermostType);
		++MemberIdx;
	}
}

inline void FMemberReader::SkipSchema(FMemberType InnermostType)
{
	if (InnermostType.IsStruct())
	{
		InnerSchemaIdx += !InnermostType.AsStruct().IsDynamic;
	}
	else
	{
		InnerSchemaIdx += InnermostType.AsLeaf().Type == ELeafType::Enum;
	}

#if DO_CHECK
	check(InnerSchemaIdx <= NumInnerSchemas);
#endif
}

FSchemaId FMemberReader::GrabInnerSchema()
{
#if DO_CHECK
	check(InnerSchemaIdx < NumInnerSchemas);
#endif
	uint32 Idx = InnerSchemaIdx++;
	return GetInnerSchemas()[Idx];
}

FStructSchemaId FMemberReader::GrabStructSchema(FStructType Type)
{
	return Type.IsDynamic	? FStructSchemaId { ValueIt.Grab<uint32>() }
							: static_cast<FStructSchemaId&&>(GrabInnerSchema());
}

FOptionalSchemaId FMemberReader::GrabRangeSchema(FMemberType InnermostType)
{
	if (InnermostType.IsStruct())
	{
		return FOptionalSchemaId(GrabStructSchema(InnermostType.AsStruct()));
	}

	ELeafType Type = InnermostType.AsLeaf().Type;
	return Type == ELeafType::Enum ? FOptionalSchemaId(GrabEnumSchema()) : NoId;
}

FLeafView FMemberReader::GrabLeaf()
{
	FLeafView Out = { PeekType().AsLeaf(), Batch };
	Out.Enum = Out.Leaf.Type == ELeafType::Enum ? GrabEnumSchema() : FEnumSchemaId{};
	
	if (Out.Leaf.Type == ELeafType::Bool)
	{
		Out.Value.bValue = GrabBit();
	}
	else
	{
		Out.Value.Ptr = ValueIt.GrabBytes(SizeOf(Out.Leaf.Width));
	}

	AdvanceToNextMember();

	return Out;
}

FStructView FMemberReader::GrabStruct()
{
	check(HasMore());
	FStructType Type = PeekType().AsStruct();
	FStructSchemaId Struct = Type.IsDynamic ? FStructSchemaId{ ValueIt.Grab<uint32>() } : GrabStructSchema(Type);
	FMemoryView Values = ValueIt.GrabSkippableSlice();

	AdvanceToNextMember();

	return { { Struct, Batch }, FByteReader(Values) };
}

TConstArrayView<FMemberType> FMemberReader::GrabRangeTypes()
{
	return GrabInnerRangeTypes(MakeArrayView(GetRangeTypes(), NumRangeTypes), /* in-out */ RangeTypeIdx);
}

FRangeView FMemberReader::GrabRange()
{
	check(HasMore());

	FMemberTypeRange RangeTypes = GrabRangeTypes();
	FRangeView Out;
	Out.Schema.Batch = Batch;
	Out.Schema.InnermostSchema = GrabRangeSchema(RangeTypes.Last());									// #1
	Out.Schema.ItemType = RangeTypes[0];
	Out.Schema.NestedItemTypes = RangeTypes.Num() > 1 ? &RangeTypes[1] : nullptr;
	Out.NumItems = GrabRangeNum(PeekType().AsRange().MaxSize, /* in-out */ ValueIt, /* in-out */ Bits); // #2
	Out.Values = GrabRangeValues(Out.NumItems, Out.Schema.ItemType, /* in-out */ ValueIt);				// #3
	
	AdvanceToNextMember();

	return Out;
}

//FAnyMemberView FMemberReader::GrabAnyMember()
//{
//	FMemberValue Value = GetValueAndAdvance(Type, /* in-out */ ValueIt);
//	
//	return {Type, Id, BatchId, Value};
//}

void FMemberReader::SkipMember()
{
	switch (PeekKind())
	{
		case EMemberKind::Leaf:		(void)GrabLeaf();		break;
		case EMemberKind::Struct:	(void)GrabStruct();		break;
		case EMemberKind::Range:	(void)GrabRange();		break;
	}
}

void FMemberReader::GrabLeaves(void* Out, uint32 Num, SIZE_T Size)
{
	checkSlow(Num);
	check(MemberIdx + Num <= NumMembers);
	const FMemberType* Types = GetMemberTypes() + MemberIdx;
	FLeafType Leaf = Types[0].AsLeaf();
	checkSlow(Leaf.Type != ELeafType::Enum);
	checkSlow(SizeOf(Leaf.Width) == Size);
	for (FMemberType Type : MakeArrayView(Types + 1, Num - 1))
	{
		check(Type == Types[0]);
	}

	
	if (IsSparse)
	{
		uint8* OutIt = static_cast<uint8*>(Out);
		for (uint8* OutLast = OutIt + Num * Size - Size; OutIt != OutLast; OutIt += Size)
		{
			FMemory::Memcpy(OutIt, ValueIt.GrabBytes(Size), Size);
			bool bSkip = GrabBit();
			check(!bSkip);
		}

		FMemory::Memcpy(OutIt, ValueIt.GrabBytes(Size), Size);
		
		MemberIdx += Num;
		SkipMissingSparseMembers();
	}
	else
	{
		const SIZE_T NumBytes = Num * Size;
		FMemory::Memcpy(Out, ValueIt.GrabBytes(NumBytes), NumBytes);
		MemberIdx += Num;
	}
}

//////////////////////////////////////////////////////////////////////////

static TOptional<FStructView> TryGrabSuper(FMemberReader& Members)
{
	if (Members.HasMore() && IsSuper(Members.PeekType()))
	{
		return Members.GrabStruct();
	}

	return NullOpt;
}

FFlatMemberReader::FReader::FReader(FStructView Struct)
: FMemberReader(Struct)
, Owner(Struct.Schema.Resolve().Type)
{}


FFlatMemberReader::FFlatMemberReader(FStructView Struct)
{
	Lineage.Emplace(Struct);
	while (TOptional<FStructView> Super = TryGrabSuper(Lineage.Last()))
	{
		Lineage.Emplace(*Super);
	}
	It = &Lineage.Last();
}

//////////////////////////////////////////////////////////////////////////

FBatchIds::FBatchIds(FSchemaBatchId Batch) : Schemas(GReadSchemas.Get(Batch)), BatchId(Batch) {}
uint32 FBatchIds::NumEnums() const { return Schemas.NumSchemas - Schemas.NumStructSchemas; }
uint32 FBatchIds::NumStructs() const { return Schemas.NumStructSchemas; }
FType FBatchIds::Resolve(FEnumSchemaId Id) const { return ResolveEnumSchema(Schemas, Id).Type; }
FType FBatchIds::Resolve(FStructSchemaId Id) const	{ return ResolveStructSchema(Schemas, Id).Type; }

uint32 FStableBatchIds::NumNestedScopes() const { return Schemas.NumNestedScopes; }
uint32 FStableBatchIds::NumParametricTypes() const { return Schemas.NumParametricTypes; }
FNestedScope FStableBatchIds::Resolve(FNestedScopeId Id) const { return ResolveNestedScope(Schemas, Id); }
FParametricTypeView FStableBatchIds::Resolve(FParametricTypeId Id) const { return ResolveParametricType(Schemas, Id); }

} // namespace PlainProps
