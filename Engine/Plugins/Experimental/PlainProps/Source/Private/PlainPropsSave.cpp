// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsSave.h"
#include "PlainPropsBind.h"
#include "PlainPropsInternalBind.h"
#include "PlainPropsInternalBuild.h"
#include "PlainPropsInternalDiff.h"
#include "PlainPropsInternalFormat.h"
#include "PlainPropsSaveMember.h"
#include "Algo/Find.h"
#include <type_traits>

namespace PlainProps
{

FBaseline::FBaseline(FDefaultLink* Chain UE_LIFETIMEBOUND, int32 Num)
: Ptr(reinterpret_cast<uint64>(Chain) | TagBit)
{
	check(Num >= 2);
	for (FDefaultLink& Link : MakeArrayView(Chain, Num))
	{
		checkSlow(!(Link.Ptr & TagBit));
	}

	// Tag last link
	Chain[Num - 1].Ptr |= TagBit;
}

static uint64 GetBit(uint8 Byte, uint8 BitIdx)
{
	return (Byte >> BitIdx) & 1;
}

static uint64 SaveLeaf(const uint8* Member, FUnpackedLeafBindType Leaf)
{
#if DO_CHECK
	if (Leaf.Type == ELeafBindType::Float)
	{
		switch (Leaf.Width)
		{
			case ELeafWidth::B32: return ValueCast(*reinterpret_cast<const float*>(Member));
			case ELeafWidth::B64: return ValueCast(*reinterpret_cast<const double*>(Member));
			default: check(false); return 0;
		}

	}
#endif

	if (Leaf.Type == ELeafBindType::BitfieldBool)
	{
		return GetBit(*Member, Leaf.BitfieldIdx);
	}
	else
	{
		// Undefined behavior. If problematic, use memcpy or switch(Leaf.Type) and ValueCast(reinterpret_cast<...>(Member))  
		switch (Leaf.Width)
		{
			case ELeafWidth::B8:	return *Member;
			case ELeafWidth::B16:	return *reinterpret_cast<const uint16*>(Member);
			case ELeafWidth::B32:	return *reinterpret_cast<const uint32*>(Member);
			case ELeafWidth::B64:	return *reinterpret_cast<const uint64*>(Member);
		}
	}
	check(false);
	return uint64(0);
}

struct FLeafRangeSaver
{
	FBuiltRange* Out;
	uint8* OutIt;
	uint8* OutEnd;

	FLeafRangeSaver(FScratchAllocator& Scratch, uint64 Num, SIZE_T LeafSize)
	: Out(FBuiltRange::Create(Scratch, Num, LeafSize))
	, OutIt(Out->Data)
	, OutEnd(OutIt + Num * LeafSize)
	{}

	void Append(FExistingItemSlice Slice, uint32 Stride, SIZE_T LeafSize, const FSaveContext&)
	{
		check(OutIt + Slice.Num * LeafSize <= OutEnd);
		FMemory::Memcpy(OutIt, Slice.Data, Slice.Num * LeafSize);
		OutIt += Slice.Num * LeafSize;
	}

	FBuiltRange* Finish()
	{
		check(OutIt == OutEnd);
		return Out;
	}
};

template<SIZE_T LeafSize>
struct TStridingLeafRangeSaver : FLeafRangeSaver
{
	inline TStridingLeafRangeSaver(FScratchAllocator& Scratch, uint64 Num, SIZE_T) : FLeafRangeSaver(Scratch, Num, LeafSize) {}

	inline void Append(FExistingItemSlice Slice, uint32 Stride, SIZE_T, const FSaveContext&)
	{
		uint8* Dst = OutIt;
		for (const uint8* Src = static_cast<const uint8*>(Slice.Data), *SrcEnd = Src + Slice.Num * Stride; Src != SrcEnd; Src += Stride, Dst += LeafSize)
		{
			FMemory::Memcpy(Dst, Src, LeafSize);
		}
		OutIt = Dst;
	}

};

//////////////////////////////////////////////////////////////////////////

template<typename BuiltItemType, typename ItemSchemaType>
struct TNonLeafRangeSaver
{
	FBuiltRange* Out;
	BuiltItemType* It;

	TNonLeafRangeSaver(FScratchAllocator& Scratch, uint64 Num, ItemSchemaType)
	: Out(FBuiltRange::Create(Scratch, Num, sizeof(BuiltItemType)))
	, It(reinterpret_cast<BuiltItemType*>(Out->Data))
	{}

	void Append(FExistingItemSlice Slice, uint32 Stride, ItemSchemaType Schema, const FSaveContext& OuterCtx)
	{
		check(It + Slice.Num <= reinterpret_cast<BuiltItemType*>(Out->Data) + Out->Num);
		for (uint64 Idx = 0; Idx < Slice.Num; ++Idx)
		{
			*It++ = SaveRangeItem(Slice.At(Idx, Stride), Schema, OuterCtx);
		}
	}

	[[nodiscard]] FBuiltRange* Finish()
	{
		check(It == reinterpret_cast<BuiltItemType*>(Out->Data) + Out->Num);
		return Out;
	}
};

struct FDefaultStruct
{
	FBindId			Id;
	const void*		Struct;
};

using FInternalNestedRangeSaver = TNonLeafRangeSaver<FBuiltRange*, FRangeMemberBinding>;
using FInternalStructRangeSaver = TNonLeafRangeSaver<FBuiltStruct*, FBindId>;
using FInternalStructRangeDeltaSaver = TNonLeafRangeSaver<FBuiltStruct*, FDefaultStruct>;

//////////////////////////////////////////////////////////////////////////


template<class SaverType, typename InnerContextType>
[[nodiscard]] inline FBuiltRange* SaveRangeItems(FSaveRangeContext& ReadCtx, const IItemRangeBinding& Binding, const FSaveContext& OuterCtx, InnerContextType InnerCtx)
{
	const uint64 NumTotal = ReadCtx.Items.NumTotal;
	SaverType Saver(OuterCtx.Scratch, NumTotal, InnerCtx);
	while (true)
	{
		check(ReadCtx.Items.Slice.Num > 0);
		Saver.Append(ReadCtx.Items.Slice, ReadCtx.Items.Stride, InnerCtx, OuterCtx);
	
		ReadCtx.Request.NumRead += ReadCtx.Items.Slice.Num;
		if (ReadCtx.Request.NumRead >= NumTotal)
		{
			check(ReadCtx.Request.NumRead == NumTotal);	
			return Saver.Finish();
		}

		Binding.ReadItems(ReadCtx);	
	}
}

template<class SaverType, typename InnerContextType>
[[nodiscard]] FBuiltRange* SaveNonLeafRange(const void* Range, const IItemRangeBinding& Binding, const FSaveContext& OuterCtx, InnerContextType InnerCtx)
{
	FSaveRangeContext ReadCtx = { { Range } };
	Binding.ReadItems(ReadCtx);

	if (ReadCtx.Items.NumTotal)
	{
		return SaveRangeItems<SaverType>(ReadCtx, Binding, OuterCtx, InnerCtx);
	}
	
	return nullptr;
}

[[nodiscard]] static FBuiltRange* SaveLeafRange(const void* Range, const IItemRangeBinding& Binding, const FSaveContext& OuterCtx, ELeafWidth Width)
{
	SIZE_T LeafSize = SizeOf(Width);
	FSaveRangeContext ReadCtx = { { Range } };
	Binding.ReadItems(ReadCtx);

	if (const uint64 NumTotal = ReadCtx.Items.NumTotal)
	{
		if (ReadCtx.Items.Stride == LeafSize)
		{
			return SaveRangeItems<FLeafRangeSaver>(ReadCtx, Binding, OuterCtx, LeafSize);
		}
		else switch (Width)
		{
			case ELeafWidth::B8:	return SaveRangeItems<TStridingLeafRangeSaver<1>>(ReadCtx, Binding, OuterCtx, LeafSize);
			case ELeafWidth::B16:	return SaveRangeItems<TStridingLeafRangeSaver<2>>(ReadCtx, Binding, OuterCtx, LeafSize);
			case ELeafWidth::B32:	return SaveRangeItems<TStridingLeafRangeSaver<4>>(ReadCtx, Binding, OuterCtx, LeafSize);
			case ELeafWidth::B64:	return SaveRangeItems<TStridingLeafRangeSaver<8>>(ReadCtx, Binding, OuterCtx, LeafSize);
		}
	}

	return nullptr;
}

FBuiltRange* SaveLeafRange(const void* Range, const ILeafRangeBinding& Binding, FUnpackedLeafType Leaf, const FSaveContext& Ctx)
{
	FLeafRangeAllocator Allocator(Ctx.Scratch, Leaf);
	Binding.SaveLeaves(Range, Allocator);
	return Allocator.GetAllocatedRange();
}

inline const FStructDeclaration& GetDeclaration(const FSaveContext& Ctx, FBindId BindId)
{
	if (const FStructDeclaration* Out = Ctx.Customs.FindDeclaration(BindId))
	{
		return *Out;
	}

	return Ctx.Schemas.GetDeclaration(BindId);
}

[[nodiscard]] static FBuiltRange* SaveStructRange(const void* Range, const IItemRangeBinding& ItemBinding, const FSaveContext& Ctx, FBindId Id)
{
	if (Ctx.Defaults && GetDeclaration(Ctx, Id).Occupancy == EMemberPresence::AllowSparse)
	{
		FDefaultStruct Default = { Id, Ctx.Defaults->Get(Id) };
		return SaveNonLeafRange<FInternalStructRangeDeltaSaver>(Range, ItemBinding, Ctx, Default);
	}
	else
	{
		return SaveNonLeafRange<FInternalStructRangeSaver>(Range, ItemBinding, Ctx, Id);
	}
}

[[nodiscard]] FBuiltRange* SaveRange(const void* Range, FRangeMemberBinding Member, const FSaveContext& Ctx)
{
	FRangeBinding Binding = Member.RangeBindings[0];
	FMemberBindType InnerType = Member.InnerTypes[0];

	if (Binding.IsLeafBinding())
	{
		return SaveLeafRange(Range, Binding.AsLeafBinding(), UnpackNonBitfield(InnerType.AsLeaf()), Ctx);
	}

	const IItemRangeBinding& ItemBinding = Binding.AsItemBinding();
	switch (InnerType.GetKind())
	{
	case EMemberKind::Leaf:		return SaveLeafRange(Range, ItemBinding, Ctx, GetItemWidth(InnerType.AsLeaf()));
	case EMemberKind::Range:	return SaveNonLeafRange<FInternalNestedRangeSaver>(Range, ItemBinding, Ctx, GetInnerRange(Member));
	case EMemberKind::Struct:	return SaveStructRange(Range, ItemBinding, Ctx, Member.InnermostId.Get().AsStructBindId());
	}

	check(false);
	return nullptr;
}

//////////////////////////////////////////////////////////////////////////

[[nodiscard]] static FBuiltRange* SaveRangeItem(const uint8* Range, FRangeMemberBinding Member, const FSaveContext& Ctx)
{ 
	return SaveRange(Range, Member, Ctx);
}

[[nodiscard]] static FBuiltStruct* SaveRangeItem(const uint8* Struct, FBindId Id, const FSaveContext& Ctx)
{
	return SaveStruct(Struct, Id, Ctx);
}

[[nodiscard]] static FBuiltStruct* SaveRangeItem(const uint8* Struct, FDefaultStruct Default, const FSaveContext& Ctx)
{
	return SaveStructDelta(Struct, Default.Struct, Default.Id, Ctx);
}

[[nodiscard]] static FMemberType* CreateInnerRangeTypes(FScratchAllocator& Scratch, uint32 NumInnerTypes, const FMemberBindType* InnerTypes)
{
	if (NumInnerTypes <= 1)
	{
		return nullptr;
	}

	FMemberType* Out = Scratch.AllocateArray<FMemberType>(NumInnerTypes);
	for (uint32 Idx = 0; Idx < NumInnerTypes; ++Idx)
	{
		Out[Idx] = ToMemberType(InnerTypes[Idx]);
	}
	return Out;
}

[[nodiscard]] FMemberSchema CreateRangeSchema(FScratchAllocator& Scratch, FRangeMemberBinding Member)
{
	FMemberType* InnerRangeTypes = CreateInnerRangeTypes(Scratch, Member.NumRanges, Member.InnerTypes);
	return { FMemberType(Member.RangeBindings[0].GetSizeType()), ToMemberType(Member.InnerTypes[0]), Member.NumRanges, Member.InnermostId, InnerRangeTypes };
}

void SaveMember(FMemberBuilder& Out, const void* Struct, FMemberId Name, const FSaveContext& Ctx, FLeafMemberBinding Member)
{
	FUnpackedLeafType Type = ToUnpackedLeafType(Member.Leaf);
	Out.AddLeaf(Name, Type, Member.Enum, SaveLeaf(At(Struct, Member.Offset), Member.Leaf));
}

void SaveMember(FMemberBuilder& Out, const void* Struct, FMemberId Name, const FSaveContext& Ctx, FRangeMemberBinding Member)
{
	Out.AddRange(Name, { CreateRangeSchema(Ctx.Scratch, Member), SaveRange(At(Struct, Member.Offset), Member, Ctx) });
}

void SaveMember(FMemberBuilder& Out, const void* Struct, FMemberId Name, const FSaveContext& Ctx, FStructMemberBinding Member)
{
	Out.AddStruct(Name, Member.Id, SaveStruct(At(Struct, Member.Offset), Member.Id, Ctx));
}

inline void SaveNamedMembers(FMemberBuilder& Out, const void* Struct, FMemberVisitor It, TConstArrayView<FMemberId> Names, const FSaveContext& Ctx)
{
	for (FMemberId Name : Names)
	{
		switch (It.PeekKind())
		{
			case EMemberKind::Leaf:		SaveMember(Out, Struct, Name, Ctx, It.GrabLeaf());		break;
			case EMemberKind::Range:	SaveMember(Out, Struct, Name, Ctx, It.GrabRange());		break;
			case EMemberKind::Struct:	SaveMember(Out, Struct, Name, Ctx, It.GrabStruct());	break;
		}
	}
	checkSlow(!It.HasMore());
}

static const FStructDeclaration& SaveStruct(FMemberBuilder& Out, const void* Struct, FBindId Id, const FSaveContext& Ctx)
{
	const FStructDeclaration* Declaration = nullptr;
	if (ICustomBinding* Custom = Ctx.Customs.FindStructToSave(Id, /* out */ Declaration))
	{
		Custom->SaveCustom(Out, Struct, nullptr, Ctx);
	}
	else
	{
		const FSchemaBinding& Schema = Ctx.Schemas.GetStruct(Id, /* out */ Declaration);

		FMemberVisitor It(Schema);
		if (Declaration->Super)
		{
			const FStructDeclaration& SuperDecl = SaveStruct(Out, Struct, It.GrabSuper(), Ctx);
			Out.BuildSuperStruct(Ctx.Scratch, SuperDecl, Ctx.Schemas.GetDebug());
			checkSlow(SuperDecl.Id == ToOptionalStruct(Declaration->Super));
		}

		SaveNamedMembers(Out, Struct, It, Declaration->GetMemberOrder(), Ctx);
	}
	return *Declaration;
}

FBuiltStruct* SaveStruct(const void* Struct, FBindId Id, const FSaveContext& Ctx)
{
	FMemberBuilder Out;
	const FStructDeclaration& Declaration = SaveStruct(Out, Struct, Id, Ctx);
	return Out.BuildAndReset(Ctx.Scratch, Declaration, Ctx.Schemas.GetDebug());
}

////////////////////////////////////////////////////////////////////////////////////////////////

static bool DiffItem(const uint8* A, const uint8* B, const FSaveContext& Ctx, FRangeMemberBinding Member)
{
	FRangeBinding Binding = Member.RangeBindings[0];
	FMemberBindType InnerType = Member.InnerTypes[0];

	if (Binding.IsLeafBinding())
	{
		return Binding.AsLeafBinding().DiffLeaves(A, B);
	}

	const IItemRangeBinding& ItemBinding = Binding.AsItemBinding();
	switch (InnerType.GetKind())
	{
	case EMemberKind::Leaf:		return DiffItemRange(A, B, ItemBinding, Ctx, SizeOf(GetItemWidth(InnerType.AsLeaf())));
	case EMemberKind::Range:	return DiffItemRange(A, B, ItemBinding, Ctx, GetInnerRange(Member));
	case EMemberKind::Struct:	return DiffItemRange(A, B, ItemBinding, Ctx, Member.InnermostId.Get().AsStructBindId());
	}

	check(false);
	return false;
}

static bool DiffItem(const uint8* A, const uint8* B, const FSaveContext& Ctx, FBindId Id)
{
	if (const ICustomBinding* Custom = Ctx.Customs.FindStruct(Id))
	{
		return Custom->DiffCustom(A, B, Ctx);
	}
	
	bool bOut = false;
	for (FMemberVisitor It(Ctx.Schemas.GetStruct(Id)); It.HasMore() && !bOut; )
	{
		uint32 Offset = It.PeekOffset();
		const uint8* ItemA = A + Offset;
		const uint8* ItemB = B + Offset;

		switch (It.PeekKind())
		{
		case EMemberKind::Leaf:		bOut = !!DiffLeaf(ItemA, ItemB, It.GrabLeaf().Leaf);	break;
		case EMemberKind::Range:	bOut = DiffItem(ItemA, ItemB, Ctx, It.GrabRange());		break;
		case EMemberKind::Struct:	bOut = DiffItem(ItemA, ItemB, Ctx, It.GrabStruct().Id);	break;
		}
	}

	return bOut;
}

////////////////////////////////////////////////////////////////////////////////////////////////

static void SaveMemberDelta(FMemberBuilder& Out, const void* Struct, const void* Default, FMemberId Name, const FSaveContext& Ctx, FLeafMemberBinding Member)
{
	if (DiffLeaf(At(Struct, Member.Offset), At(Default, Member.Offset), Member.Leaf))
	{
		SaveMember(Out, Struct, Name, Ctx, Member);
	}
}

static void SaveMemberDelta(FMemberBuilder& Out, const void* Struct, const void* Default, FMemberId Name, const FSaveContext& Ctx, FRangeMemberBinding Member)
{
	const uint8* Range = At(Struct, Member.Offset);
	if (DiffItem(Range, At(Default, Member.Offset), Ctx, Member))
	{
		Out.AddRange(Name, { CreateRangeSchema(Ctx.Scratch, Member), SaveRange(Range, Member, Ctx) });
	}
}

static void SaveMemberDelta(FMemberBuilder& Out, const void* Struct, const void* Default, FMemberId Name, const FSaveContext& Ctx, FStructMemberBinding Member)
{
	if (FBuiltStruct* Delta = SaveStructDeltaIfDiff(At(Struct, Member.Offset), At(Default, Member.Offset), Member.Id, Ctx))
	{
		Out.AddStruct(Name, Member.Id, Delta);
	}
}

const FStructDeclaration& SaveStructDelta(FMemberBuilder& Out, const void* Struct, FBaseline Base, FBindId Id, const FSaveContext& Ctx);

static void SaveSchemaBoundStructDelta(FMemberBuilder& Out, const void* Struct, FBaseline Base, const FSchemaBinding& Schema, const FStructDeclaration& Declaration, const FSaveContext& Ctx)
{
	FMemberVisitor It(Schema);
	if (Declaration.Super)
	{
		FBindId SuperId = It.GrabSuper();
		// A schema bound struct may have a custom bound super struct
		check(LowerCast(SuperId) == Declaration.Super);
		const FStructDeclaration& SuperDecl = SaveStructDelta(Out, Struct, Base.Super(), SuperId, Ctx);
		Out.BuildSuperStruct(Ctx.Scratch, SuperDecl, Ctx.Schemas.GetDebug());
	}

	const void* Default = Base.Get();
	if (Declaration.Occupancy == EMemberPresence::AllowSparse && Default)
	{
		for (FMemberId Name : Declaration.GetMemberOrder())
		{
			switch (It.PeekKind())
			{
				case EMemberKind::Leaf:		SaveMemberDelta(Out, Struct, Default, Name, Ctx, It.GrabLeaf());	break;
				case EMemberKind::Range:	SaveMemberDelta(Out, Struct, Default, Name, Ctx, It.GrabRange());	break;
				case EMemberKind::Struct:	SaveMemberDelta(Out, Struct, Default, Name, Ctx, It.GrabStruct());	break;
			}
		}
		checkSlow(!It.HasMore());
	}
	else
	{
		SaveNamedMembers(Out, Struct, It, Declaration.GetMemberOrder(), Ctx);
	}
}

const FStructDeclaration& SaveStructDelta(FMemberBuilder& Out, const void* Struct, FBaseline Base, FBindId Id, const FSaveContext& Ctx)
{
	const FStructDeclaration* Declaration = nullptr;
	if (ICustomBinding* Custom = Ctx.Customs.FindStructToSave(Id, /* out */ Declaration))
	{
		const void* Default = Base.Get();
		if (!Default || Custom->DiffCustom(Struct, Default, Ctx))
		{
			Custom->SaveCustom(Out, Struct, Base, Ctx);
			// Could type-check saved custom struct against declaration here,
			// or add a type-checking BuildAndReset overload and call that.
			// It's not important to type-check generate bindings.
		}
	}
	else
	{
		const FSchemaBinding& Schema = Ctx.Schemas.GetStruct(Id, /* out */ Declaration);
		SaveSchemaBoundStructDelta(Out, Struct, Base, Schema, *Declaration, Ctx);
	}
	
	return *Declaration;
}

FBuiltStruct* SaveStructDelta(const void* Struct, FBaseline Base, FBindId Id, const FSaveContext& Ctx)
{
	FMemberBuilder Out;
	const FStructDeclaration& Declaration = SaveStructDelta(Out, Struct, Base, Id, Ctx);
	return Out.BuildAndReset(Ctx.Scratch, Declaration, Ctx.Schemas.GetDebug());
}

FBuiltStruct* SaveStructDeltaIfDiff(const void* Struct, FBaseline Base, FBindId Id, const FSaveContext& Ctx)
{
	FMemberBuilder Out;
	const void* Default = Base.Get();
	const FStructDeclaration* Declaration;
	if (ICustomBinding* Custom = Ctx.Customs.FindStructToSave(Id, /* out */ Declaration))
	{
		if (!Default || Custom->DiffCustom(Struct, Default, Ctx))
		{
			Custom->SaveCustom(Out, Struct, Base, Ctx);
		}
	}
	else
	{
		const FSchemaBinding& Schema = Ctx.Schemas.GetStruct(Id, /* out */ Declaration);
		if (!Default || Declaration->Occupancy == EMemberPresence::AllowSparse || DiffStructs(Struct, Default, Id, Ctx))
		{
			SaveSchemaBoundStructDelta(Out, Struct, Base, Schema, *Declaration, Ctx);
		}
	}
	
	return Out.IsEmpty() ? nullptr : Out.BuildAndReset(Ctx.Scratch, *Declaration, Ctx.Schemas.GetDebug());
}

void SaveSuper(FMemberBuilder& Out, const void* Super, FBindId SuperId, const FSaveContext& Ctx)
{
	const FStructDeclaration& Declaration = SaveStruct(Out, Super, SuperId, Ctx);
	Out.BuildSuperStruct(Ctx.Scratch, Declaration, Ctx.Schemas.GetDebug());
}

void SaveSuperDelta(FMemberBuilder& Out, const void* Super, FBaseline SuperBase, FBindId SuperId, const FSaveContext& Ctx)
{
	const FStructDeclaration& Declaration = SaveStructDelta(Out, Super, SuperBase, SuperId, Ctx);
	Out.BuildSuperStruct(Ctx.Scratch, Declaration, Ctx.Schemas.GetDebug());
}

////////////////////////////////////////////////////////////////////////////////////////////////

FRangeSaverBase::FRangeSaverBase(FScratchAllocator& Scratch, uint64 Num, SIZE_T ItemSize)
: Range(FBuiltRange::Create(Scratch, Num, ItemSize))
, It(Range->Data)
#if DO_CHECK
, End(It + Num * ItemSize)
#endif
{}

} // namespace PlainProps
