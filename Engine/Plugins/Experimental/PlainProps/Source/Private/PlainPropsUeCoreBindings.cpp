// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsUeCoreBindings.h"
#include "PlainPropsBuild.h"
#include "PlainPropsDiff.h"
#include "PlainPropsSave.h"
#include "Math/Transform.h"
#include "Misc/DefinePrivateMemberPtr.h"

#if UE_FNAME_OUTLINE_NUMBER
#if WITH_CASE_PRESERVING_NAME
UE_DEFINE_PRIVATE_MEMBER_PTR(FNameEntryId, GDisplayIndexInternal, FName, DisplayIndex);
#endif
UE_DEFINE_PRIVATE_MEMBER_PTR(FNameEntryId, GComparisonIndexInternal, FName, ComparisonIndex);
#endif

namespace PlainProps::UE
{

#if UE_FNAME_OUTLINE_NUMBER // dll exported FSensitiveName::InitXXX functions that rely on accessing private members of FName
#if WITH_CASE_PRESERVING_NAME
uint32 FSensitiveName::InitValue(FName Name)
{
	FNameEntryId DisplayId = Name.*GDisplayIndexInternal;
	return DisplayId.ToUnstableInt() | (DisplayId != Name.*GComparisonIndexInternal) * DifferentIdsFlag;
}
#else
FNameEntryId FSensitiveName::InitDisplayId(FName Name)
{
	return Name.*GComparisonIndexInternal;
}
#endif
#endif

FMemberId FNumeralGenerator::Grow(int32 Max)
{
	check(Max >= Cache.Num());
	int32 OldNum = Cache.Num();
	Cache.SetNumUninitialized(Max + 1);

	FName Numeral("_");
	for (int32 Idx = OldNum; Idx <= Max; ++Idx)
	{
		Numeral.SetNumber(NAME_EXTERNAL_TO_INTERNAL(Idx));
		Cache[Idx] = Indexer.NameMember(Numeral);
	}

	return Cache[Max];
}

void FTransformBinding::Save(FMemberBuilder& Dst, const FTransform& Src, const FTransform* Default, const FSaveContext& Ctx) const
{
	static_assert(std::is_same_v<decltype(FTransform().GetTranslation().X), double>);
	
	// Lazily cache declarations on first save, then hoist them to the stack
	if (!VectorDeclaration)
	{
		VectorDeclaration = &Ctx.Schemas.GetDeclaration(VectorId);
		QuatDeclaration = &Ctx.Schemas.GetDeclaration(QuatId);
	}
	const FStructDeclaration& VectorDecl = *VectorDeclaration;
	const FStructDeclaration& QuatDecl = *QuatDeclaration;

	FDenseMemberBuilder Inner = { Ctx.Scratch, Ctx.Schemas.GetDebug() };

	FVector T = Src.GetTranslation();
	FQuat R = Src.GetRotation();
	FVector S = Src.GetScale3D();

	if (Default)
	{
		if (T != Default->GetTranslation())
		{
			Dst.AddStruct(MemberIds[(uint8)EMember::Translate], VectorId, Inner.BuildHomogeneous(VectorDecl, T.X, T.Y, T.Z));
		}

		if (R != Default->GetRotation())
		{
			Dst.AddStruct(MemberIds[(uint8)EMember::Rotate], QuatId, Inner.BuildHomogeneous(QuatDecl, R.X, R.Y, R.Z, R.W));
		}

		if (S != Default->GetScale3D())
		{
			Dst.AddStruct(MemberIds[(uint8)EMember::Scale], VectorId, Inner.BuildHomogeneous(VectorDecl, S.X, S.Y, S.Z));
		}
	}
	else
	{	
		Dst.AddStruct(MemberIds[(uint8)EMember::Translate], VectorId, Inner.BuildHomogeneous(VectorDecl, T.X, T.Y, T.Z));		
		Dst.AddStruct(MemberIds[(uint8)EMember::Rotate], QuatId, Inner.BuildHomogeneous(QuatDecl, R.X, R.Y, R.Z, R.W));
		Dst.AddStruct(MemberIds[(uint8)EMember::Scale], VectorId, Inner.BuildHomogeneous(VectorDecl, S.X, S.Y, S.Z));
	}
}

inline FDiffNode MakeStructDiff(const void* A, const void* B, FMemberId Name, FBindId Id, FDiffContext& Ctx)
{
	return { DefaultStructBindType, Name, {.Struct = Id}, A, B, Ctx.LastSibling };
}

bool FTransformBinding::Diff(const FTransform& A, const FTransform& B, FDiffContext& Ctx) const
{
	if (!A.TranslationEquals(B, 0.0))
	{
		Ctx.AddDiff(MakeStructDiff(&A, &B, MemberIds[(uint8)EMember::Translate], VectorId, Ctx));
		return true;
	}
	if (!A.RotationEquals(B, 0.0))
	{
		Ctx.AddDiff(MakeStructDiff(&A, &B, MemberIds[(uint8)EMember::Rotate], QuatId, Ctx));
		return true;
	}
	if (!A.Scale3DEquals(B, 0.0))
	{
		Ctx.AddDiff(MakeStructDiff(&A, &B, MemberIds[(uint8)EMember::Scale], VectorId, Ctx));
		return true;
	}

	return false;
}

template<typename T>
T GrabAndMemcpy(FMemberLoader& Members)
{
	T Out;
	FStructLoadView Struct = Members.GrabStruct();
	Struct.Values.CheckSize(sizeof(T));
	FMemory::Memcpy(&Out, Struct.Values.Peek(), sizeof(T));
	return Out;
}

void FTransformBinding::Load(FTransform& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
{
	static_assert(std::is_same_v<decltype(FTransform().GetTranslation().X), double>);

	FMemberLoader Members(Src);

	if (Method == ECustomLoadMethod::Construct)
	{
		::new (&Dst) FTransform;
	}
				
	if (!Members.HasMore())
	{
		return;
	}

	if (Members.PeekNameUnchecked() == MemberIds[(uint8)EMember::Translate])
	{
		Dst.SetTranslation(GrabAndMemcpy<FVector>(Members));

		if (!Members.HasMore())
		{
			return;
		}
	}

	if (Members.PeekNameUnchecked() == MemberIds[(uint8)EMember::Rotate])
	{
		Dst.SetRotation(GrabAndMemcpy<FQuat>(Members));

		if (!Members.HasMore())
		{
			return;
		}
	}

	checkSlow(Members.PeekNameUnchecked() == MemberIds[(uint8)EMember::Scale]);
	Dst.SetScale3D(GrabAndMemcpy<FVector>(Members));
	checkSlow(!Members.HasMore());
}

//////////////////////////////////////////////////////////////////////////

void FGuidBinding::Save(FMemberBuilder& Dst, const FGuid& Src, const FGuid*, const FSaveContext&) const
{
	Dst.AddHex(MemberIds[0], Src.A);
	Dst.AddHex(MemberIds[1], Src.B);
	Dst.AddHex(MemberIds[2], Src.C);
	Dst.AddHex(MemberIds[3], Src.D);

	static_assert(offsetof(FGuid, A) == 0);
	static_assert(offsetof(FGuid, B) == 4);
	static_assert(offsetof(FGuid, C) == 8);
	static_assert(offsetof(FGuid, D) == 12);
}

//////////////////////////////////////////////////////////////////////////

void FColorBinding::Save(FMemberBuilder& Dst, const FColor& Src, const FColor*, const FSaveContext&) const
{
	Dst.Add(MemberIds[0], Src.B);
	Dst.Add(MemberIds[1], Src.G);
	Dst.Add(MemberIds[2], Src.R);
	Dst.Add(MemberIds[3], Src.A);

	static_assert(offsetof(FColor, B) == 0);
	static_assert(offsetof(FColor, G) == 1);
	static_assert(offsetof(FColor, R) == 2);
	static_assert(offsetof(FColor, A) == 3);
}

//////////////////////////////////////////////////////////////////////////

void FLinearColorBinding::Save(FMemberBuilder& Dst, const FLinearColor& Src, const FLinearColor*, const FSaveContext&) const
{
	Dst.Add(MemberIds[0], Src.R);
	Dst.Add(MemberIds[1], Src.G);
	Dst.Add(MemberIds[2], Src.B);
	Dst.Add(MemberIds[3], Src.A);

	static_assert(offsetof(FLinearColor, R) == 0);
	static_assert(offsetof(FLinearColor, G) == 4);
	static_assert(offsetof(FLinearColor, B) == 8);
	static_assert(offsetof(FLinearColor, A) == 12);
}

} // namespace PlainProps::UE

//////////////////////////////////////////////////////////////////////////
namespace PlainProps
{
	
template<>
void AppendString(FUtf8StringBuilderBase& Out, const UE::FSensitiveName& Name)
{
	Name.AppendString(Out);
}

}
