// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsUpgrade.h"
#include "PlainPropsInternalUpgrade.h"
#include "PlainPropsIndex.h"
#include "PlainPropsInternalBind.h"
#include "PlainPropsInternalBuild.h"
#include "PlainPropsInternalFormat.h"
#include "PlainPropsInternalRead.h"
#include "Algo/Count.h"
#include "Algo/Find.h"
#include "Containers/BitArray.h"
#include <type_traits>

namespace PlainProps
{

static_assert(sizeof(FLooseType) == sizeof(uint64), "AsInt() requirement");
static_assert(std::has_unique_object_representations_v<FLooseType>, "AsInt() requirement");

inline bool IsDecomposed(FInnermostType T)
{
	return (T.Kind == EInnermostKind::Struct && T.Struct.IsDecomposed) || (T.Kind==EInnermostKind::Enum && T.Enum.IsDecomposed);
}

static FLooseType ClearDecomposed(FLooseType Type)
{
	if (Type.InnermostIsStruct())
	{
		check(Type.InnermostType.Struct.IsDecomposed);
		Type.InnermostType.Struct.IsDecomposed = 0;
	}
	else
	{
		check(Type.InnermostIsEnum());
		check(Type.InnermostType.Enum.IsDecomposed);
		Type.InnermostType.Enum.IsDecomposed = 0;
	}
	return Type;
}

inline bool CompareDecomposedAndRecomposed(FLooseType Decomposed, FLooseType Recomposed)
{
	return ClearDecomposed(Decomposed).CompareAll(Recomposed);
}

void FLooseMember::RecordDynamicType(FLooseType In)
{
	check(Type.InnermostType == InnermostDynamicStruct);
	check(!Type.InnermostId || Type.InnermostId == In.InnermostId && Type.InnermostVersion == In.InnermostVersion);
	Type.InnermostId = In.InnermostId;
	Type.InnermostVersion = In.InnermostVersion;
}

static void DebugPrintVersion(FString& Out, uint16 Version)
{
	if (Version)
	{
		Out.AppendChar('@');
		Out.AppendInt(Version);
	}
}

static FString PrintType(const FDebugIds& Debug, FLooseType Type)
{
	FString Out;
	Out.Reserve(64);
	for (uint32 N = Type.NumRanges; N; --N)
	{
		Out.AppendChar('[');
	}

	if (Type.InnermostIsStruct())
	{
		Out.Append(Type.InnermostId ? Debug.Print(Type.InnermostId.Get().AsStruct()) : "struct");
		DebugPrintVersion(Out, Type.InnermostVersion);
		Out.Append(Type.InnermostType.Struct.IsDynamic ? "(dynamic)" : "");
		Out.Append(Type.InnermostType.Struct.IsSuper ? "(super)" : "");
		Out.Append(Type.InnermostType.Struct.IsDecomposed ? "(decomposed)" : "");
	}
	else if (Type.InnermostIsEnum())
	{
		Out.Append(Type.InnermostId ? Debug.Print((Type.InnermostId.Get().AsEnum())) : "enum");
		DebugPrintVersion(Out, Type.InnermostVersion);
		Out.Append(Type.InnermostType.Enum.FlagMode ? "(flag)" : "(flat)");
		Out.Append(Type.InnermostType.Enum.IsDecomposed ? "(decomposed)" : "");
	}
	else
	{
		FAnsiStringView Leaves[14] = { ("Bool"), ("S8"), ("S16"), ("S32"), ("S64"), ("U8"), ("U16"), ("U32"), ("U64"), ("F32"), ("F64"), ("UTF8"), ("UTF16"), ("UTF32") }; 
		Out.Append(Leaves[uint8(Type.InnermostType.Kind) - 2]);
	}

	if (Type.OptionalParameter)
	{
		Out.AppendChar('?');
	}

	for (uint32 N = Type.NumRanges; N; --N)
	{
		Out.AppendChar(']');
	}

	return Out;
}

////////////////////////////////////////////////////////////////////////////////////////////////

struct F64BitIndexIterator
{
	uint64					Bits = 0;
	
	uint8					operator*() const						{ checkSlow(Bits); return static_cast<uint8>(FMath::FloorLog2NonZero_64(Bits)); }
	void					operator++() 							{ Bits -= (uint64(1) << operator*()); }
	bool					operator!=(F64BitIndexIterator O) const	{ return Bits != O.Bits; }
};

// Range-based for adapter
struct F64BitIndexRange
{
	uint64					Bits = 0;
	
	F64BitIndexIterator		begin() const	{ return {Bits}; }
	F64BitIndexIterator		end() const		{ return {0}; }
};
inline F64BitIndexRange BitIndices(uint64 Bits) { return {Bits}; }

//struct FFlags
//{
//	uint64 Bits = 0;
//
//	void Raise(uint8 Idx) { Bits |= (uint64(1) << Idx); }
//	void Lower(uint8 Idx) { Bits &= ~(uint64(1) << Idx); }
//	explicit operator bool() const { return !!Bits; }
//};

struct FEnumeratorIdSetIterator : F64BitIndexIterator
{
	FEnumeratorIdSetIterator(uint64 Bits) : F64BitIndexIterator{.Bits = Bits} {}
	FEnumeratorId	operator*() const { return { .Idx = F64BitIndexIterator::operator*() }; }
};
static FEnumeratorIdSetIterator begin(FEnumeratorIdSet Flags)		{ return {Flags.Bits}; }
static FEnumeratorIdSetIterator end(FEnumeratorIdSet)				{ return {0}; }

static F64BitIndexIterator		begin(FEnumeratorIndexSet Flags)	{ return {Flags.Bits}; }
static F64BitIndexIterator		end(FEnumeratorIndexSet)			{ return {0}; }

////////////////////////////////////////////////////////////////////////////////////////////////

namespace Upgrade
{

FString FDebugPrinter::Print(FLooseType Type) const { return PrintType(*this, Type); }
	
// Experimental TLS debug printer
thread_local FDebugPrinter* TlsDebug;

static FString TlsPrint(FLooseType T) { return TlsDebug->Print(T); }

////////////////////////////////////////////////////////////////////////////////////////////////

struct FFriend
{
	static FLooseMember&			Edit(FIn& I)						{ return I; }
	static const FLooseMember&		Read(const FOut& O)					{ return O; }
	static TArrayView<FLooseMember>	Edit(TArrayView<FOut> Os)			{ return MakeArrayView(static_cast<FLooseMember*>(Os.GetData()), Os.Num()); }

	template<class T>
	static T*						NewRange(FContext Ctx, uint64 Num)	{ return Ctx.Scratch.AllocateArray<T>(Num, T{}); }
	static FScratchAllocator&		Scratch(FContext Ctx)				{ return Ctx.Scratch; }
};

////////////////////////////////////////////////////////////////////////////////////////////////

static TSet<FNameId> GetDeclaredOrder(const FEnumDeclaration& Decl)
{
	TSet<FNameId> Out;
	Out.Reserve(Decl.NumEnumerators);
	for (FEnumerator Enumerator : Decl.GetEnumerators())
	{
		Out.Add(Enumerator.Name);
	}
	return Out;
}

static FParameterIdSet GetDeclaredOrder(const FStructDeclaration& Decl)
{
	FParameterIdSet Out;
	Out.Reserve(Decl.NumMembers);
	if (Decl.Super)
	{
		Out.Add(NoId);
	}
	for (FMemberId Name : Decl.GetMemberOrder())
	{
		Out.Add(Name);
	}
	return Out;
}

template<class IndexerT, typename IdT>
IndexerT& GetIndexer(TMap<IdT, TUniquePtr<IndexerT>>& Map, IdT Id, const IDeclarations& Declarations)
{
	TUniquePtr<IndexerT>& Out = Map.FindOrAdd(Id);
	if (!Out)
	{
		const auto Decl = Declarations.Find(Id);
		Out.Reset(Decl ? new IndexerT(GetDeclaredOrder(*Decl)) : new IndexerT);
	}
	return *Out;
}

FEnumeratorIndexer& FLooseIndexers::Get(FEnumId Id) { return GetIndexer(/* in-out */ Enums, Id, Declarations); }
FParameterIndexer& FLooseIndexers::Get(FDeclId Id)	{ return GetIndexer(/* in-out */ Structs, Id, Declarations); }

////////////////////////////////////////////////////////////////////////////////////////////////

static FMemoryView ViewSkippableSlice(const void* Slice)
{
	uint32 NumBytesRead;
	uint64 Size = ReadVarUInt(Slice, NumBytesRead);
	return FMemoryView((const uint8*)Slice + NumBytesRead, Size);
}

static FStructView AsSourceStruct(FLooseValue Value, FSchemaBatchId Batch)
{
	return {{FStructSchemaId(Value.Meta.InnermostSchema.Get()), Batch}, FByteReader(ViewSkippableSlice(Value.Data.Struct.Source))};
}

class FSourceStructRangeReader
{
public:
	FSourceStructRangeReader(FStructRange Structs, uint64 Num) : SourceIt(Structs.Source), NumLeft(Num) {}

	bool			HasMore() const							{ return NumLeft > 0; }
	FStructView		GrabView(FStructSchemaHandle Schema)	{ return {Schema, FByteReader(GrabSlice())}; }	
	FStructData		GrabData()
	{
		FStructData Out = { .Source = static_cast<const uint8*>(SourceIt) };
		(void)GrabSlice(); // Advance
		return Out;
	}
private:
	const void*		SourceIt;
	uint64			NumLeft;

	FMemoryView		GrabSlice()
	{
		check(NumLeft);
		NumLeft--;
		FMemoryView Out = ViewSkippableSlice(SourceIt);
		SourceIt = Out.GetDataEnd();
		return Out;
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////

//@pre Idx < 64
inline void SetBit(uint64& Word, uint32 Idx, bool bValue)
{
	uint64 Mask = uint64(1) << Idx;
	if (bValue)
	{
		Word |= Mask;
	}
	else
	{
		Word &= ~Mask;
	}
}

static uint64 OutlinedHiBit = uint64(1) << 63;

inline bool IsOutlined(FEnumeratorIdSet Flags)
{
	return Flags.Bits & OutlinedHiBit;
}

inline uint64* AsOutlined(FEnumeratorIdSet Flags)
{
	return reinterpret_cast<uint64*>(Flags.Bits << 1);
}

FEnumeratorIdSet NewOutlined(FScratchAllocator& Scratch, uint16 NumEnumeratorIds)
{
	uint64* Words = Scratch.AllocateArray<uint64>(Align(NumEnumeratorIds, 64) / 64);
	uint64 Bits = OutlinedHiBit | (reinterpret_cast<uint64>(Words) >> 1);
	return { Bits };
}

// todo: this will be used by TranslateOutValue
static void SetLooseFlag(FEnumeratorIdSet& Out, FEnumeratorId Name, bool bValue)
{
	uint64& Word = IsOutlined(Out) ? AsOutlined(Out)[Name.Idx/64] : Out.Bits;
	SetBit(Word, Name.Idx % 64, bValue);
}

////////////////////////////////////////////////////////////////////////////////////////////////

inline ELooseState GetInnermostState(FLooseMetadata Meta)
{
	if (!Meta.IsSet)
	{
		return ELooseState::Unset;
	}
	else if (Meta.InnermostSchema)
	{
		return ELooseState::Source;
	}

	return Meta.IsFullyUpgraded ? ELooseState::UpgradedLoose : ELooseState::Loose;
}

inline FStructMember AsStruct(FLooseType Type, FLooseMetadata Meta, FStructData Data)
{
	check(Type.IsStruct());
	FDeclId Id = Type.InnermostId.Get().AsStructDeclId();
	FOptionalStructSchemaId Schema = ToOptionalStruct(Meta.InnermostSchema);
	return { Id, Schema, Meta.Name, Type.InnermostVersion, GetInnermostState(Meta), Meta.Mismatch, Meta.Num, Data };
}

inline FEnumMember AsEnum(FLooseType Type, FLooseMetadata Meta, FEnumData Data)
{
	check(Type.IsEnum());
	FEnumId Id = Type.InnermostId.Get().AsEnum();
	FOptionalEnumSchemaId Schema = ToOptionalEnum(Meta.InnermostSchema);
	return { Id, Schema, Meta.Name, Type.InnermostVersion, GetInnermostState(Meta), Type.InnermostType.Enum.FlagMode, IntCastChecked<uint16>(Meta.Num), Data };
}

inline FRangeMember AsRange(FLooseMember In)
{
	check(In.GetType().IsRange());
	return { In.GetType(), In.GetValue().Meta, In.GetValue().Data.Range };
}

inline FEnumMember AsEnum(FLooseMember In)
{
	return AsEnum(In.GetType(), In.GetValue().Meta, In.GetValue().Data.Enum);
}

inline FStructMember AsStruct(FLooseMember In)
{
	return AsStruct(In.GetType(), In.GetValue().Meta, In.GetValue().Data.Struct);
}

inline FStructView AsSource(FStructSchemaId Id, FSchemaBatchId Batch, FByteReader Data)
{
	return { {Id, Batch}, Data };
}

inline FStructView AsSource(FStructMember In, FSchemaBatchId Batch)
{
	check(In.State == ELooseState::Source);
	return { {In.Schema.Get(), Batch}, FByteReader(In.Data.Source, In.Num) };
}

static FLooseMember AsMember(FStructMember In)
{
	FLooseType Type = {.InnermostType = InnermostStaticStruct, .InnermostVersion = In.Version, .InnermostId = FInnerId(In.Id) };
	FLooseMetadata Meta = { .IsSet = 1, .IsFullyUpgraded = In.State == ELooseState::UpgradedLoose, .Name = In.Name, .Num = In.Num };
	FLooseData Data = { .Struct = In.Data };	
	return FLooseMember(Type, {Meta, Data});
}

#if DO_CHECK

// Opaque enums ignore flag mode and reuse FInEnum/FOutEnum[Item] for simplicity
// ReflectItem lack type info and report flag enums as flat, this works around that
inline bool OpaqueEnumEquals(FInnermostType Actual, FInnermostType Expected) 
{
	return Actual == InnermostFlagEnum && Expected == InnermostFlatEnum;
}

void CheckRangeType(FLooseType Actual, uint16 NumNamed, FInnermostType InnermostType, uint8 NumRanges)
{
	FLooseType Expected = {.NumRanges = NumRanges, .InnermostType = InnermostType};

	checkf(Actual.NumRanges == NumRanges && (Actual.InnermostType == InnermostType || OpaqueEnumEquals(Actual.InnermostType, InnermostType)), 
		TEXT("Parameter is a %s, not a %s"), *TlsPrint(Actual), *TlsPrint(Expected));
	check(IsDecomposed(InnermostType) == !!NumNamed);
	check(!IsDecomposed(InnermostType) || Actual.InnermostId);
}
#endif

////////////////////////////////////////////////////////////////////////////////////////////////

FInStruct FIn::Struct() const
{
	check(Type.IsStruct());
	check(Type.InnermostType == InnermostStaticStruct);
	check(Value);
	return FInStruct(Type, Value);
}

FInStruct FIn::OptStruct() const
{
	check(Type.OptionalParameter);
	check(Type.IsStruct());
	check(Type.InnermostType == InnermostStaticStruct);
	return FInStruct(Type, Value);
}

FInAny FIn::StructAny() const
{
	check(Type.IsStruct());
	check(Type.InnermostType == InnermostDynamicStruct);
	check(Value);
	return FInAny(Type, Value);
}

FInMembers FIn::StructMembers() const
{
	check(Type.IsStruct());
	check(Type.InnermostType == InnermostDecomposedStruct);
	check(Value.Meta.Num > 0);
	check(Value);
	check(Value.Data.Struct.Loose);
	return FInMembers(Type, Value);
}

FInEnum FIn::Enum() const
{
	check(Type.IsEnum());
	check(!Type.InnermostType.Enum.IsDecomposed);
	check(Value.Meta.Num == 0);
	check(!Value.Meta.InnermostSchema); // All enum are loosened since out enum ranges cant mix loose/source enums
	check(Value);
	return FInEnum(Type, Value.Data.Enum.Loose.Enumerator);
}

////////////////////////////////////////////////////////////////////////////////////////////////

FOut::FOut(FOutParameter Param)
: FLooseMember(Param.Type, FLooseValue{{.Name = Param.Name, .Num = Param.NumInners}, {}})
{
	check(Param.NumInners == 0 || Param.Type.InnermostType.Kind == EInnermostKind::Enum);
}

FOut::FOut(FOutParameter Param, FOut* Members)
: FLooseMember(Param.Type, FLooseValue{{.IsSet = 1, .Name = Param.Name, .Num = Param.NumInners}, {.Struct = {.Loose = Members}}})
{
	check(Param.NumInners > 0);
}

FOutFlags FOut::EnumFlags()
{
	check(Type.IsEnum());
	check(Type.InnermostType.Enum.FlagMode);
	check(!Value.Meta.InnermostSchema);
	check(Value.Meta.Num > 0 && Value.Meta.Num <= 64);
	return FOutFlags(Type, Value);
}

FOutStruct FOut::Struct()
{
	check(Type.IsStruct());
	check(Type.InnermostType == InnermostStaticStruct);
	return FOutStruct(Type, Value);
}

FOutMembers FOut::StructMembers()
{
	check(Type.IsStruct());
	check(Type.InnermostType == InnermostDecomposedStruct);
	check(!Value.Meta.InnermostSchema);
	check(Value.Meta.Num > 0);
	check(Value);
	check(Value.Data.Struct.Loose);
	return FOutMembers(Type, Value);
}

void FOutStruct::operator=(FInStruct In)
{
	checkf(Type.InnermostId == In.GetType().InnermostId, TEXT("Set wrong struct type"));
	checkf(Type.InnermostVersion == In.GetType().InnermostVersion, TEXT("Set wrong struct version"));
	Value.SetNamed(In.GetValue());
}

void FOutAny::operator=(FInStruct In)
{
	Owner.RecordDynamicType(In.GetType());
	Value.SetNamed(In.GetValue());
}

void FOutEnum::operator=(FInEnum In)
{
	check(Type.Compare(In.Type));
	Value.Meta.IsSet = 1;
	Value.Data.Enum.Loose.Enumerator = In.Value;
}

void FOutFlag::operator=(bool bValue)
{
	Out.Meta.IsSet = 1;
	SetBit(Out.Data.Enum.Loose.Enumerator.Flags.Bits, Idx, bValue);
}

void FOutFlag::operator=(TOptional<bool> bValue)
{
	if (bValue)
	{
		operator=(*bValue);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////

void FOutBoolItem::operator=(bool bValue)
{
	if (bValue)
	{
		Out |= Mask;
	}
	else
	{
		Out &= ~Mask;
	}	
}

void FOutBoolIt::Increment()
{
	Mask <<= 1;
	if (Mask == 0)
	{
		++Byte;
		Mask = 1;
	}
}

void FOutEnumItem::operator=(FInEnum In)
{
	check(Type.Compare(In.Type));
	Out = In.Value;
}

void FOutEnumeratorItem::operator=(uint16 Index)
{
	check(Index < Max);
	Out.Index = Index;
}

void FOutFlagItem::operator=(bool bValue)
{
	SetBit(Out.Bits, Idx, bValue);
}

void FOutFlagsItem::operator=(uint64 Indices)
{
	check(FMath::FloorLog2_64(Indices) < Max);
	Out.Bits = Indices;
}

////////////////////////////////////////////////////////////////////////////////////////////////

void FOutRange::Set(const FInRange& In)
{
	check(In.NumNamed == NumNamed);
	checkSlow(Out.GetType().Compare(In.Type));
	Out.EditTypechecked().SetNamed(In.Value);
}

void FOutRange::Set(FLooseNestedRange In)
{
	FLooseValue& Value = Out.EditTypechecked();
	Value.Meta.IsSet = 1;
	Value.Meta.Num = In.Num;
	Value.Data.Range = In.Data;
}

FRangeData FOutRange::NewNested(uint64 Num, FContext Ctx) const
{
	return { .Ranges = FFriend::NewRange<FLooseNestedRange>(Ctx, Num) };
}
FRangeData FOutRange::NewStructs(uint64 Num, FContext Ctx) const
{
	return { .Structs = {.Loose = FFriend::NewRange<FLooseValue>(Ctx, Num)} };
}
FRangeData FOutRange::NewMembers(uint64 Num, FContext Ctx) const
{
	FLooseValue* Structs = FFriend::NewRange<FLooseValue>(Ctx, Num);
	for (FLooseValue& Struct : MakeArrayView64(Structs, Num))
	{
		Struct.Meta.Num = NumNamed;
	}
	return { .Structs = {.Loose = Structs} };
}
FRangeData FOutRange::NewEnums(uint64 Num, FContext Ctx) const
{
	return { .Enums = {.Loose = FFriend::NewRange<FEnumLooseData>(Ctx, Num)} };
}
FRangeData FOutRange::NewEnumerators(uint64 Num, FContext Ctx) const
{
	return { .Enums = {.Loose = FFriend::NewRange<FEnumLooseData>(Ctx, Num)} };
}
FRangeData FOutRange::NewFlags(uint64 Num, FContext Ctx) const
{
	return { .Enums = {.Loose = FFriend::NewRange<FEnumLooseData>(Ctx, Num)} };
}
FRangeData FOutRange::NewBools(uint64 Num, FContext Ctx) const
{
	return { .Arithmetics = {.Loose = FFriend::NewRange<uint8>(Ctx, (Num + 7) / 8)} };
}
FRangeData FOutRange::NewArithmetics(uint64 Num, FContext Ctx, uint32 SizeOf) const
{
	return { .Arithmetics = {.Loose = FFriend::Scratch(Ctx).AllocateZeroed(Num * SizeOf, SizeOf)} };
}

////////////////////////////////////////////////////////////////////////////////////////////////

FEnumLoosener::~FEnumLoosener()
{
	if (Exists() && !IsFlag())
	{
		TMap<uint64, FEnumeratorId> Empty;
		Swap(reinterpret_cast<FFlatEnumLoosener*>(Ptr & PtrMask)->ConstantNames, Empty);
	}
}

FEnumLooseners::FEnumLooseners(TConstArrayView<FEnumTransformation> Xforms, uint32 InNumStructs)
: Data(Xforms.GetData())
, NumStructs(InNumStructs)
, NumSchemas(InNumStructs + Xforms.Num())
{}

const FEnumLoosener& FEnumLooseners::operator[](FEnumSchemaId Id) const
{ 
	check(Id.Idx >= NumStructs && Id.Idx < NumSchemas);
	return Data[Id.Idx - NumStructs].Source;
}

static FEnumLooseners ViewLooseners(FBatchContext& Ctx)
{
	return FEnumLooseners(Ctx.Transformations.Enums, Ctx.Transformations.Schemas.NumStructs);
}

////////////////////////////////////////////////////////////////////////////////////////////////

static FEnumeratorData Loosen(const FFlatEnumLoosener& Loosener, uint64 Constant)
{
	return { .Flat = Loosener.ConstantNames.FindChecked(Constant) };
}

static FEnumeratorData Loosen(const FFlagEnumLoosener& Loosener, uint64 Flags)
{
	FEnumeratorIdSet Out;
	for (uint8 FlagIdx : BitIndices(Flags))
	{
		FEnumeratorId Name = Loosener.FlagNames[FlagIdx];
		check(Name.Idx < 64); // todo: support >64 loose enumerators (e.g. scratch allocate bitset and maybe store num words in FLooseMeta::Num)
		Out.Bits |= uint64(1) << Name.Idx;
	}
	return { .Flags = Out };
}

static FEnumeratorData Loosen(const FEnumLoosener& Loosener, uint64 Constant)
{
	return Loosener.IsFlag() ? Loosen(Loosener.AsFlag(), Constant) : Loosen(Loosener.AsFlat(), Constant);
}

static uint64 Fasten(TConstArrayView<FEnumerator> Enumerators, FEnumeratorId Flat, const FEnumeratorIndexer& Indexer)
{
	FNameId Name = Indexer.Resolve(Flat);
	const FEnumerator* Enumerator = Algo::FindBy(Enumerators, Name, &FEnumerator::Name);
	check(Enumerator);
	return Enumerator->Constant;
}

static uint64 Fasten(TConstArrayView<FEnumerator> Enumerators, FEnumeratorIdSet Flags, const FEnumeratorIndexer& Indexer)
{
	// opt: O(N^2). Make a FEnumFlagFastener with e.g. TMap<FEnumeratorId, uint64> or TSet<FEnumeratorId> + Decl ptr
	uint64 Constants = 0;
	for (FEnumeratorId Flag : Flags)
	{
		Constants |= Fasten(Enumerators, Flag, Indexer);
	}
	return Constants;
}

static uint64 Fasten(FEnumFastener Fastener, FEnumeratorId Flat, const FEnumeratorIndexer& Indexer)
{
	return Fastener.Decl ? Fasten(Fastener.Decl->GetEnumerators(), Flat, Indexer) : Flat.Idx;
}

static uint64 Fasten(FEnumFastener Fastener, FEnumeratorIdSet Flags, const FEnumeratorIndexer& Indexer)
{
	return Fastener.Decl ? Fasten(Fastener.Decl->GetEnumerators(), Flags, Indexer) : Flags.Bits;
}

////////////////////////////////////////////////////////////////////////////////////////////////

static uint64 GetIntegerValue(FLeafView In)
{
	if (In.Leaf.Type == ELeafType::Bool)
	{
		return In.Value.bValue;
	}

	uint64 Out = 0;
	FMemory::Memcpy(&Out, In.Value.Ptr, SizeOf(In.Leaf.Width));
	return Out;
}

FLooseMemberReader::FLooseMemberReader(const FLooseSchema& Schema, FStructView Source, FEnumLooseners InLooseners, FScratchAllocator& InScratch)
: FMemberReader(Source)
, Parameters(Schema.Members)
, Scratch(InScratch)
, Looseners(InLooseners)
{}

FLooseParameter FLooseMemberReader::Peek() const
{
	check(HasMore());
	return Parameters[MemberIdx];
}

FLooseValue FLooseMemberReader::Grab()
{
	FLooseParameter Next = Peek();
	if (Next.Type.IsRange())
	{
		FRangeMember Range = ToLooseRange(Next.Name, Next.Type, FMemberReader::GrabRange());
		return { Range.Meta, { .Range = Range.Data } };
	}
	else if (Next.Type.IsStruct())
	{
		FStructView Struct = FMemberReader::GrabStruct();
		FLooseMetadata Meta = { .IsSet = 1, .Name = Next.Name, .InnermostSchema = Struct.Schema.Id, .Num = Struct.Values.CheckableSize() };
		return { Meta, {.Struct = { Struct.Values.Peek() }} };
	}
	
	FLeafView Leaf = GrabLeaf();
	uint64 Value = GetIntegerValue(Leaf);
	FLooseMetadata Meta = { .IsSet = 1, .Name = Next.Name };

	if (Leaf.Enum)
	{
		const FEnumLoosener& Loosener = Looseners[Leaf.Enum.Get()];
		if (Loosener.Needed())
		{
			FEnumeratorData Data = Next.Type.InnermostType.Enum.FlagMode 
								 ? Loosen(Loosener.AsFlag(), Value)
								 : Loosen(Loosener.AsFlat(), Value);
			return { Meta, {.Enum = {.Loose = {.Enumerator = Data }}} };		
		}
		else // source schema matches declaration
		{
			Meta.InnermostSchema = ToOptionalSchema(Leaf.Enum);
			return { Meta, {.Enum = {.Source = Value}} };	
		}
	}
	else
	{
		return { Meta, FLooseData{.Arithmetic = Value} };
	}
}

FRangeMember FLooseMemberReader::ToLooseRange(FParameterId Name, FLooseType Type, FRangeView Range)
{
	FOptionalSchemaId SchemaId = GetSchema(Range).InnermostSchema;
	FLooseMetadata Meta = { .IsSet = 1, .IsSourceRange = 1, .Name = Name, .InnermostSchema = SchemaId, .Num = Range.Num() };
	if (Type.NumRanges > 1)
	{
		Meta.IsSourceRange = 0;
		return { Type, Meta, {.Ranges = LoosenRanges(Range.AsRanges())} };
	}
	else if (Type.InnermostType.Kind == EInnermostKind::Struct)
	{
		return { Type, Meta, {.Structs = {.Source = GetValues(Range).GetData()}} };
	}
	else if (Type.InnermostType.Kind == EInnermostKind::Enum)
	{
		return { Type, Meta, {.Enums = {.Source = GetValues(Range).GetData()}} };
	}
	else
	{
		return { Type, Meta, {.Arithmetics = {.Source = GetValues(Range).GetData()}} };
	}
}

FLooseNestedRange* FLooseMemberReader::LoosenRanges(FNestedRangeView Ins)
{
	checkSlow(Ins.GetSchema().ItemType.IsRange());
	FLooseNestedRange* Outs = Scratch.AllocateArray<FLooseNestedRange>(Ins.Num());
	FLooseNestedRange* OutIt = Outs;
	FMemberType ItemType = Ins.GetSchema().NestedItemTypes[0];
	bool bEnum = IsEnum(ItemType);
	if (ItemType.IsRange())
	{
		for (FRangeView In : Ins)
		{
			*OutIt++ = { In.Num(), {.Ranges = LoosenRanges(In.AsRanges())} };
		}
	}
	else if (ItemType.IsStruct())
	{
		for (FRangeView In : Ins)
		{
			*OutIt++ = { In.Num(), {.Structs = {.Source = GetValues(In).GetData()}} };
		}
	}
	else if (bEnum)
	{
		for (FRangeView In : Ins)
		{
			*OutIt++ = { In.Num(), {.Enums = {.Source = GetValues(In).GetData()}} };
		}
	}
	else
	{
		for (FRangeView In : Ins)
		{
			*OutIt++ = { In.Num(), {.Arithmetics = {.Source = GetValues(In).GetData()}} };
		}
	}

	return Outs;
}

////////////////////////////////////////////////////////////////////////////////////////////////

template<class T>
T PopFirst(TArrayView<T>& View)
{
	T Out = View[0];
	View.RightChopInline(1);
	return Out;
}

template<class T>
TArrayView<T> PopFront(TArrayView<T>& View, int32 Num)
{
	TArrayView<T> Out = View.Slice(0, Num);
	View.RightChopInline(Num);
	return Out;
}

////////////////////////////////////////////////////////////////////////////////////////////////

static constexpr uint32 OutputStackSize = 2048;

FOutputStack::FOutputStack(FScratchAllocator& InScratch)
: Data(static_cast<FOut*>(InScratch.Allocate(sizeof(FOut) * 2048, alignof(FOut))))
, Scratch(InScratch)
{}

static FOut AllocateStructMembers(FOutParameter Param, TConstArrayView<FOutParameter>& InOutParams, FScratchAllocator& Scratch)
{
	uint32 Num = Param.NumInners;
	FOut* Members = static_cast<FOut*>(Scratch.Allocate(sizeof(FOut) * Num, alignof(FOut)));
	for (uint8 Idx = 0; Idx < Num; ++Idx)
	{
		FOutParameter Member = PopFirst(InOutParams);
		new (Members + Idx) FOut(Member.NumInners && Param.Type.IsStruct() ? AllocateStructMembers(Member, InOutParams, Scratch) : MoveTemp(Member));
	}
	return FOut(Param, Members);
}

FOuts FOutputStack::Push(TConstArrayView<FOutParameter> Params)
{
	check(Num + Params.Num() < OutputStackSize);

	// TPagedArray impl -- requires adding TPagedArray::SetNumUninitialized
	//int32 Idx = Stack.Num();
	//Stack.SetNumUninitialized(Idx + Params.Num());
	//check(&Stack[Idx + Params.Num() - 1] - &Stack[Idx] == Params.Num() - 1);
	//// Copy expected types so FOut can type-check
	//FOut* OutIt = &Stack[Idx];
	//for (FLooseParameter Param : Params)
	//{
	//	new (OutIt++) FOut(Param); 
	//}

	const int32 Idx = Num;
	while (!Params.IsEmpty())
	{
		FOutParameter Param = PopFirst(/* in-out */ Params);
		new (Data + Num) FOut(Param.NumInners && Param.Type.IsStruct() ? AllocateStructMembers(Param, /* in-out */ Params, Scratch) : FOut(Param));
		++Num;
	}
	
	return MakeArrayView(Data + Idx, Num - Idx);
}

void FOutputStack::Pop(int32 InNum)
{
	static_assert(std::is_trivially_destructible_v<FOut>);
	checkSlow(Num >= InNum);
	Num -= InNum;
}

FOutputScope::FOutputScope(FOutputStack& InStack, TConstArrayView<FOutParameter> Params)
: Stack(InStack)
, Outputs(Stack.Push(Params))
{}

//////////////////////////////////////////////////////////////////////////////////////////////////
//
//struct FOutsIt
//{
//	friend inline FOutsIt begin(FOuts Outs) { return {Outs.begin(), Outs.Num}; }
//	friend inline FOutsIt end(FOuts Outs)	{ return {Outs.end(), Outs.Num}; }
//
//	FLooseType*		It;
//	const uint32	Num;
//
//	FOutItem operator*() const { return {*It, *reinterpret_cast<FLooseValue*>(It+Num); }
//};


////////////////////////////////////////////////////////////////////////////////////////////////

inline TConstArrayView<const FVersionUpgrades*> ViewVersions(const FStructUpgrades& Struct)
{
	return MakeArrayView(Struct.Versions, Struct.NumVersions);
}

inline const FVersionUpgrades* GetVersion(const FStructUpgrades& Struct, uint16 Version)
{
	check(Version >= Struct.FirstVersion && Version < Struct.FirstVersion + Struct.NumVersions);
	return Struct.Versions[Version - Struct.FirstVersion];
}

static TConstArrayView<const FVersionUpgrades*> ViewVersionsFrom(const FStructUpgrades& Struct, uint16 FromVersion)
{
	int32 Skip = FromVersion - Struct.FirstVersion;
	check(Skip >= 0 && Skip <= Struct.NumVersions);
	return MakeArrayView(Struct.Versions + Skip, Struct.NumVersions - Skip);
}

inline TConstArrayView<FMiniSlice> ViewNameMatches(const FVersionUpgrades& Version)
{
	return MakeArrayView(Version.NameMatches, static_cast<int32>(Version.NumNames));
}

inline TConstArrayView<FMemberMatch> ViewMatches(const FVersionUpgrades& Version)
{
	return MakeArrayView(Version.GetMatches(), static_cast<int32>(Version.NumMatches));
}

inline TConstArrayView<FOp> ViewOps(const FVersionUpgrades& Version)
{
	return MakeArrayView(Version.GetOps(), static_cast<int32>(Version.NumOps));
}

////////////////////////////////////////////////////////////////////////////////////////////////

// Dependencies are reused between all FVersionUpgrades that need them, i.e. has any multi-input op
static TArrayView<FInputDependencies*> AllocateRequiredPartialInputs(FScratchAllocator& Scratch, TConstArrayView<const FVersionUpgrades*> Versions)
{
	uint16 MaxOps = 0;
	for (const FVersionUpgrades* Version : Versions)
	{
		for (FOp Op : ViewOps(*Version))	
		{
			if (Op.NumInputs() > 1)
			{
				MaxOps = FMath::Max(MaxOps, Version->NumOps);
				break;
			}
		}
	}

	FInputDependencies** Deps = MaxOps ? Scratch.AllocateArray<FInputDependencies*>(MaxOps, nullptr) : nullptr;
	return MakeArrayView(Deps, MaxOps);
}

FInstanceUpgrader::FInstanceUpgrader(FBatchContext& InCtx, const FStructUpgrades& InStruct, uint16 Version)
: Ctx(InCtx)
, Struct(InStruct)
, Current(GetVersion(Struct, Version))
, NumNames(InStruct.Versions[0]->NumNames)
, NameToScanIndices(Ctx.Scratch.AllocateArray<uint8>(NumNames, 0))
, PartialInputs(AllocateRequiredPartialInputs(Ctx.Scratch, ViewVersionsFrom(InStruct, Version)))
{}

static FLooseMember MakeMember(FLooseParameter Param)
{
	FLooseValue UnsetNamedValue = {{.Name = Param.Name}, {}};
	return FLooseMember(Param.Type, UnsetNamedValue);
}

static TArrayView<FLooseMember> CreateDummyMembers(uint16 Num, FBatchContext& Ctx)
{
	FLooseMember Dummy(Ctx.Obsolete, {});
	return MakeArrayView(Ctx.Scratch.AllocateArray<FLooseMember>(Num, Dummy), Num);
}

static TArrayView<FLooseMember> CreateMembersInNameOrder(TConstArrayView<FLooseParameter> Params, FBatchContext& Ctx)
{
	check(Params.Num() > 0);

	uint16 MaxNameIdx = 0;
	for (FLooseParameter Param : Params)
	{
		MaxNameIdx = FMath::Max(MaxNameIdx, Param.Name.Idx);
	}
	const int32 Num = MaxNameIdx + 1;

	TArrayView<FLooseMember> Out = CreateDummyMembers(Num, Ctx);
	for (FLooseParameter Param : Params)
	{
		check(Out[Param.Name.Idx].GetType().Compare(Ctx.Obsolete));
		Out[Param.Name.Idx] = MakeMember(Param);
	}

	return Out;
}

void FInstanceUpgrader::Expect(const FLooseSchema* Target)
{
	check(!Target || Target->NumMembers <= NumNames);
	Outcome = Target ? CreateMembersInNameOrder(Target->GetMembers(), Ctx) 
					 : CreateDummyMembers(Ctx.LooseIds.Get(Struct.Id).Num(), Ctx);
	bUntypedOutcome = !Target;
}

struct FMatchHelper
{
	explicit FMatchHelper(const FVersionUpgrades& Version)
	: NameMatches(ViewNameMatches(Version))
	, AllMatches(ViewMatches(Version))
	{}

	TConstArrayView<FMiniSlice>		NameMatches;
	TConstArrayView<FMemberMatch>	AllMatches;
};

void FInstanceUpgrader::Feed(FLooseMemberReader&& It)
{
	const FMatchHelper Helper(*Current);
	while (It.HasMore())
	{
		FLooseParameter Next = It.Peek();
		Give(Next.Type, It.Grab(), Helper);
	}
}

void FInstanceUpgrader::Feed(TConstArrayView<FLooseMember> Members)
{
	check(uint32(Members.Num()) <= NumNames);
	const FMatchHelper Helper(*Current);
	for (FLooseMember Member : Members)
	{
		if (Member)
		{
			Give(Member.GetType(), Member.GetValue(), Helper);
		}
	}
}

static const FTransformation& Lookup(const FTransformations& Xforms, FOptionalStructSchemaId Schema, FOptionalDeclId Id)
{
	return Schema ? Xforms[Schema.Get()] : Xforms[Id.Get()];
}

static const FTransformation* FindUpgrade(FLooseType Type, FLooseMetadata Meta, FBatchContext& Ctx)
{
	if (!Meta.IsSet || Meta.IsFullyUpgraded || !Type.InnermostId)
	{
		return nullptr;
	}
	else if (Type.InnermostType.Kind == EInnermostKind::Struct)
	{
		const FTransformation& Xform = Lookup(Ctx.Transformations, ToOptionalStruct(Meta.InnermostSchema), ToOptionalDeclId(Type.InnermostId));
		return Xform.Upgrades ? &Xform : nullptr;
	}
	else
	{
		check(Type.InnermostType.Kind == EInnermostKind::Enum);
		return nullptr; // todo: enum upgrades
	}
}

FORCENOINLINE static FLooseMember UpgradeUnmatchedMember(FLooseType Type, FLooseValue Value, const FTransformation& Xform, FBatchContext& Ctx)
{
	checkf(Type.IsStruct(), TEXT("TODO: Upgrade outdated [nested] struct and enum ranges"));
	FStructMember In = AsStruct(Type, Value.Meta, Value.Data.Struct);
	FInstanceUpgrader Upgrader(Ctx, *Xform.Upgrades, Type.InnermostVersion);
	FStructMember Out = Upgrader.Upgrade(In, Xform.TargetSchema);
	return AsMember(Out);
}

// Source range of source structs -> loose range of source structs
static FStructRange LoosenStructRangeOnly(FLooseNestedRange Range, FSchemaId Schema, FScratchAllocator& Scratch)
{
	FLooseValue* Out = Scratch.AllocateArray<FLooseValue>(Range.Num, FLooseValue{{.IsSet = 1, .InnermostSchema = Schema}});
	FSourceStructRangeReader SourceIt(Range.Data.Structs, Range.Num);
	for (uint64 Idx = 0; Idx < Range.Num; ++Idx)
	{
		Out[Idx].Data.Struct = SourceIt.GrabData();
	}
	return { .Loose = Out };
}

template<typename T, typename LoosenerType>
void LoosenEnums(FEnumLooseData* Dst, TConstArrayView64<T> Src, const LoosenerType& Loosener)
{
	for (T Constant : Src)
	{
		*Dst++ = { .Enumerator = Loosen(Loosener, Constant) };
	}
}

template<typename T>
void BitCastEnums(FEnumLooseData* Dst, TConstArrayView64<T> Src)
{
	for (T Constant : Src)
	{
		*Dst++ = { .Enumerator = BitCast<FEnumeratorData>(uint64(Constant)) };
	}
}

template<typename T>
void LoosenEnums(FEnumLooseData* Dst, const T* Src, uint64 Num, const FEnumLoosener& Loosener)
{
	// @see LoosenerRequired()
	if (!Loosener.Exists())
	{
		return BitCastEnums<T>(Dst, MakeArrayView64(Src, Num));
	}

	return Loosener.IsFlag()	? LoosenEnums<T>(Dst, MakeArrayView64(Src, Num), Loosener.AsFlag())
								: LoosenEnums<T>(Dst, MakeArrayView64(Src, Num), Loosener.AsFlat());
}

static FEnumRange LoosenEnums(ELeafWidth SourceWidth, FLooseNestedRange Range, const FEnumLoosener& Loosener, FScratchAllocator& Scratch)
{
	const void* Src = Range.Data.Enums.Source;
	FEnumLooseData* Dst = Scratch.AllocateArray<FEnumLooseData>(Range.Num);
	switch (SourceWidth)
	{
		case ELeafWidth::B8:	LoosenEnums(Dst, static_cast<const uint8* >(Src), Range.Num, Loosener); break;
		case ELeafWidth::B16:	LoosenEnums(Dst, static_cast<const uint16*>(Src), Range.Num, Loosener); break;
		case ELeafWidth::B32:	LoosenEnums(Dst, static_cast<const uint32*>(Src), Range.Num, Loosener); break;
		case ELeafWidth::B64:	LoosenEnums(Dst, static_cast<const uint64*>(Src), Range.Num, Loosener); break;
		default:				check(false); break;
	}

	return { .Loose = Dst };
}

static FRangeData LoosenRangeForInput(FLooseType Type, FLooseNestedRange Value, FSchemaId SchemaId, FBatchContext& Ctx)
{
	check(Value.Num);

	FLooseType ItemType = ItemOf(Type);
	if (ItemType.IsRange())
	{
		for (FLooseNestedRange& Range : MakeArrayView64(Value.Data.Ranges, Value.Num))
		{
			Range.Data = Range.Num ? LoosenRangeForInput(ItemType, Range, SchemaId, Ctx) : Range.Data;
		}
		return Value.Data;
	}
	else if (ItemType.IsStruct())
	{
		check(!Type.InnermostType.Struct.IsDecomposed);
		return { .Structs = LoosenStructRangeOnly(Value, SchemaId, Ctx.Scratch) };
	}
	else
	{
		check(ItemType.IsEnum());
		check(!Type.InnermostType.Enum.IsDecomposed);
		FEnumSchemaId Id(SchemaId);
		const FEnumSchema& Schema = ResolveEnumSchema(Ctx.Batch, Id);
		return { .Enums = LoosenEnums(Schema.Width, Value, ViewLooseners(Ctx)[Id], Ctx.Scratch) };
	}
}

static FLooseValue LoosenForInput(FLooseType Type, FLooseValue Value, FSchemaId SchemaId, FBatchContext& Ctx)
{
	if (Type.IsRange())
	{
		check(Type.NumRanges > 1 || Value.Meta.IsSourceRange);
		if (uint64 Num = Value.Meta.Num)
		{
			Value.Data.Range = LoosenRangeForInput(Type, {Value.Meta.Num, Value.Data.Range}, SchemaId, Ctx);
		}
	}
	else
	{
		check(Type.IsEnum());
		Value.Data.Enum.Loose.Enumerator = Loosen(ViewLooseners(Ctx)[FEnumSchemaId(SchemaId)], Value.Data.Enum.Source);
	}

	Value.Meta.InnermostSchema = NoId;
	Value.Meta.IsSourceRange = 0;
	return Value;
}
				
void FInstanceUpgrader::Give(FLooseType Type, FLooseValue Value, FMatchHelper Helper)
{
	FParameterId Name = Value.Meta.Name;
	if (Name.Idx < NumNames)
	{
		const FMiniSlice NameMatches = Helper.NameMatches[Name.Idx];
		uint8& ScanIdx = NameToScanIndices[Name.Idx];
Rematch:
		for (FMemberMatch Match : Helper.AllMatches.Slice(NameMatches.Idx + ScanIdx, NameMatches.Num - ScanIdx))
		{
			++ScanIdx;
			check(ScanIdx);
			if (Match.Type.Compare(Type)) // todo: check code-gen and change WithoutMetadata impl if needed
			{
				if (Match.Decomposes())
				{
					Value = Decompose(Type, Value, Match.Idx.Decomp);
					if (Value.IsMismatch())
					{
						checkSlow(Value.Meta.Mismatch == EMismatch::Inners || Value.Meta.Mismatch == EMismatch::Inputs);
						Value.Meta.Mismatch = EMismatch::Inputs;
						goto Done;
					}

					checkf(ScanIdx < NameMatches.Num, TEXT("Input decomposition match can't be last match for '%s' in '%s'"), *Print(Name), *Print(Struct.Id));
					Match = Helper.AllMatches[NameMatches.Idx + ScanIdx];
					++ScanIdx;
					checkf(CompareDecomposedAndRecomposed(Match.Type, Type), TEXT("The match after an input decomposition match should be of same type, expected %s, actual %s"), *Print(Type), *Print(Match.Type));
					checkf(!Match.Decomposes(), TEXT("Can't input decompose twice"));
				}
				else if (Value.Meta.InnermostSchema && (Type.InnermostIsEnum() || Type.IsRange()))
				{
					Value = LoosenForInput(Type, Value, Value.Meta.InnermostSchema.Get(), Ctx);
				}

				Input(Match, Value);
				return;
			}
		}

		// Unmatched

		if (const FTransformation* Xform = FindUpgrade(Type, Value.Meta, Ctx))
		{
			// Todo: Only upgrade to next matching version, e.g. upgrading V1 -> V2 -> V3 would miss a V2 match.
			//		 Determine next version by looking at matches
			FLooseMember Upgraded = UpgradeUnmatchedMember(Type, Value, *Xform, Ctx);
			checkSlow(Name.Idx == Upgraded.GetName().Idx);

			bool bVersionUpdated = !Upgraded.GetType().Compare(Type);
			Type = Upgraded.GetType();
			Value = Upgraded.GetValue();

			if (bVersionUpdated)
			{
				ScanIdx = 0;
				goto Rematch;
			}
		}
	}
Done:
	Stash(Type, Value);
}

void FInstanceUpgrader::Give(FLooseType Type, FLooseValue Value)
{
	check(Current);
	Give(Type, Value, FMatchHelper(*Current));
}

void FInstanceUpgrader::Stash(FLooseType Type, FLooseValue Value)
{
	EMismatch Mismatch = Value.Meta.Mismatch;
	FParameterId Name = Value.Meta.Name;
	if (Mismatch != EMismatch::No)
	{}
	else if (Name.Idx >= static_cast<uint32>(Outcome.Num()))
	{
		Mismatch = EMismatch::Surplus;
	}
	else if (FLooseMember& Out = Outcome[Name.Idx]; Out)
	{
		Mismatch = EMismatch::Duplicate;
		Out.EditTypechecked().SetFirstMismatch(EMismatch::Duplicate);
	}
	else if (bUntypedOutcome)
	{
		Out = FLooseMember(Type, Value);
		return;
	}
	else if (Type.Compare(Out.GetType()))
	{
		Out.SetTypechecked(Value);
		return;
	}
	else
	{
		Mismatch = EMismatch::Type;
	}

	Value.Meta.Mismatch = Mismatch;
	Mismatches.Emplace(Type, Value);
}

void FInstanceUpgrader::Input(FMemberMatch Match, FLooseValue Value)
{
	check(Match.Idx.Op < Current->NumOps);
	check(Value.Meta.Mismatch == EMismatch::No);
	FOp Op =  Current->GetOps()[Match.Idx.Op];
	FIn In(Match.Type, Value);
	if (Op.NumInputs() == 1)
	{
		Run(Op, &In, Match.Recipient); // Unary op
	}
	else
	{
		FInputDependencies*& Deps = PartialInputs[Match.Idx.Op];
		if (Deps == nullptr)
		{
			FInputDependencies Header = {Op.NumRequiredInputs, Op.NumOptionalInputs};
			uint32 Size = offsetof(FInputDependencies, Inputs) + sizeof(FIn) * Op.NumInputs();
			Deps = new (Ctx.Scratch.Allocate(Size, alignof(FInputDependencies))) FInputDependencies(Header);
			FMemory::Memzero(Deps->Inputs, sizeof(FIn) * Op.NumInputs());
			FirstPartialOpIdx = FMath::Min(FirstPartialOpIdx, Match.Idx.Op);
		}

		// Set parameter
		checkf(!Deps->Inputs[Match.InputIdx], TEXT("Parameter alredy set"));
		Deps->Inputs[Match.InputIdx] = In;

		// Decrement dependency count
		int32& NumMissing = Match.Type.OptionalParameter ? Deps->MissingOptional : Deps->MissingRequired;
		check(NumMissing > 0);
		--NumMissing;

		if (Deps->MissingRequired + Deps->MissingOptional == 0)
		{
			Run(Op, Deps->Inputs, Match.Recipient);

			// Let Flush know Op has executed and prepare reuse by later FVersionUpgrades
			Deps = nullptr;
		}
	}
}

static FLooseValue LoosenStruct(const FLooseSchema& LooseSchema, FStructView Source, FEnumLooseners Looseners, FScratchAllocator& Scratch)
{
	TArray<FLooseMember, TInlineAllocator<16>> Members;
	FLooseMemberReader It(LooseSchema, Source, Looseners, Scratch);
	while (It.HasMore())
	{
		FLooseParameter Param = It.Peek(); // Peek() before Grab()
		Members.Emplace(Param.Type, It.Grab());
	}
	
	uint64 Num(Members.Num());
	return {{.IsSet = 1, .Num = Num }, {.Struct = {.Loose = Scratch.CloneArray(Num, Members.GetData())}}};
}

static const FLooseSchema& ObtainSourceSchema(FBatchContext& Ctx, FDeclId Id, FStructSchemaId SchemaId);

static FRangeData LoosenItemsForDecomposition(FLooseType ItemType, FSchemaId SchemaId, FLooseNestedRange Range, FBatchContext& Ctx)
{
	if (ItemType.IsRange())
	{
		FLooseType InnerItemType = ItemOf(ItemType);
		for (FLooseNestedRange Item : MakeArrayView64(Range.Data.Ranges, Range.Num))
		{
			LoosenItemsForDecomposition(InnerItemType, SchemaId, Item, Ctx);
		}
		return {.Ranges = Range.Data.Ranges};
	}
	else if (ItemType.IsStruct())
	{
		check(!ItemType.InnermostType.Struct.IsDecomposed);
		FDeclId Id = ItemType.InnermostId.Get().AsStructDeclId();
		const FLooseSchema& LooseSchema = ObtainSourceSchema(Ctx, Id, FStructSchemaId(SchemaId));
		FEnumLooseners Looseners = ViewLooseners(Ctx);

		FLooseValue* Out = Ctx.Scratch.AllocateArray<FLooseValue>(Range.Num);
		FStructSchemaHandle Schema = { FStructSchemaId(SchemaId), Ctx.Batch };
		FSourceStructRangeReader SourceIt(Range.Data.Structs, Range.Num);
		for (uint64 Idx = 0; Idx < Range.Num; ++Idx)
		{
			Out[Idx] = LoosenStruct(LooseSchema, SourceIt.GrabView(Schema), Looseners, Ctx.Scratch);
		}

		return {.Structs = {.Loose = Out}};
	}
	else
	{
		check(ItemType.IsEnum());
		check(!ItemType.InnermostType.Enum.IsDecomposed);
		FEnumSchemaId Id(SchemaId);
		const FEnumSchema& Schema = ResolveEnumSchema(Ctx.Batch, Id);
		return { .Enums = LoosenEnums(Schema.Width, Range, ViewLooseners(Ctx)[Id], Ctx.Scratch) };
	}
}

FLooseValue FInstanceUpgrader::Decompose(FLooseType Type, FLooseValue Value, uint16 DecompIdx)
{
	check(DecompIdx < Current->NumDecompositions);
	uint32 Offset = Current->GetDecompOffsets()[DecompIdx];
	const void* Ptr = ((const uint8*)Current) + Offset;
	if (Type.IsRange())
	{
		FLooseType ItemType = ItemOf(Type);
		if (FOptionalSchemaId Schema = Value.Meta.InnermostSchema)
		{
			Value.Data.Range = LoosenItemsForDecomposition(ItemType, Schema.Get(), {Value.Meta.Num, Value.Data.Range}, Ctx);
			Value.Meta.InnermostSchema = NoId;
		}

		Decompose(ItemType, {Value.Meta.Num, Value.Data.Range}, Ptr);
		return Value;
	}
	
	return Type.IsStruct()
		? Decompose(AsStruct(Type, Value.Meta, Value.Data.Struct), *static_cast<const FStructDecomposition*>(Ptr))
		: Decompose(AsEnum(Type, Value.Meta, Value.Data.Enum), *static_cast<const FEnumDecomposition*>(Ptr));
}

void FInstanceUpgrader::Decompose(FLooseType ItemType, FLooseNestedRange Range, const void* Decomp)
{
	if (Range.Num == 0)
	{
		return;
	}
	else if (ItemType.IsRange())
	{
		for (FLooseNestedRange Item : MakeArrayView64(Range.Data.Ranges, Range.Num))
		{
			Decompose(ItemOf(ItemType), Item, Decomp);
		}
	}
	else if (ItemType.IsStruct())
	{
		const FStructDecomposition& StructDecomp = *static_cast<const FStructDecomposition*>(Decomp);
		for (FLooseValue& Item : MakeArrayView64(Range.Data.Structs.Loose, Range.Num))
		{
			Item = Decompose(AsStruct(ItemType, Item.Meta, Item.Data.Struct), StructDecomp);
		}
	}
	else
	{
		const FEnumDecomposition& EnumDecomp = *static_cast<const FEnumDecomposition*>(Decomp);
		const FLooseMetadata Dummy = {};
		for (FEnumLooseData& Item : MakeArrayView64(Range.Data.Enums.Loose, Range.Num))
		{
			Item = Decompose(AsEnum(ItemType, Dummy, {.Loose = Item}), EnumDecomp).Data.Enum.Loose;
		}
	}
}

FLooseValue FInstanceUpgrader::Decompose(FLooseType Type, FLooseValue Value, const FEnumDecomposition& Decomp)
{
	check(Value.Meta.IsSet);
	return Decomp.InputOrder.IsEmpty() && Value.Meta.IsFullyUpgraded ? Value : Decompose(AsEnum(Type, Value.Meta, Value.Data.Enum), Decomp);
}

FLooseValue FInstanceUpgrader::Decompose(FEnumMember In, const FEnumDecomposition& Decomp)
{
	FEnumeratorData Enumerator = {};
	if (In.State == ELooseState::Source)
	{
		const FEnumLoosener& Loosener = Ctx.Transformations[In.Schema.Get()].Source;
		Enumerator = In.FlagMode	? Loosen(Loosener.AsFlag(), In.Data.Source)
									: Loosen(Loosener.AsFlat(), In.Data.Source);
	}
	else
	{
		Enumerator = In.Data.Loose.Enumerator;
	}

	FEnumIndexData Decomposed = Reorder(Enumerator, Decomp.InputOrder, In.FlagMode);
	return { {.IsSet = 1, .Name = In.Name}, {.Enum = {.Loose = {.Decomposed = Decomposed}}} };
}

static int32 GetInputIndex(FEnumeratorId Name, const TSet<FEnumeratorId>& Order)
{
	FSetElementId Idx = Order.FindId(Name);
	check(Idx.IsValidId());
	return Idx.AsInteger();
}

FEnumIndexData FInstanceUpgrader::Reorder(FEnumeratorData In, const TSet<FEnumeratorId>& Order, bool bFlagMode)
{
	if (Order.IsEmpty())
	{
		return BitCast<FEnumIndexData>(In);
	}
	return bFlagMode ? Reorder(In.Flags, Order) : Reorder(In.Flat, Order); 
}

FEnumIndexData FInstanceUpgrader::Reorder(FEnumeratorId Flat, const TSet<FEnumeratorId>& Order)
{
	return { .Flat = { .Index = static_cast<uint16>(GetInputIndex(Flat, Order)) } };
}

FEnumIndexData FInstanceUpgrader::Reorder(FEnumeratorIdSet Flags, const TSet<FEnumeratorId>& Order)
{
	uint64 Indices = 0;
	for (FEnumeratorId Flag : Flags)
	{
		int32 OutIdx = GetInputIndex(Flag, Order);
		check(OutIdx < 64);
		Indices |= uint64(1) << OutIdx;
	}
	return { .Flags { .Bits = Indices } };
}

static TArrayView<FLooseMember> Fuse(TArrayView<FLooseMember> As, TConstArrayView<FLooseMember> Bs, FScratchAllocator& Scratch)
{
	// Compact As
	int32 Pos = 0;
	for (FLooseMember& A : As)
	{
		if (A || A.GetValue().Meta.Mismatch == EMismatch::Missing)
		{
			As[Pos++] = A;
		}
	}

	// Place mismatches last, might contain EMismatch::Duplicate
	Algo::StableSort(As.Left(Pos), [](FLooseMember A1, FLooseMember A2) { return A1.GetValue().IsMismatch() < A2.GetValue().IsMismatch(); });

	// Place Bs after compacted As, reallocate if needed
	TArrayView<FLooseMember> Out(As.GetData(), Pos + Bs.Num());
	if (Out.Num() > As.Num())
	{
		static_assert(std::is_trivially_copyable_v<FLooseMember> && std::is_trivially_destructible_v<FLooseMember>);
		Out = MakeArrayView(reinterpret_cast<FLooseMember*>(Scratch.Allocate(Out.NumBytes(), alignof(FLooseMember))), Out.Num());
		FMemory::Memcpy(Out.GetData(), As.GetData(), As.Left(Pos).NumBytes());
	}
	FMemory::Memcpy(&Out[Pos], Bs.GetData(), Bs.NumBytes());

	return Out;
}

static void AddInput(FLooseMember* Expected, FLooseType Type, FLooseValue Value, TArray<FLooseMember>& OutUnexpected)
{
	check(Value.Meta.IsSet);
	if (Expected && Type.Compare(Expected->GetType()))
	{
		Expected->SetTypechecked(Value);
	}
	else
	{
		Value.Meta.Mismatch = Expected ? EMismatch::Type : EMismatch::Surplus;
		OutUnexpected.Emplace(Type, Value);
	}
}

static TArrayView<FLooseMember> FinalizeInputs(TArrayView<FLooseMember> Expected, TConstArrayView<FLooseMember> Unexpected, FScratchAllocator& Scratch, EMismatch& OutMismatch)
{
	// Flag missing inputs as mismatches
	for (FLooseMember& Input : Expected)
	{
		if (!Input && !Input.GetType().OptionalParameter)
		{
			OutMismatch = EMismatch::Inputs;
			Input.EditTypechecked().Meta.Mismatch = EMismatch::Missing;
		}
	}

	// Fuse surplus and type mismatches
	if (!Unexpected.IsEmpty())
	{
		OutMismatch = EMismatch::Inputs;
		return Fuse(Expected, Unexpected, Scratch);
	}

	return Expected;
}

static TArrayView<FLooseMember> NewMembersInInputOrder(TConstArrayView<FLooseParameter> Params, FScratchAllocator& Scratch)
{
	int32 Num = Params.Num();
	FLooseMember* OutIt = static_cast<FLooseMember*>(Scratch.Allocate(Num * sizeof(FLooseMember), alignof(FLooseMember)));
	for (FLooseParameter Param : Params)
	{
		new (OutIt++) FLooseMember(MakeMember(Param));
	}
	return MakeArrayView(OutIt - Num, Num);
}

static TArrayView<FLooseMember> ReorderInputs(TConstArrayView<FLooseMember> Unsorted, TConstArrayView<FLooseParameter> Params, const TSet<FParameterId>& Order, FScratchAllocator& Scratch, EMismatch& OutMismatch)
{
	check(Params.Num() == Order.Num());

	TArray<FLooseMember> Unexpected;
	TArrayView<FLooseMember> Expected = NewMembersInInputOrder(Params, Scratch);
	for (FLooseMember Src : Unsorted)
	{
		if (Src)
		{
			FSetElementId Place = Order.FindId(Src.GetName());
			AddInput(Place.IsValidId() ? &Expected[Place.AsInteger()] : nullptr, Src.GetType(), Src.GetValue(), /* out */ Unexpected);
		}
	}

	return FinalizeInputs(Expected, Unexpected, Scratch, OutMismatch);
}

FLooseValue FInstanceUpgrader::Decompose(FStructMember In, const FStructDecomposition& Decomp)
{
	check(In.State != ELooseState::Unset);

	const FLooseSchema& To = *Decomp.InputSchema;
	TArrayView<FLooseMember> Members;
	EMismatch Mismatch = EMismatch::No;
	if (In.State == ELooseState::Source)
	{
		Members = Decompose(AsSource(In, Ctx.Batch), Decomp, In.Name, /* out */ Mismatch);
	}
	else
	{
		if (In.State == ELooseState::Loose && Decomp.Upgrades)
		{
			FInstanceUpgrader Upgrader(Ctx, *Decomp.Upgrades, To.Version);
			FStructMember Upgraded = Upgrader.Upgrade(In, &To);
			Members = MakeArrayView(Upgraded.Data.Loose, static_cast<int32>(Upgraded.Num));
			Mismatch = Upgraded.Mismatch;
		}
		else
		{
			Members = MakeArrayView(In.Data.Loose, static_cast<int32>(In.Num));
		}
	
		if (!Decomp.InputOrder.IsEmpty())
		{
			Members = ReorderInputs(Members, To.GetMembers(), Decomp.InputOrder, Ctx.Scratch, /* out */ Mismatch);
		}
		else if (Members.Num() > To.NumMembers) // Declaration order
		{
			Mismatch = EMismatch::Inputs;
			for (FLooseMember& Surplus : Members.RightChop(To.NumMembers))
			{
				Surplus.EditTypechecked().Meta.Mismatch = EMismatch::Surplus;
			}
		}
	}
	check(Members.Num() == To.NumMembers || Mismatch != EMismatch::No);
	
	return { {.IsSet = 1, .Mismatch = Mismatch, .Name = In.Name, .Num = static_cast<uint32>(Members.Num())}, {.Struct = {.Loose = Members.GetData()}} };
}

TArrayView<FLooseMember> FInstanceUpgrader::Decompose(FStructView In, const FStructDecomposition& Decomp, FParameterId Name, EMismatch& OutMismatch)
{
	const FTransformation& Xform = Ctx.Transformations[In.Schema.Id];
	const FLooseSchema& From = *Xform.SourceSchema;
	const FLooseSchema& To = *Decomp.InputSchema;
	
	if (Decomp.Upgrades)
	{
		FInstanceUpgrader Upgrader(Ctx, *Decomp.Upgrades, To.Version);
		FLooseStruct Upgraded = Upgrader.Upgrade(In, From, &To, OutMismatch);
		OutMismatch = OutMismatch == EMismatch::Inners ? EMismatch::Inputs : OutMismatch;
		TArrayView<FLooseMember> Members(Upgraded.Members, Upgraded.Num);
		return Decomp.InputOrder.IsEmpty() ? Members : ReorderInputs(Members, To.GetMembers(), Decomp.InputOrder, Ctx.Scratch, OutMismatch);
	}

	// Loosen source into input order
	check(!Xform.Upgrades);
	TArrayView<FLooseMember> Expected = NewMembersInInputOrder(To.GetMembers(), Ctx.Scratch);
	TArray<FLooseMember> Unexpected; // Mismatches
	FLooseMemberReader It(To, In, ViewLooseners(Ctx), Ctx.Scratch);
	const TSet<FParameterId>& Order = Decomp.InputOrder;
	if (Order.IsEmpty())
	{
		while (It.HasMore())
		{
			FLooseParameter Param = It.Peek();
			AddInput(Param.Name.Idx < Expected.Num() ? &Expected[Param.Name.Idx] : nullptr, Param.Type, It.Grab(), /* out */ Unexpected);
		}	
	}
	else
	{
		while (It.HasMore())
		{
			FLooseParameter Param = It.Peek();
			FSetElementId Place = Order.FindId(Param.Name);
			AddInput(Place.IsValidId() ? &Expected[Place.AsInteger()] : nullptr, Param.Type, It.Grab(), /* out */ Unexpected);
		}
	}

	return FinalizeInputs(Expected, Unexpected, Ctx.Scratch, OutMismatch);
}

FOuter FInstanceUpgrader::Resolve(uint8 Recipient)
{
	FInstanceUpgrader* It = this;
	while (--Recipient)
	{
		It = Outer.Instance;
		check(It);
	}
	return It->Outer;
}

void FInstanceUpgrader::Recompose(FRangeData Out, uint64 Num, uint8 Depth, TConstArrayView<FEnumeratorId> Order, FInnermostEnumType Type)
{
	if (Depth > 1)
	{
		for (FLooseNestedRange Inner : TConstArrayView64<FLooseNestedRange>(Out.Ranges, Num))
		{
			Recompose(Inner.Data, Inner.Num, Depth - 1, Order, Type);
		}
	}
	else
	{
		for (FEnumLooseData& Enum : TArrayView64<FEnumLooseData>(Out.Enums.Loose, Num))
		{
			Enum.Enumerator = Recompose(Enum.Decomposed, Order, Type);
		}
	}
}

// Opt: avoid creating "no-op" recompositions if IsDeclarationOrder(Order)
FEnumeratorData FInstanceUpgrader::Recompose(FEnumIndexData In, TConstArrayView<FEnumeratorId> Order, FInnermostEnumType Type)
{
	check(Type.IsDecomposed);
	if (Type.FlagMode)
	{
		FEnumeratorIdSet Out = {};
		for (uint8 InputIdx : In.Flags)
		{
			FEnumeratorId Enumerator = Order[InputIdx];
			check(Enumerator.Idx < 64); // todo: implement large FEnumeratorIdSet support
			Out.Bits |= uint64(1) << Enumerator.Idx;
		}

		return { .Flags = Out };
	}
	else
	{
		return { .Flat = Order[In.Flat.Index] };
	}
}

static TConstArrayView<FEnumeratorId> GetRecompOrder(const FVersionUpgrades& Version, FOutParameter Output)
{
	const uint16& Jump = Version.GetRecompJumps()[Output.RecompIdx];
	return MakeArrayView(reinterpret_cast<const FEnumeratorId*>(&Jump + Jump), Output.NumInners);
}

void FInstanceUpgrader::Recompose(FOuts Outputs, TConstArrayView<FOutParameter> Params)
{
	checkSlow(Outputs.Num() <= Params.Num());
	for (FLooseMember& Output : FFriend::Edit(Outputs))
	{
		FOutParameter Param = PopFirst(Params);
		
		check(Output.GetType().CompareAll(Param.Type));
		checkf(Output || Param.Type.OptionalParameter, TEXT("Required output '%s' missing while upgrading %s"), 
			*Print(Output.GetName()), *Ctx.Debug.Print(Struct.Id));
		
		if (Param.NumInners)
		{
			if (Param.Type.InnermostType.Kind == EInnermostKind::Enum)
			{
				TConstArrayView<FEnumeratorId> Order = GetRecompOrder(*Current, Param);
				FLooseValue& OutValue = Output.EditTypechecked();
				if (Param.Type.IsRange())
				{
					Recompose(OutValue.Data.Range, OutValue.Meta.Num, Param.Type.NumRanges, Order, Param.Type.InnermostType.Enum);						
				}
				else
				{
					check(OutValue.Meta.Num == Param.NumInners);
					OutValue.Meta.Num = 0; // Reset num enumerator indices
					FEnumLooseData& Enum = OutValue.Data.Enum.Loose;
					Enum.Enumerator = Recompose(Enum.Decomposed, Order, Param.Type.InnermostType.Enum);			
				}
				Output.MarkEnumRecomposed();
			}
			else
			{
				check(Param.Type.InnermostType.Kind == EInnermostKind::Struct);
				Output.MarkStructRecomposed();
			}
		}
	}
}

void FInstanceUpgrader::Run(FOp Op, const FIn* Inputs, uint8 Recipient)
{
	TConstArrayView<FOutParameter> OutParams(Current->GetOutputs() + Op.Outputs.Idx, Op.Outputs.Num);
	FOutputScope Scope(Ctx.OutputStack, OutParams);

	(*Op.Function)(MakeArrayView(Inputs, Op.NumInputs()), Scope.Outputs, Ctx.OpCtx);

	Recompose(Scope.Outputs, OutParams);

	if (Recipient)
	{
		FOuter To = Resolve(Recipient);
		FMatchHelper Helper(*To.Instance->Current);
		for (const FOut& Output : Scope.Outputs)
		{
			if (const FLooseMember& Member = FFriend::Read(Output))
			{
				To.Instance->Give(Member.GetType(), Member.GetValue(), Helper);
			}
		}
	}
	else
	{
		for (const FOut& Output : Scope.Outputs)
		{
			if (const FLooseMember& Member = FFriend::Read(Output))
			{
				Give(Member.GetType(), Member.GetValue());
			}
		}
	}
}

static int64 DebugVersionNo(const FStructUpgrades& Struct, const FVersionUpgrades* Version)
{
	FVersionUpgrades const*const* Pos = Algo::Find(ViewVersions(Struct), Version);
	return Struct.FirstVersion + (Pos - Struct.Versions);
}

inline bool CompareAll(const FLooseMember& Actual, FLooseParameter Expected)
{
	return Actual.GetName() == Expected.Name && Actual.GetType().CompareAll(Expected.Type);
}

// Enables type-checking FIn::OptXyz() functions
static void MakeMissingInputs(TArrayView<FIn> OutInputs, TConstArrayView<FLooseParameter> OptionalInputs)
{
	int32 Idx = 0;
	for (FIn& Input : OutInputs)
	{
		if (FLooseMember& In = FFriend::Edit(Input))
		{
			check(!In.GetType().OptionalParameter || CompareAll(In, OptionalInputs[Idx]));
			Idx += In.GetType().OptionalParameter;
		}
		else
		{
			In = MakeMember(OptionalInputs[Idx++]);
		}
	}
	check(Idx == OptionalInputs.Num());
}

void FInstanceUpgrader::Flush()
{
	check(Current);

	if (FirstPartialOpIdx != 0xFFFF)
	{
		uint32 OpIdx = FirstPartialOpIdx;
		for (FInputDependencies* Deps : PartialInputs.Slice(OpIdx, Current->NumOps - OpIdx))
		{
			if (Deps)
			{
				// todo: recoverable error handling? 
				checkf(Deps->MissingRequired == 0, TEXT("Required input missing for op %d when upgrading version %lld of %s"),
													OpIdx, DebugVersionNo(Struct, Current), *Ctx.Debug.Print(Struct.Id));
				check(Deps->MissingOptional > 0);
				FOp Op =  Current->GetOps()[OpIdx];
				check(Deps->MissingOptional < Op.NumOptionalInputs + (Op.NumRequiredInputs > 0));
				MakeMissingInputs(/* out */ MakeArrayView(Deps->Inputs, Op.NumInputs()), MakeArrayView(Current->GetOptionalInputs() + Op.OptionalInputIdx, Op.NumOptionalInputs));
				Run(Op, Deps->Inputs, /* Recipient, inputs already given to this */ 0);
			}
			++OpIdx;
		}

		FirstPartialOpIdx = 0xFFFF;
	}
	
	Current = nullptr;
}

static void RenameMembers(TArrayView<FLooseMember> Members, const FParameterIndexer& From, FParameterIndexer& To)
{
	for (FLooseMember& Member : Members)
	{
		if (Member || Member.IsMismatch())
		{
			FOptionalMemberId Name = From.Resolve(Member.GetName());
			FParameterId Reindexed = To.Index(Name);
			Member.Rename(Reindexed);
		}
		else
		{
			Member.Rename({});
		}
	}
}

FLooseStruct FInstanceUpgrader::Finish(EMismatch& OutMismatch)
{
	Flush();

	if (Struct.Id != Struct.FinalId)
	{
		RenameMembers(Outcome, Ctx.LooseIds.Get(Struct.Id), Ctx.LooseIds.Get(Struct.FinalId));
	}

	for (const FLooseMember& Member : Outcome)
	{
		if (!Member && !Member.IsOptional() &&
			!Algo::FindByPredicate(Mismatches, [&Member](const FLooseMember& Mismatch) { return Member.GetName() == Mismatch.GetName(); }))
		{
			Mismatches.Add_GetRef(Member).EditTypechecked().Meta.Mismatch = EMismatch::Missing;
		}
	}

	if (Mismatches.Num() > 0)
	{
		OutMismatch = EMismatch::Inners;
		Outcome = Fuse(Outcome, Mismatches, Ctx.Scratch);
	}

	return { Struct.FinalId, Struct.FinalVersion, static_cast<uint16>(Outcome.Num()), Outcome.GetData() };
}

FString FInstanceUpgrader::Print(FParameterId Parameter) const
{
	return Ctx.Debug.Print(Ctx.LooseIds.Get(Struct.Id).Resolve(Parameter));
}

FString FInstanceUpgrader::Print(FLooseType Type) const
{
	return Ctx.Debug.Print(Type);
}

FLooseStruct FInstanceUpgrader::Upgrade(FStructView In, const FLooseSchema& From, const FLooseSchema* To, EMismatch& OutMismatch)
{
	check(Current == GetVersion(Struct, From.Version));
	Expect(To);
	Feed(FLooseMemberReader(From, In, ViewLooseners(Ctx), Ctx.Scratch));
	return Finish(OutMismatch);
}

FLooseStruct FInstanceUpgrader::Upgrade(FLooseStruct In, const FLooseSchema* To, EMismatch& OutMismatch)
{
	check(Current == GetVersion(Struct, In.Version));
	Expect(To);
	Feed(MakeArrayView(In.Members, In.Num));
	return Finish(OutMismatch);
}

FStructMember FInstanceUpgrader::Upgrade(FStructMember In, const FLooseSchema* To, FOuter InOuter)
{
	check(Current == GetVersion(Struct, In.Version));
	check(In.Mismatch == EMismatch::No);

	Outer = InOuter;
	Expect(To);

	switch (In.State)
	{
		case ELooseState::Source:
		{
			const FLooseSchema& SourceSchema = *Ctx.Transformations[In.Schema.Get()].SourceSchema;
			check(SourceSchema.Version == In.Version);
			Feed(FLooseMemberReader(SourceSchema, AsSource(In, Ctx.Batch), ViewLooseners(Ctx), Ctx.Scratch));
			break;
		} 
		case ELooseState::Loose:
			Feed(MakeArrayView(In.Data.Loose, In.Num));
			break;
		case ELooseState::UpgradedLoose: 
			checkf(false, TEXT("Already fully upgraded"));
			break;
		default:
			checkf(false, TEXT("Unset"));
			break;
	}
	
	EMismatch Mismatch = EMismatch::No;
	FLooseStruct Out = Finish(Mismatch);
	// todo: upgrade limit should return ELooseState::Loose
	return { Out.Id, NoId, In.Name, Out.Version, ELooseState::UpgradedLoose, Mismatch, Out.Num, { .Loose = Out.Members } };
}

////////////////////////////////////////////////////////////////////////////////////////////////

struct FImplicitFlags
{
	uint64 operator[](int32 Idx) const { return uint64(1) << Idx; } 
};

struct FImplicitConstants
{
	uint64 operator[](int32 Idx) const { return uint64(Idx); } 
};

template<typename T>
static bool	MatchEnumerators(TConstArrayView<FEnumerator> Declaration, TConstArrayView<FNameId> Names, T Constants)
{
	if (Names.Num() > Declaration.Num())
	{
		return false;
	}

	for (int32 Idx = 0; Idx < Names.Num(); ++Idx)
	{
		FEnumerator Needed = {Names[Idx], Constants[Idx]};
		int32 FoundIdx = Declaration.Find(Needed);
		if (FoundIdx == INDEX_NONE)
		{
			return false;
		}

		// Drop first enumerator if that matched to avoid O(N^2) in normal case, can be optimized better...
		Declaration.RightChopInline(FoundIdx == 0);
	}
	
	return true;
}

// Check if a persisted enum schema matches latest declared schema
static bool	MatchEnum(const FEnumDeclaration& Decl, const FEnumSchema& Schema)
{
	// Flag mode can differ if constants happen to match (unlikely)
	
	if (Decl.Width != Schema.Width)
	{
		return false;
	}

	TConstArrayView<FEnumerator> Enumerators = Decl.GetEnumerators();
	TConstArrayView<FNameId> SchemaNames(Schema.Footer, Schema.Num);
	if (!Schema.ExplicitConstants)
	{
		return Schema.FlagMode	? MatchEnumerators(Enumerators, SchemaNames, FImplicitFlags{})
								: MatchEnumerators(Enumerators, SchemaNames, FImplicitConstants{});
	}
	else switch (Schema.Width)
	{
		case ELeafWidth::B8:	return MatchEnumerators(Enumerators, SchemaNames, GetConstants<uint8>(Schema));
		case ELeafWidth::B16:	return MatchEnumerators(Enumerators, SchemaNames, GetConstants<uint16>(Schema));
		case ELeafWidth::B32:	return MatchEnumerators(Enumerators, SchemaNames, GetConstants<uint32>(Schema));
		case ELeafWidth::B64:	return MatchEnumerators(Enumerators, SchemaNames, GetConstants<uint64>(Schema));
		default:				check(false); return false;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////

// Check if a persisted struct schema matches latest declared schema
static bool	MatchStruct(const FStructDeclaration& Decl, const FStructSchema& Schema, FInnerIds RuntimeIds)
{
	bool bSkipSuperSchema = SkipDeclaredSuperSchema(Schema.Inheritance);
	uint16 NamedInnerSchemas = static_cast<uint16>(Schema.NumInnerSchemas - bSkipSuperSchema);

	// Match header
	uint16 D[4] = { Decl.Version,	Decl.NumMembers,	Decl.NumInnerIds,	Decl.NumInnerRanges };
	uint16 S[4] = { Schema.Version, Schema.NumMembers,	NamedInnerSchemas,	Schema.NumRangeTypes };
	if (BitCast<uint64>(D) != BitCast<uint64>(S))
	{
		return false;
	}

	// Match member names
	if (!Algo::Compare(Decl.GetMemberOrder(), Schema.GetMemberNames()))
	{
		return false;
	}

	// Match member types. Assumes both FStructDeclaration and FStructSchema
	// store range member types directly after member types
	uint32 NumMemberTypes = Decl.NumMembers + Decl.NumInnerRanges;
	if (FMemory::Memcmp(Decl.GetTypes(), Schema.Footer, NumMemberTypes * sizeof(FMemberType)) != 0)
	{
		return false;
	}

	// Match super
	FOptionalDeclId SchemaSuper = Schema.GetSuper() ? ToOptional(RuntimeIds[Schema.GetSuper().Get()]) : NoId;
	if (Decl.Super != SchemaSuper)
	{
		return false;
	}

	// Match inner ids
	if (NamedInnerSchemas > 0)
	{
		const FInnerId* DeclIdIt = Decl.GetInnerIds();
		const FSchemaId* SchemaIdIt = Schema.GetInnerSchemas();
		const FSchemaId* SchemaIdEnd = SchemaIdIt + NamedInnerSchemas;
		TConstArrayView<FMemberType> RangeTypes = Schema.GetRangeTypes();
		uint16 RangeTypeIdx = 0;
		for (FMemberType Member : MakeArrayView(Schema.Footer + bSkipSuperSchema, Schema.NumMembers - bSkipSuperSchema))
		{
			FMemberType Innermost = Member.IsRange() ? GrabInnerRangeTypes(RangeTypes, /* in-out */ RangeTypeIdx).Last() : Member;
			if (IsStructOrEnum(Innermost))
			{
				FInnerId SchemaId = RuntimeIds[*SchemaIdIt++];
				FInnerId DeclId = *DeclIdIt++;
				if (SchemaId != DeclId)
				{
					return false;
				}
			}
		}
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////

struct FNestedMemberType
{
	FMemberType			Type;
	uint32				NumInnerRanges = 0;
	const FMemberType*	InnerRangeTypes = nullptr;
};

// CreateSourceSchema/CreateTargetSchema/FastenStruct helper
struct FMemberTypeIterator
{
	const FMemberType*				TypeIt;
	const FMemberType*				TypeEnd;
	const FMemberType*				RangeIt;
	const FMemberType*				RangeEnd;

	FMemberTypeIterator(TConstArrayView<FMemberType> TypesWithoutSuper, TConstArrayView<FMemberType> RangeTypes)
	: TypeIt(TypesWithoutSuper.GetData())
	, TypeEnd(TypeIt + TypesWithoutSuper.Num())
	, RangeIt(RangeTypes.GetData())
	, RangeEnd(RangeIt + RangeTypes.Num())
	{}

	~FMemberTypeIterator()
	{
		check(TypeIt == TypeEnd);
		check(RangeIt == RangeEnd);
	}

	FNestedMemberType Grab()
	{
		check(TypeIt < TypeEnd);
		FMemberType Type = *TypeIt++;
		return Type.IsRange() ? GrabInnerRanges(Type) : FNestedMemberType{Type};
	}

	FNestedMemberType GrabInnerRanges(FMemberType Type)
	{
		const FMemberType* Out = RangeIt;
		while ((RangeIt++)->IsRange())
		{
			check(RangeIt <= RangeEnd);
		}
		return {Type, static_cast<uint32>(RangeIt - Out), Out};
	}
};

struct FMidMemberType
{
	FMidMemberType(FNestedMemberType In)
	: NumRanges(static_cast<uint8>(In.NumInnerRanges))
	, Innermost(In.NumInnerRanges ? In.InnerRangeTypes[In.NumInnerRanges - 1] : In.Type)
	{
		check(In.NumInnerRanges < MaxRangeNesting);
	}

	uint8							NumRanges;
	FMemberType						Innermost;
};

// CreateSourceSchema helper
struct FSchemaInnermostIdIterator
{
	const FSchemaId*				It;
	FInnerIds						RuntimeIds;
	TConstArrayView<uint16>			StructVersions;
	const TBitArray<>&				EnumFlagModes;

	FInnerId GrabStruct(uint16& OutVersion)
	{
		OutVersion = StructVersions[It->Idx];
		FSchemaId Id = *It++;
		return RuntimeIds[Id];
	}

	FInnerId GrabEnum(EEnumMode& OutMode)
	{
		OutMode = EnumFlagModes[It->Idx - StructVersions.Num()] ? EEnumMode::Flag : EEnumMode::Flat;
		FSchemaId Id = *It++;
		return RuntimeIds[Id];
	}
};

// CreateTargetSchema helper
struct FDeclaredInnermostIdIterator
{
	const FInnerId*					It;
	const IDeclarations&			Declarations;
	
	FInnerId						Grab()
	{
		return *It++;
	}

	FInnerId						GrabStruct(uint16& OutVersion)
	{
		OutVersion = Declarations.Get(It->AsStruct()).Version;
		return Grab();
	}

	FInnerId						GrabEnum(EEnumMode& OutMode)
	{
		OutMode = Declarations.Get(It->AsEnum()).Mode;
		return Grab();
	}
};

static FInnermostType ArithmeticToInnermost(ELeafType Type, ELeafWidth Width)
{
	// todo: checkSlow for illegal type and width combinations
	static constexpr EInnermostKind SInts[4]	= {EInnermostKind::S8, EInnermostKind::S16, EInnermostKind::S32, EInnermostKind::S64};
	static constexpr EInnermostKind UInts[4]	= {EInnermostKind::U8, EInnermostKind::U16, EInnermostKind::U32, EInnermostKind::U64};
	static constexpr EInnermostKind Chars[4]	= {EInnermostKind::UTF8, EInnermostKind::UTF16, EInnermostKind::UTF32, EInnermostKind::UTF32};
	static constexpr EInnermostKind Floats[4]	= {EInnermostKind::F32, EInnermostKind::F32, EInnermostKind::F32, EInnermostKind::F64};

	switch (Type)
	{
	case ELeafType::Bool:		return { .Kind = EInnermostKind::Bool };
	case ELeafType::IntS:		return { .Kind = SInts[(uint8)Width] };
	case ELeafType::IntU:		return { .Kind = UInts[(uint8)Width] };
	case ELeafType::Float:		return { .Kind = Floats[(uint8)Width] };
	case ELeafType::Hex:		return { .Kind = UInts[(uint8)Width] };
	case ELeafType::Unicode:	return { .Kind = Chars[(uint8)Width] };
	default:					check(false); return {};
	}
}

// CreateSourceSchema/CreateTargetSchema helper
template <typename InnermostIterator>
FInnermostType GrabInnermost(InnermostIterator& InnermostIt, FMemberType In, FOptionalInnerId& OutId, uint16& OutVersion)
{
	if (In.IsStruct())
	{
		FStructType InStruct = In.AsStruct();
		if (InStruct.IsDynamic)
		{
			return { .Struct = { EInnermostKind::Struct, /* IsDynamic */ 1, InStruct.IsSuper } };	
		}
		OutId = ToOptional(InnermostIt.GrabStruct(OutVersion));
		return { .Struct = { EInnermostKind::Struct, /* IsDynamic */ 0, InStruct.IsSuper } };
	}

	FUnpackedLeafType Leaf = In.AsLeaf();
	if (Leaf.Type == ELeafType::Enum)
	{
		EEnumMode Mode;
		OutId = ToOptional(InnermostIt.GrabEnum(/* out */ Mode));
		return { .Enum = { EInnermostKind::Enum, Mode == EEnumMode::Flag } };
	}

	return ArithmeticToInnermost(Leaf.Type, Leaf.Width);
}

// CreateSourceSchema helper
struct FSourceLooseTypeIterator
{
	FSourceLooseTypeIterator(const FStructSchema& Struct, const FSourceSchemas& Schemas)
	: FSourceLooseTypeIterator(Struct, Schemas.RuntimeIds, Schemas.StructVersions, Schemas.EnumFlagModes)
	{}

	FSourceLooseTypeIterator(const FStructSchema& Struct, FInnerIds RuntimeIds, TConstArrayView<uint16> Versions, const TBitArray<>& FlagModes)
	: SkipSuper(SkipDeclaredSuperSchema(Struct.Inheritance))
	, TypeIt(Struct.GetMemberTypes().LeftChop(SkipSuper), Struct.GetRangeTypes())
	, InnerIdIt({Struct.GetInnerSchemas() + SkipSuper, RuntimeIds, Versions, FlagModes})
	{}

	const uint8					SkipSuper;
	FMemberTypeIterator			TypeIt;
	FSchemaInnermostIdIterator	InnerIdIt;

	FLooseType					Grab()
	{
		FMidMemberType Mid = TypeIt.Grab();
		FLooseType Out = { .NumRanges = Mid.NumRanges };
		Out.InnermostType = GrabInnermost(/* in-out */ InnerIdIt, Mid.Innermost, Out.InnermostId, Out.InnermostVersion);
		return Out;
	}
};

// CreateTargetSchema/FastenStruct helper
struct FTargetLooseTypeIterator
{
	FTargetLooseTypeIterator(const FStructDeclaration& Struct, const IDeclarations& Types)
	: TypeIt(MakeArrayView(Struct.GetTypes(), Struct.NumMembers), 
			 MakeArrayView(Struct.GetInnerRangeTypes(), Struct.NumInnerRanges))
	, InnerIdIt({Struct.GetInnerIds(), Types})
	{}

	FMemberTypeIterator				TypeIt;
	FDeclaredInnermostIdIterator	InnerIdIt;

	FLooseType						GrabLoose()
	{
		FMidMemberType Mid = TypeIt.Grab();
		FLooseType Out = { .NumRanges = Mid.NumRanges };
		Out.InnermostType = GrabInnermost(/* in-out */ InnerIdIt, Mid.Innermost, Out.InnermostId, Out.InnermostVersion);
		return Out;
	}

	FMemberSchema					GrabSchema()
	{
		FNestedMemberType Mid = TypeIt.Grab();
		FMemberType InnerRangeType = Mid.InnerRangeTypes ? Mid.InnerRangeTypes[0] : Mid.Type;
		FMemberType InnermostType = Mid.NumInnerRanges ? Mid.InnerRangeTypes[Mid.NumInnerRanges - 1] : Mid.Type;
		FOptionalInnerId InnermostId = IsStructOrEnum(InnermostType) ? ToOptional(InnerIdIt.Grab()) : NoId;
		return { Mid.Type, InnerRangeType, static_cast<uint16>(Mid.NumInnerRanges), InnermostId, Mid.InnerRangeTypes };
	}
};

// FBatchUpgrader::Upgrade helper
struct FBatchRun
{
	FBatchRun(const FHistory& Upgrades, FScratchAllocator& InScratch, TConstArrayView<FInnerId> InnerIds)
	: RuntimeIds{{InnerIds}}
	, History(Upgrades)
	, Scratch(InScratch)
	, Debug(Upgrades.Ids, History.LooseIds)
	{
		TlsDebug = &Debug;
	}
	~FBatchRun()
	{
		TlsDebug = nullptr;
	}

	FInnerIds						RuntimeIds;
	FTransformations				Transformations;
	const FHistory&					History;
	FScratchAllocator&				Scratch;
	FSchemaBatchId					SourceBatch;
	FDebugPrinter					Debug;

	void							CreateTransformations(const TBitArray<>& MatchingSchemas, const FSchemaBatch& Batch);
	FUpgradedInstance				UpgradeOrRebuild(FStructView Source);
	FUpgradedInstance				UpgradeStruct(FStructView Source, FTransformation Xform) const;
	FBuiltStruct*					RebuildStruct(FStructView Source) const;
	FBuiltStruct*					RebuildStruct(FStructView Source, FDeclId Id) const;
	void							RebuildMembers(FMemberBuilder& Out, FStructView Source, FDeclId Id) const;
	FTypedValue						RebuildValue(FMemberReader& It) const;
	FTypedValue						RebuildValue(FLeafView Leaf) const;
	FTypedValue						RebuildValue(FMemberType Type, FRangeView Range) const;
	FTypedValue						RebuildValue(FStructView Struct) const;
	FBuiltRange*					RebuildRange(FRangeView Range) const;
	FBuiltRange*					RebuildLeaves(FLeafRangeView Leaves) const;
	FBuiltRange*					RebuildRanges(FNestedRangeView Ranges) const;
	FBuiltRange*					RebuildStructs(FStructRangeView Structs) const;
	FBuiltRange*					RebuildStructs(FStructRangeView Structs, FDeclId Id) const;
	FBuiltStruct*					FastenStruct(FLooseStruct Loose) const;
	void							FastenMembers(FMemberBuilder& Out, TConstArrayView<FLooseMember> Members, const FParameterIndexer& Names, const FStructDeclaration& Declaration) const;
	FBuiltValue						FastenMember(FLooseMember Member) const;
	uint64							FastenEnum(FEnumMember Enum) const;
	FBuiltStruct*					FastenStruct(FStructMember Struct) const;
	FBuiltRange*					FastenRange(FRangeMember Range) const;
	FBuiltRange*					FastenRange(FLooseType Type, FLooseMetadata Meta, FRangeData Data) const;
	FBuiltRange*					FastenRanges(FLooseType Type, FLooseMetadata Meta, TConstArrayView64<FLooseNestedRange> Ranges) const;
	FBuiltRange*					FastenStructs(FDeclId Id, FLooseMetadata Meta, FStructRange Structs) const;
	FBuiltRange*					FastenEnums(FEnumId Id, FInnermostEnumType Type, FLooseMetadata Meta, FRangeData Data) const;
	FBuiltRange*					FastenArithmetics(EInnermostKind Kind, uint64 Num, const void* Data) const;
	FBuiltRange*					FastenLeaves(uint64 Num, FMemoryView Data) const;
};

static FLooseSchema* CreateSchema(FScratchAllocator& Scratch, uint16 Version, uint16 NumMembers)
{
	const uint32 Size = offsetof(FLooseSchema, Members) + NumMembers * sizeof(FLooseParameter);
	FLooseSchema Header = {.Version = Version, .NumMembers = NumMembers};
	return new (Scratch.Allocate(Size, alignof(FLooseSchema))) FLooseSchema(Header);
}

static FLooseSchema* CreateSourceSchema(const FStructSchema& Schema, const FSourceSchemas& Schemas, FParameterIndexer& ParamIds, FScratchAllocator& Scratch)
{
	FLooseSchema* Out = CreateSchema(Scratch, Schema.Version, Schema.NumMembers);
	const FMemberId* NameIt = Schema.GetMemberNames().GetData();
	FSourceLooseTypeIterator TypeIt(Schema, Schemas);
	for (FLooseParameter& Param : MakeArrayView(Out->Members, Schema.NumNames()))
	{
		Param.Name = ParamIds.Index(*NameIt++);
		Param.Type = TypeIt.Grab();
	}
	return Out;
}

static FLooseSchema* CreateTargetSchema(const FStructDeclaration& Struct, const IDeclarations& Types, FParameterIndexer& ParamIds)
{
	uint16 Num = Struct.NumMembers;
	const uint32 Size = offsetof(FLooseSchema, Members) + Num * sizeof(FLooseParameter);
	FLooseSchema Header = {.Version = Struct.Version, .NumMembers = Num};
	FLooseSchema* Out = new (FMemory::Malloc(Size, alignof(FLooseSchema))) FLooseSchema(Header);

	TArrayView<FLooseParameter> Parameters(Out->Members, Num);
	if (FOptionalDeclId Super = Struct.Super)
	{
		uint16 SuperVersion = Types.Get(Super.Get()).Version;
		Parameters[0].Name = ParamIds.IndexSuper();
		// todo: remove or set FInnermostStructType::IsSuper
		Parameters[0].Type = FLooseType{ 0, 0, InnermostStaticStruct, SuperVersion, ToOptionalInner(Super) };
		Parameters.RightChopInline(1);
	}

	const FMemberId* NameIt = Struct.GetMemberOrder().GetData();
	FTargetLooseTypeIterator TypeIt(Struct, Types);
	for (FLooseParameter& Parameter : Parameters)
	{
		Parameter.Name = ParamIds.Index(*NameIt++);
		Parameter.Type = TypeIt.GrabLoose();
	}

	return Out;
}

static TArray<uint16> ExtractVersions(TSchemaRange<const FStructSchema> Schemas)
{
	TArray<uint16> Out;
	Out.Reserve(Schemas.Num());
	for (const FStructSchema& Schema : Schemas)
	{
		Out.Add(Schema.Version);
	}
	return Out;
}

static TBitArray<> ExtractFlagModes(TSchemaRange<const FEnumSchema> Schemas)
{
	TBitArray<> Out;
	Out.Reserve(Schemas.Num());
	for (const FEnumSchema& Schema : Schemas)
	{
		Out.Add(Schema.FlagMode);
	}
	return Out;
}

static void AddEnumInputSchemas(TBitArray<>& OutEnumSchemaInputs, const TSet<FEnumId>& EnumLookup, const FVersionUpgrades& Version)
{
	for (FMemberMatch Match : MakeArrayView(Version.GetMatches(), Version.NumMatches))
	{
		if (Match.Type.InnermostIsEnum())
		{
			FEnumId Id = Match.Type.InnermostId.Get().AsEnum();
			FSetElementId Idx = EnumLookup.FindId(Id);
			if (Idx.IsValidId() && Idx.AsInteger() < OutEnumSchemaInputs.Num())
			{
				OutEnumSchemaInputs[Idx.AsInteger()] = true;
			}
		}
	}
}

static void AddInnerIdOutputs(TSet<FDeclId>& OutStructs, TSet<FEnumId>& OutEnums, const FVersionUpgrades& Version)
{
	const FOutParameter* Outputs = Version.GetOutputs();
	for (FOp Op : MakeArrayView(Version.GetOps(), Version.NumOps))
	{
		for (FOutParameter Output : MakeArrayView(Outputs + Op.Outputs.Idx, Op.Outputs.Num))
		{
			if (Output.Type.InnermostId)
			{
				FInnerId Inner = Output.Type.InnermostId.Get();
				if (Output.Type.InnermostIsStruct())
				{
					OutStructs.Emplace(Inner.AsStruct());
				}
				else
				{
					check(Output.Type.InnermostIsEnum());
					OutEnums.Add(Inner.AsEnum());
				}
			}
		}
	}
}


static bool LoosenerRequired(const FFlatEnumLoosener& Loosener)
{
	for (TPair<uint64, FEnumeratorId> Pair : Loosener.ConstantNames)
	{
		if (Pair.Key != Pair.Value.Idx)
		{
			return true;
		}
	}
	return false;
}

static bool LoosenerRequired(const FFlagEnumLoosener& Loosener, uint32 Num)
{
	for (uint32 Idx = 0; Idx < Num; ++Idx)
	{
		if (Loosener.FlagNames[Idx].Idx != Idx)
		{
			return true;
		}
	}
	return false;
}

template<typename T>
static FEnumLoosener CreateFlatLoosener(const FEnumSchema& Schema, bool bNeeded, FEnumeratorIndexer& Ids, FScratchAllocator& Scratch)
{
	FFlatEnumLoosener Out;
	Out.ConstantNames.Reserve(Schema.Num);
	const FNameId* Names = Schema.Footer;
	if (const T* Constants = Schema.ExplicitConstants ? GetConstants<T>(Schema).GetData() : nullptr)
	{
		for (uint32 Idx = 0, Num = Schema.Num; Idx < Num; ++Idx)
		{
			Out.ConstantNames.Emplace(Constants[Idx], Ids.Index(Names[Idx]));
		}
	}
	else
	{
		for (uint32 Idx = 0, Num = Schema.Num; Idx < Num; ++Idx)
		{
			Out.ConstantNames.Emplace(Idx, Ids.Index(Names[Idx]));
		}
	}

	return LoosenerRequired(Out) ? FEnumLoosener(bNeeded, new (Scratch) FFlatEnumLoosener(MoveTemp(Out))) : FEnumLoosener();
}

template<typename T>
static FEnumLoosener CreateFlagLoosener(const FEnumSchema& Schema, bool bNeeded, FEnumeratorIndexer& Ids, FScratchAllocator& Scratch)
{
	FFlagEnumLoosener Out = {};
	if (const T* Constants = Schema.ExplicitConstants ? GetConstants<T>(Schema).GetData() : nullptr)
	{
		for (uint32 Idx = 0, Num = Schema.Num; Idx < Num; ++Idx)
		{
			check(FMath::CountBits64(Constants[Idx]) == 1);
			uint64 BitIdx = FMath::FloorLog2NonZero_64(Constants[Idx]);
			Out.FlagNames[BitIdx] = Ids.Index(Schema.Footer[Idx]);
		}
	}
	else
	{
		for (uint32 Idx = 0, Num = Schema.Num; Idx < Num; ++Idx)
		{
			Out.FlagNames[Idx] = Ids.Index(Schema.Footer[Idx]);
		}
	}

	return LoosenerRequired(Out, Schema.Num) ? FEnumLoosener(bNeeded, *new (Scratch) FFlagEnumLoosener(Out)) : FEnumLoosener();
}

static FEnumLoosener CreateEnumLoosener(const FEnumSchema& Schema, bool bNeeded, FEnumeratorIndexer& Ids, FScratchAllocator& Scratch)
{
	switch (Schema.Width)
	{
		case ELeafWidth::B8:	return Schema.FlagMode	? CreateFlagLoosener<uint8 >(Schema, bNeeded, Ids, Scratch)
														: CreateFlatLoosener<uint8 >(Schema, bNeeded, Ids, Scratch);
		case ELeafWidth::B16:	return Schema.FlagMode	? CreateFlagLoosener<uint16>(Schema, bNeeded, Ids, Scratch)
														: CreateFlatLoosener<uint16>(Schema, bNeeded, Ids, Scratch);
		case ELeafWidth::B32:	return Schema.FlagMode	? CreateFlagLoosener<uint32>(Schema, bNeeded, Ids, Scratch)
														: CreateFlatLoosener<uint32>(Schema, bNeeded, Ids, Scratch);
		case ELeafWidth::B64:	return Schema.FlagMode	? CreateFlagLoosener<uint64>(Schema, bNeeded, Ids, Scratch)
														: CreateFlatLoosener<uint64>(Schema, bNeeded, Ids, Scratch);
		default:				check(false); return FEnumLoosener();
	}
}

// Contiguous constants will match contiguous declaration order of FEnumeratorId indices
static bool RequiresFastening(TConstArrayView<FEnumerator> Enumerators, EEnumMode Mode)
{
	if (Mode == EEnumMode::Flat)
	{
		for (int32 Idx = 0; Idx < Enumerators.Num(); ++Idx)
		{
			if (Enumerators[Idx].Constant != uint32(Idx))
			{
				return true;
			}
		}
	}
	else
	{
		for (int32 Idx = 0; Idx < Enumerators.Num(); ++Idx)
		{
			if (Enumerators[Idx].Constant != (uint64(1) << Idx))
			{
				return true;
			}
		}
	}

	return false;
}

static FEnumFastener CreateEnumFastener(const FEnumDeclaration* Decl, FScratchAllocator& Scratch)
{
	return { Decl && RequiresFastening(Decl->GetEnumerators(), Decl->Mode) ? Decl : nullptr };
}

static const FLooseSchema& ObtainSourceSchema(FBatchContext& Ctx, FDeclId Id, FStructSchemaId SchemaId)
{
	const FTransformation& Xform = Ctx.Transformations[Id];
	check(!Xform.Upgrades);
	const FLooseSchema*& Cache = Xform.SourceSchema;
	if (!Cache)
	{
		const FSourceSchemas& Schemas = Ctx.Transformations.Schemas;
		const FStructSchema& Schema = GetStructSchemas(*Schemas.Batch)[SchemaId.Idx];
		Cache = CreateSourceSchema(Schema, Schemas, Ctx.LooseIds.Get(Id), Ctx.Scratch);
	}

	return *Cache;
}

void FBatchRun::CreateTransformations(const TBitArray<>& MatchingSchemas, const FSchemaBatch& Batch)
{
	FLooseIndexers& LooseIds = History.LooseIds;
	const IDeclarations& Declarations = History.Latest;
	const TMap<FDeclId, FStructUpgradesPtr>& CompiledUpgrades = History.CompiledUpgrades;
	const TSchemaRange<const FStructSchema> StructSchemas = GetStructSchemas(Batch);
	const TSchemaRange<const FEnumSchema> EnumSchemas = GetEnumSchemas(Batch);
	Transformations.Schemas = { uint32(StructSchemas.Num()), &Batch, RuntimeIds, ExtractVersions(StructSchemas),  ExtractFlagModes(EnumSchemas) }; 

	TArray<FTransformation>& StructXforms = Transformations.Structs;
	TSet<FDeclId>& StructLookup = Transformations.StructLookup;
	TArray<FEnumTransformation>& EnumXforms = Transformations.Enums;
	TSet<FEnumId>& EnumLookup = Transformations.EnumLookup;
	StructXforms.SetNum(StructSchemas.Num());
	StructLookup.Reserve(StructSchemas.Num());
	EnumXforms.SetNum(EnumSchemas.Num());
	EnumXforms.Reserve(EnumSchemas.Num());

	// Create initial struct transforms
	for (uint32 Idx = 0; int32(Idx) < StructSchemas.Num(); ++Idx)
	{
		FDeclId Id = RuntimeIds[FStructSchemaId{Idx}];
		if (!MatchingSchemas[Idx])
		{
			FParameterIndexer& Ids = LooseIds.Get(Id);
			FTransformation& Xform = StructXforms[Idx];
			Xform.SourceSchema = CreateSourceSchema(StructSchemas[Idx], Transformations.Schemas, Ids, Scratch);
			Xform.Upgrades = CompiledUpgrades.FindChecked(Id).Get();
			Xform.TargetSchema = History.LatestSchemas.Obtain(Id, Declarations, Ids);
		}

		// Ensure lookup add order follow source schema id index
		StructLookup.Add(Id);
	}

	// Create initial enum transforms
	for (uint32 Idx = StructSchemas.Num(); const FEnumSchema& Schema : EnumSchemas)
	{
		FEnumId Id = RuntimeIds[FEnumSchemaId{Idx}];
		if (!MatchingSchemas[Idx])
		{
			FEnumeratorIndexer& Ids = LooseIds.Get(Id);
			FEnumTransformation& Xform = EnumXforms[EnumLookup.Num()];
			Xform.Source = CreateEnumLoosener(Schema, /* bNeeded */ true, Ids, Scratch);
			Xform.Loose = CreateEnumFastener(Declarations.Find(Id), Scratch);
		}
	
		// Ensure lookup add order follow source schema id index
		EnumLookup.Add(Id);
		++Idx;
	}
	
	// Find renames and inner types outputted by upgrade ops,
	// which aren't part of schema batch
	TBitArray<> EnumInputSchemas(false, EnumSchemas.Num());
	TArrayView<FTransformation> Unscanned = StructXforms;
	while (Unscanned.Num() > 0)
	{
		for (const FTransformation& Xform : Unscanned)
		{
			if (const FStructUpgrades* Upgrades = Xform.Upgrades)
			{
				// todo: opt - no need to scan older versions
				// uint16 FirstVersion = Xform.SourceSchema->Version;
				// for (const FVersionUpgrades* Version : ViewVersionsFrom(*Upgrades, FirstVersion))
				for (const FVersionUpgrades* Version : ViewVersions(*Upgrades))
				{
					AddEnumInputSchemas(/* out */ EnumInputSchemas, EnumLookup, *Version);
					AddInnerIdOutputs(/* out */ StructLookup, /* out */ EnumLookup, *Version);
				}

				if (Upgrades->Id != Upgrades->FinalId)
				{
					StructLookup.Add(Upgrades->FinalId);
				}
			}
		}

		// Create FTransformations and target schemas for missing structs,
		// regardless if upgrades exist so operator[](FDeclId) can return a reference
		int32 Idx = StructXforms.Num();
		StructXforms.SetNum(StructLookup.Num());
		Unscanned = MakeArrayView(StructXforms).RightChop(Idx);
		for (FTransformation& Xform : Unscanned)
		{
			FDeclId Id = StructLookup.Get(FSetElementId::FromInteger(Idx));			
			const FStructUpgradesPtr* Upgrades = CompiledUpgrades.Find(Id);
			Xform.Upgrades = Upgrades ? Upgrades->Get() : nullptr;
			Xform.TargetSchema = History.LatestSchemas.Obtain(Id, Declarations, LooseIds.Get(Id));
			++Idx;
		}
	}

	// Create source looseners for all input enums
	TBitArray<> MissingLooseners;
	MissingLooseners.AddRange(MatchingSchemas, EnumSchemas.Num(), StructSchemas.Num());
	MissingLooseners.CombineWithBitwiseAND(EnumInputSchemas, EBitwiseOperatorFlags::MinSize);
	for (TConstSetBitIterator<> It(MissingLooseners); It; ++It)
	{
		int32 Idx = It.GetIndex();
		FEnumId Id = EnumLookup.Get(FSetElementId::FromInteger(Idx));
		FEnumTransformation& Xform = EnumXforms[Idx];
		check(!Xform.Source.Exists());
		Xform.Source = CreateEnumLoosener(EnumSchemas[Idx], /* bNeeded */ false, LooseIds.Get(Id), Scratch);
		// Create fastener too, only needed for output enums
		Xform.Loose = CreateEnumFastener(Declarations.Find(Id), Scratch);
	}

	// Create fasteners for enum outputs that lack schema in the batch
	EnumXforms.SetNum(EnumLookup.Num());
	TArrayView<FEnumTransformation> OutputEnumXforms = MakeArrayView(EnumXforms).RightChop(EnumSchemas.Num());
	for (int32 Idx = EnumSchemas.Num(); FEnumTransformation& Xform : OutputEnumXforms)
	{
		FEnumId Id = EnumLookup.Get(FSetElementId::FromInteger(Idx));
		FEnumeratorIndexer& Ids = LooseIds.Get(Id);
		Xform.Loose = CreateEnumFastener(Declarations.Find(Id), Scratch);
		++Idx;
		// todo: cache fasteners in History.LatestSchemas?
	}
}

static FLooseType IndexObsolete(FLiteralIndexerBase& Ids)
{
	FStructId Obsolete = Ids.IndexStruct(FType(NoId, FTypenameId(Ids.IndexTypenameLiteral("__obsolete__"))));
	return { .OptionalParameter = 1, .InnermostType = InnermostStaticStruct, .InnermostVersion = 0xBAAD, .InnermostId = FInnerId(Obsolete) };
}

// @param In isn't mismatched
static FLooseStruct ReorderAsDeclaration(FLooseStruct In, const FLooseSchema& Declared, EMismatch& OutMismatch, FScratchAllocator& Scratch, FDebugPrinter Debug)
{
	// opt: check if In members have consecutive names, if so type-check and return In
	//		could also compact and only return IsSet members and reuse the existing In allocation

	FLooseMember* OutIt = static_cast<FLooseMember*>(Scratch.AllocateZeroed(Declared.NumMembers * sizeof(FLooseMember), alignof(FLooseMember)));
	for (FLooseParameter Param : Declared.GetMembers())
	{
		new (OutIt++) FLooseMember(MakeMember(Param));
	}

	TArray<FLooseMember> Mismatches;
	TArrayView<FLooseMember> Out(OutIt - Declared.NumMembers, Declared.NumMembers);
	for (FLooseMember Src : MakeArrayView(In.Members, In.Num))
	{
		if (Src)
		{
			FParameterId Name = Src.GetName();
			FLooseMember* Dst = Name.Idx < Out.Num() ? &Out[Name.Idx] : nullptr;
			if (Dst && Src.GetType().Compare(Dst->GetType()) && !*Dst)
			{
				Dst->SetTypechecked(Src.GetValue());
			}
			else
			{
				Src.EditTypechecked().SetFirstMismatch(Dst ? (*Dst ? EMismatch::Duplicate : EMismatch::Type): EMismatch::Surplus);
				Mismatches.Add(Src);
			}
		}
	}

	if (Mismatches.Num())
	{
		// Report first mismatch type
		OutMismatch = Mismatches[0].GetValue().Meta.Mismatch;
		check(OutMismatch != EMismatch::No);

		int32 NumAll = Declared.NumMembers + Mismatches.Num();
		FLooseMember* All = static_cast<FLooseMember*>(Scratch.AllocateZeroed(NumAll * sizeof(FLooseMember), alignof(FLooseMember)));
		FMemory::Memcpy(All, Out.GetData(), Out.NumBytes());
		FMemory::Memcpy(All + Declared.NumMembers, Mismatches.GetData(), Mismatches.NumBytes());
		Out = MakeArrayView(All, NumAll);
	}

	return { In.Id, In.Version, static_cast<uint16>(Out.Num()), Out.GetData() };
}

FUpgradedInstance FBatchRun::UpgradeStruct(FStructView Source, FTransformation Xform) const
{
	FBatchContext Ctx = { Transformations, Scratch, History.Ids, History.LooseIds, IndexObsolete(History.Ids), Source.Schema.Batch, {Scratch}, FContext(Scratch), Debug };
	FInstanceUpgrader SourceUpgrader(Ctx, *Xform.Upgrades, Xform.SourceSchema->Version);
	EMismatch Mismatch = EMismatch::No;
	FLooseStruct Upgraded = SourceUpgrader.Upgrade(Source, *Xform.SourceSchema, Xform.TargetSchema, /* out */ Mismatch);

	// Handle renamed structs
	while (Upgraded.Id != Xform.Upgrades->Id && Mismatch == EMismatch::No)
	{
		Xform = Transformations[Upgraded.Id];
		if (Xform.Upgrades)
		{
			FInstanceUpgrader Upgrader(Ctx, *Xform.Upgrades, Upgraded.Version);
			Upgraded = Upgrader.Upgrade(Upgraded, Xform.TargetSchema, /* out */ Mismatch);
		}
		else
		{
			Upgraded = ReorderAsDeclaration(Upgraded, *Xform.TargetSchema, /* out */ Mismatch, Scratch, Ctx.Debug);
			break;
		}
	}

	const FBuiltStruct* Fixed = Mismatch == EMismatch::No ? FastenStruct(Upgraded) : nullptr;
	return { Upgraded.Id, MakeArrayView(Upgraded.Members, Upgraded.Num), Fixed };
}

FUpgradedInstance FBatchRun::UpgradeOrRebuild(FStructView Source)
{
	SourceBatch = Source.Schema.Batch;
	FStructSchemaId SchemaId = Source.Schema.Id;
	FDeclId Id = RuntimeIds[SchemaId];
	FTransformation Xform = Transformations[Source.Schema.Id];
	return Xform.Upgrades ? UpgradeStruct(Source, Xform) : FUpgradedInstance{ Id, {}, RebuildStruct(Source, Id) };
}

static FUnpackedLeafType ToLeafType(EInnermostKind Arithmetic)
{
	switch (Arithmetic)
	{
	case EInnermostKind::Bool:		return {ELeafType::Bool, ELeafWidth::B8};
	case EInnermostKind::S8:		return {ELeafType::IntS, ELeafWidth::B8};
	case EInnermostKind::S16:		return {ELeafType::IntS, ELeafWidth::B16};
	case EInnermostKind::S32:		return {ELeafType::IntS, ELeafWidth::B32};
	case EInnermostKind::S64:		return {ELeafType::IntS, ELeafWidth::B64};
	case EInnermostKind::U8:		return {ELeafType::IntU, ELeafWidth::B8};
	case EInnermostKind::U16:		return {ELeafType::IntU, ELeafWidth::B16};
	case EInnermostKind::U32:		return {ELeafType::IntU, ELeafWidth::B32};
	case EInnermostKind::U64:		return {ELeafType::IntU, ELeafWidth::B64};
	case EInnermostKind::F32:		return {ELeafType::Float, ELeafWidth::B32};
	case EInnermostKind::F64:		return {ELeafType::Float, ELeafWidth::B64};
	case EInnermostKind::UTF8:		return {ELeafType::Unicode, ELeafWidth::B8};
	case EInnermostKind::UTF16:		return {ELeafType::Unicode, ELeafWidth::B16};
	case EInnermostKind::UTF32:		return {ELeafType::Unicode, ELeafWidth::B32};
	default:						check(false); return {ELeafType::Bool, ELeafWidth::B8};
	}
}

static ELeafWidth ToLeafWidth(EInnermostKind Arithmetic)
{
	return ToLeafType(Arithmetic).Width;
}

static bool CompareLeaf(ELeafType Expected, EInnermostKind Actual)
{
	switch (Actual)
	{
		case EInnermostKind::Enum:		return Expected == ELeafType::Enum;
		case EInnermostKind::Bool:		return Expected == ELeafType::Bool;
		case EInnermostKind::S8:		return Expected == ELeafType::IntS;
		case EInnermostKind::S16:		return Expected == ELeafType::IntS;
		case EInnermostKind::S32:		return Expected == ELeafType::IntS;
		case EInnermostKind::S64:		return Expected == ELeafType::IntS;
		case EInnermostKind::U8:		return Expected == ELeafType::IntU;
		case EInnermostKind::U16:		return Expected == ELeafType::IntU;
		case EInnermostKind::U32:		return Expected == ELeafType::IntU;
		case EInnermostKind::U64:		return Expected == ELeafType::IntU;
		case EInnermostKind::F32:		return Expected == ELeafType::Float;
		case EInnermostKind::F64:		return Expected == ELeafType::Float;
		case EInnermostKind::UTF8:		return Expected == ELeafType::Unicode;
		case EInnermostKind::UTF16:		return Expected == ELeafType::Unicode;
		case EInnermostKind::UTF32:		return Expected == ELeafType::Unicode;	
		default:						return false;
	}	
}

static bool CompareInnermost(FMemberType Expected, EInnermostKind Actual)
{
	return Expected.IsStruct() ? Actual == EInnermostKind::Struct : CompareLeaf(Expected.AsLeaf().Type, Actual);
}

static void CheckSchema(FMemberSchema Expected, FLooseType Actual)
{
	check(Expected.InnerSchema == Actual.InnermostId);
	check(Expected.NumInnerRanges == Actual.NumRanges);
	check(CompareInnermost(Expected.GetInnermostType(), Actual.InnermostType.Kind));
}

FBuiltStruct* FBatchRun::FastenStruct(FLooseStruct Struct) const
{
	const FParameterIndexer& Names = History.LooseIds.Get(Struct.Id);
	const FStructDeclaration& Declaration = History.Latest.Get(Struct.Id);

	FMemberBuilder Out;
	FastenMembers(Out, MakeArrayView(Struct.Members, Struct.Num), Names, Declaration);
	return Out.BuildAndReset(Scratch, Declaration, Debug); 
}

void FBatchRun::FastenMembers(FMemberBuilder& Out, TConstArrayView<FLooseMember> Members, const FParameterIndexer& Names, const FStructDeclaration& Declaration) const
{
	// Fasten super
	if (Names.Resolve(Members[0].GetName()) == NoId)
	{
		FStructMember Super = AsStruct(Members[0]);
		Members.RightChopInline(1);
		Out.AddSuperStruct(Super.Id, FastenStruct(Super));
	}

	// Fasten named members
	FTargetLooseTypeIterator DeclIt(Declaration, History.Latest);
	for (FLooseMember Member : Members)
	{
		FMemberSchema Declared = DeclIt.GrabSchema();
		CheckSchema(Declared, Member.GetType());
		if (Member)
		{
			// todo: decide pass down Declared.GetInnermostType() to get enum width or not
			// maybe pass down innermost leaf size, 0 if not leaf
			FBuiltValue Value = FastenMember(Member /*, Declared.GetInnermostType()*/);
			FMemberId Name = Names.Resolve(Member.GetName()).Get();
			Out.Add(Name, {Declared, Value});
		}
	}
}

FBuiltValue FBatchRun::FastenMember(FLooseMember Member) const
{
	FLooseType Type = Member.GetType();
	if (Type.IsRange())
	{
		return { .Range = FastenRange(AsRange(Member)) };
	}
	else if (Type.IsStruct())
	{
		return { .Struct = FastenStruct(AsStruct(Member)) };
	}
	else if (Type.IsEnum())
	{
		return { .Leaf = FastenEnum(AsEnum(Member)) };
	}
	else
	{
		return { .Leaf = Member.GetValue().Data.Arithmetic };
	}
}

FBuiltStruct* FBatchRun::FastenStruct(FStructMember Struct) const
{
	check(Struct.Mismatch == EMismatch::No);

	if (Struct.State == ELooseState::Source)
	{
		return RebuildStruct(AsSource(Struct, SourceBatch));
	}
	
	FLooseStruct Loose = { .Id = Struct.Id, .Version = Struct.Version, .Num = IntCastChecked<uint16>(Struct.Num), .Members = Struct.Data.Loose };
	if (Struct.State == ELooseState::Loose)
	{
		FTransformation Xform = Transformations[Struct.Id];
		checkf(!Xform.Upgrades, TEXT("Struct should've been upgraded already"));
		check(Xform.TargetSchema);
		Loose = ReorderAsDeclaration(Loose, *Xform.TargetSchema, /* out */ Struct.Mismatch, Scratch, Debug);
		check(Struct.Mismatch == EMismatch::No);
	}
		
	return FastenStruct(Loose);
}

uint64 FBatchRun::FastenEnum(FEnumMember Enum) const
{
	if (Enum.State == ELooseState::Source)
	{
		return Enum.Data.Source;
	}
	//checkf(Enum.State == ELooseState::UpgradedLoose, TEXT("Enum not fully upgraded"));
	FEnumFastener Fastener = Transformations[Enum.Id].Loose;
	const FEnumeratorIndexer& Indexer = History.LooseIds.Get(Enum.Id); // opt: No need to lookup if no fastener...
	return Enum.FlagMode ? Fasten(Fastener, Enum.Data.Loose.Enumerator.Flags, Indexer)
						 : Fasten(Fastener, Enum.Data.Loose.Enumerator.Flat, Indexer);
}

FBuiltRange* FBatchRun::FastenRange(FRangeMember Range) const
{
	return FastenRange(Range.Type, Range.Meta, Range.Data);
}

FBuiltRange* FBatchRun::FastenRange(FLooseType Type, FLooseMetadata Meta, FRangeData Data) const
{
	if (Type.NumRanges > 1)
	{
		return FastenRanges(Type, Meta, MakeArrayView(Data.Ranges, Meta.Num));
	}
	else if (Type.InnermostIsStruct())
	{
		check(!Meta.IsSourceRange); // todo: call RebuildRanges. Should only matter for enums/structs
		return FastenStructs(Type.InnermostId.Get().AsStructDeclId(), Meta, Data.Structs);
	}
	else if (Type.InnermostIsEnum())
	{
		check(!Meta.IsSourceRange); // todo: call RebuildRanges. Should only matter for enums/structs
		return FastenEnums(Type.InnermostId.Get().AsEnum(), Type.InnermostType.Enum, Meta, Data);
	}
	else
	{
		return FastenArithmetics(Type.InnermostType.Kind, Meta.Num, Data.Arithmetics.Source);
	}
}

FBuiltRange* FBatchRun::FastenRanges(FLooseType Type, FLooseMetadata Meta, TConstArrayView64<FLooseNestedRange> Ranges) const
{
	if (Ranges.IsEmpty())
	{
		return nullptr;
	}

	FBuiltRange* Out = FBuiltRange::Create(Scratch, Ranges.Num(), sizeof(FBuiltRange*));
	FBuiltRange** OutIt = reinterpret_cast<FBuiltRange**>(Out->Data);

	FLooseType ItemType = ItemOf(Type);
	if (ItemType.IsRange())
	{
		for (FLooseNestedRange Range : Ranges)
		{
			*OutIt++ = FastenRanges(ItemType, Meta, MakeArrayView(Range.Data.Ranges, Range.Num));
		}
	}
	else if (ItemType.IsStruct())
	{
		FDeclId Id = ItemType.InnermostId.Get().AsStructDeclId();
		for (FLooseNestedRange Range : Ranges)
		{
			Meta.Num = Range.Num;
			*OutIt++ = FastenStructs(Id, Meta, Range.Data.Structs);
		}
	}
	else if (ItemType.IsEnum())
	{
		FEnumId Id = ItemType.InnermostId.Get().AsEnum();
		FInnermostEnumType InnermostType = ItemType.InnermostType.Enum;
		for (FLooseNestedRange Range : Ranges)
		{
			Meta.Num = Range.Num;
			*OutIt++ = FastenEnums(Id, InnermostType, Meta, Range.Data);
		}
	}
	else
	{
		for (FLooseNestedRange Range : Ranges)
		{
			*OutIt++ = FastenArithmetics(ItemType.InnermostType.Kind, Range.Num, Range.Data.Arithmetics.Source);
		}
	}

	return Out;
}

FBuiltRange* FBatchRun::FastenStructs(FDeclId Id, FLooseMetadata Meta, FStructRange Structs) const
{
	if (Meta.Num == 0)
	{
		return nullptr;
	}
	
	const FParameterIndexer& Names = History.LooseIds.Get(Id);
	const FStructDeclaration& Declaration = History.Latest.Get(Id);

	FBuiltRange* Out = FBuiltRange::Create(Scratch, Meta.Num, sizeof(FBuiltStruct*));
	FBuiltStruct** OutIt = reinterpret_cast<FBuiltStruct**>(Out->Data);
	FMemberBuilder Tmp;
	if (Meta.InnermostSchema)
	{
		FStructSchemaHandle Schema = { FStructSchemaId(Meta.InnermostSchema.Get()), SourceBatch };
		for (FSourceStructRangeReader SourceIt(Structs, Meta.Num); SourceIt.HasMore();)
		{
			RebuildMembers(Tmp, SourceIt.GrabView(Schema), Id);
			*OutIt++ = Tmp.BuildAndReset(Scratch, Declaration, Debug); 
		}
	}
	else
	{
		for (FLooseValue Struct : TConstArrayView64<FLooseValue>(Structs.Loose, Meta.Num))
		{
			if (Struct.Meta.InnermostSchema)
			{
				RebuildMembers(Tmp, AsSourceStruct(Struct, SourceBatch), Id);
			}
			else
			{
				FastenMembers(Tmp, MakeArrayView(Struct.Data.Struct.Loose, Struct.Meta.Num), Names, Declaration);
			}
			*OutIt++ = Tmp.BuildAndReset(Scratch, Declaration, Debug); 
		}
		
	}
	check(OutIt == Out->AsStructs().GetData() + Meta.Num);
	return Out;
}

template<typename T>
FBuiltRange* FastenLooseEnums(TConstArrayView64<FEnumLooseData> Ins, FEnumFastener Fastener, const FEnumeratorIndexer& Indexer, FScratchAllocator& Scratch)
{
	const uint64 Num = Ins.Num();
	FBuiltRange* Out = FBuiltRange::Create(Scratch, Num, sizeof(T)); 
	T* OutIt = reinterpret_cast<T*>(Out->Data);
	
	if (Fastener.Decl == nullptr)
	{
		for (FEnumLooseData In : Ins)
		{
			*OutIt++ = IntCastChecked<T>(BitCast<uint64>(In.Enumerator));
		}
	}
	else if (Fastener.Decl->Mode == EEnumMode::Flat)
	{
		for (FEnumLooseData In : Ins)
		{
			*OutIt++ = IntCastChecked<T>(Fasten(Fastener, In.Enumerator.Flat, Indexer));
		}
	}
	else
	{
		for (FEnumLooseData In : Ins)
		{
			*OutIt++ = IntCastChecked<T>(Fasten(Fastener, In.Enumerator.Flags, Indexer));
		}
	}

	return Out;
}

FBuiltRange* FBatchRun::FastenEnums(FEnumId Id, FInnermostEnumType Type, FLooseMetadata Meta, FRangeData Data) const
{
	if (Meta.Num == 0)
	{
		return nullptr;
	}
	else if (Meta.InnermostSchema)
	{
		const FEnumSchema& Schema = ResolveEnumSchema(SourceBatch, FEnumSchemaId(Meta.InnermostSchema.Get()));
		return FBuiltRange::Create(Scratch, Meta.Num, SizeOf(Schema.Width), Data.Enums.Source); 
	}

	TArrayView64<FEnumLooseData> Enums(Data.Enums.Loose, Meta.Num);
	FEnumFastener Fastener = Transformations[Id].Loose;
	const FEnumeratorIndexer& Indexer = History.LooseIds.Get(Id);
	const FEnumDeclaration& Declaration = History.Latest.Get(Id);
	switch (Declaration.Width)
	{
		case ELeafWidth::B8:		return FastenLooseEnums<uint8 >(Enums, Fastener, Indexer, Scratch);
		case ELeafWidth::B16:		return FastenLooseEnums<uint16>(Enums, Fastener, Indexer, Scratch);
		case ELeafWidth::B32:		return FastenLooseEnums<uint32>(Enums, Fastener, Indexer, Scratch);
		case ELeafWidth::B64:		return FastenLooseEnums<uint64>(Enums, Fastener, Indexer, Scratch);
	}

	check(false);
	return nullptr;
}

FBuiltRange* FBatchRun::FastenArithmetics(EInnermostKind Kind, uint64 Num, const void* Data) const
{
	if (Num == 0)
	{
		return nullptr;
	}

	SIZE_T LeafSize = SizeOf(ToLeafWidth(Kind));
	FBuiltRange* Out = FBuiltRange::Create(Scratch, Num, LeafSize);
	if (Kind == EInnermostKind::Bool)
	{
		FBoolRangeView(Data, Num).Copy(Out->Data, Num); // bits -> bools
	}
	else
	{
		FMemory::Memcpy(Out->Data, Data, Num * LeafSize);
	}

	return Out;
}

FBuiltStruct* FBatchRun::RebuildStruct(FStructView Source) const
{
	return RebuildStruct(Source, RuntimeIds[Source.Schema.Id]);
}

FBuiltStruct* FBatchRun::RebuildStruct(FStructView Source, FDeclId Id) const
{	
	FMemberBuilder Out;
	RebuildMembers(Out, Source, Id);
	return Out.BuildAndReset(Scratch, History.Latest.Get(Id), Debug);
}

void FBatchRun::RebuildMembers(FMemberBuilder& Out, FStructView Source, FDeclId Id) const
{
	FMemberReader It(Source);
	if (It.HasMore() && IsSuper(It.PeekType()))
	{
		FStructView Super = It.GrabStruct();
		FDeclId SuperId = RuntimeIds[Super.Schema.Id];
		Out.AddSuperStruct(SuperId, RebuildStruct(Super, SuperId));
	}

	while (It.HasMore())
	{
		FMemberId Name = It.PeekNameUnchecked();
		Out.Add(Name, RebuildValue(It));
	}
}

FTypedValue	FBatchRun::RebuildValue(FMemberReader& It) const
{
	FMemberType Type = It.PeekType();
	switch (Type.GetKind())
	{
	case EMemberKind::Leaf:		return RebuildValue(It.GrabLeaf());
	case EMemberKind::Range:	return RebuildValue(Type, It.GrabRange());
	case EMemberKind::Struct:	return RebuildValue(It.GrabStruct());
	}
	check(false);
	return {};
}

FTypedValue	FBatchRun::RebuildValue(FLeafView In) const
{
	FOptionalEnumId Enum = In.Enum ? ToOptional(RuntimeIds[In.Enum.Get()]) : NoId;
	FBuiltMember Out(FMemberId{}, In.Leaf, Enum, GetIntegerValue(In));
	return {Out.Schema, Out.Value};
}

static uint16 CountRanges(FRangeSchema Schema)
{
	if (!Schema.ItemType.IsRange())
	{
		return 1;
	}

	uint32 Idx = 0;
	while (Schema.NestedItemTypes[Idx++].IsRange());
	return IntCastChecked<uint16>(1 + Idx);
}

FTypedValue	FBatchRun::RebuildValue(FMemberType Type, FRangeView Range) const
{
	FRangeSchema InSchema = GetSchema(Range);
	FMemberSchema OutSchema = { Type, InSchema.ItemType, CountRanges(InSchema), RuntimeIds[InSchema.InnermostSchema], 
								InSchema.ItemType.IsRange() ? InSchema.NestedItemTypes : nullptr };
	FBuiltRange* OutValue = RebuildRange(Range);
	return { OutSchema, { .Range = OutValue } };
}

FTypedValue FBatchRun::RebuildValue(FStructView Struct) const
{
	FDeclId Id = RuntimeIds[Struct.Schema.Id];
	FMemberSchema Schema = { DefaultStructType, DefaultStructType, 0,  FInnerId(Id), nullptr };
	return { Schema, { .Struct = RebuildStruct(Struct, Id) } };
}

FBuiltRange* FBatchRun::RebuildRange(FRangeView Range) const
{
	if (Range.Num())
	{
		switch (Range.GetItemType().GetKind())
		{
		case EMemberKind::Leaf:		return RebuildLeaves(Range.AsLeaves());
		case EMemberKind::Range:	return RebuildRanges(Range.AsRanges());
		case EMemberKind::Struct:	return RebuildStructs(Range.AsStructs());
		}
	}
	return nullptr;
}


FBuiltRange* FBatchRun::RebuildLeaves(FLeafRangeView Leaves) const
{
	FUnpackedLeafType Leaf = GetType(Leaves);
	FBuiltRange* Out = FBuiltRange::Create(Scratch, Leaves.Num(), SizeOf(Leaf.Width));
	if (Leaf.Type == ELeafType::Bool)
	{
		Leaves.AsBools().Copy(Out->Data, Leaves.Num()); // bits -> bools
	}
	else
	{
		FMemory::Memcpy(Out->Data, GetValues(Leaves), Leaves.Num() * SizeOf(Leaf.Width));
	}

	return Out;
}

FBuiltRange* FBatchRun::RebuildRanges(FNestedRangeView Ranges) const
{
	FBuiltRange* Out = FBuiltRange::Create(Scratch, Ranges.Num(), sizeof(FBuiltRange*));
	FBuiltRange** OutIt = reinterpret_cast<FBuiltRange**>(Out->Data);
	for (FRangeView Range : Ranges)
	{
		*OutIt++ = RebuildRange(Range);
	}
	return Out;
}

FBuiltRange* FBatchRun::RebuildStructs(FStructRangeView Structs) const
{
	return RebuildStructs(Structs, RuntimeIds[Structs.GetSchema().Id]);
}

FBuiltRange* FBatchRun::RebuildStructs(FStructRangeView Structs, FDeclId Id) const
{
	const FStructDeclaration& Declaration = History.Latest.Get(Id);

	FBuiltRange* Out = FBuiltRange::Create(Scratch, Structs.Num(), sizeof(FBuiltStruct*));
	FBuiltStruct** OutIt = reinterpret_cast<FBuiltStruct**>(Out->Data);
	FMemberBuilder Tmp;
	for (FStructView Struct : Structs)
	{
		RebuildMembers(Tmp, Struct, Id);
		*OutIt++ = Tmp.BuildAndReset(Scratch, Declaration, Debug);
	}
	return Out;
}

////////////////////////////////////////////////////////////////////////////////////////////////

FBatchUpgrader::FBatchUpgrader(const FSchemaBatch& InBatch, TConstArrayView<FInnerId> InRuntimeIds, FIdIndexerBase& Ids)
: Batch(InBatch)
, RuntimeIds(InRuntimeIds)
{}

static bool InnerSchemasMatches(const FStructSchema& Schema, const TBitArray<>& MatchingSchemas)
{
	for (FSchemaId Inner : MakeArrayView(Schema.GetInnerSchemas(), Schema.NumInnerSchemas))
	{
		if (!MatchingSchemas[Inner.Idx])
		{
			return false;
		}
	}

	return true;
}

static bool UnmatchOuterSchemas(TBitArray<>& MatchingSchemas, TSchemaRange<const FStructSchema> Schemas)
{
	bool bUnmatched = false;
	for (int32 Idx = 0; const FStructSchema& Schema : Schemas)
	{
		if (MatchingSchemas[Idx] && !InnerSchemasMatches(Schema, MatchingSchemas))
		{
			bUnmatched = true;
			MatchingSchemas[Idx] = false;
		}
		++Idx;
	}
	return bUnmatched;
}

bool FBatchUpgrader::MatchSchemas(const IDeclarations& Types)
{
	check(MatchingSchemas.IsEmpty());

	const int32 Num = Batch.NumSchemas;
	const int32 NumStructs = Batch.NumStructSchemas;
	const int32 NumEnums = Num - NumStructs;
	MatchingSchemas.SetNumUninitialized(Num);

	// Cache all struct declarations
	Declarations.Reserve(NumStructs);
	for (FDeclId Id : FInnerIds({RuntimeIds}).GetDeclIds(NumStructs))
	{
		Declarations.Emplace(Types.Find(Id));
	}

	// Match struct schemas against declarations
	int32 Idx = 0;
	TSchemaRange<const FStructSchema> StructSchemas = GetStructSchemas(Batch);
	for (const FStructSchema& Schema : StructSchemas)
	{
		const FStructDeclaration* Decl = Declarations[Idx];
		MatchingSchemas[Idx] = Decl && MatchStruct(*Decl, Schema, {RuntimeIds});
		++Idx;
	}

	// Match enum schemas against declarations
	for (const FEnumSchema& Schema : GetEnumSchemas(Batch))
	{
		const FEnumDeclaration* Decl = Types.Find(RuntimeIds[Idx].AsEnum());
		MatchingSchemas[Idx] = Decl && MatchEnum(*Decl, Schema);
		++Idx;
	}

	// Flag outer structs as non-matching when inner structs or enums dont match
	// using a somewhat naive recursive approach that can be optimized	
	if (MatchingSchemas.Contains(false))
	{
		while (UnmatchOuterSchemas(/* in-out */ MatchingSchemas, StructSchemas));

		return false;
	}

	return true;
}

TArray<FUpgradedInstance> FBatchUpgrader::Upgrade(TConstArrayView<FStructView> Old, FHistory& Upgrades, FScratchAllocator& Scratch)
{
	check(!MatchingSchemas.IsEmpty());
	
	Upgrades.Compile();
	
	FBatchRun Run(Upgrades, Scratch, RuntimeIds);
	Run.CreateTransformations(MatchingSchemas, Batch);

	TArray<FUpgradedInstance> Out;
	Out.Reserve(Old.Num());
	for (FStructView Source : Old)
	{
		Out.Emplace(Run.UpgradeOrRebuild(Source));
	}

	return Out;
}

////////////////////////////////////////////////////////////////////////////////////////////////

static FType IndexType(FLiteralIndexerBase& Ids, FTypename In)
{
	return {Ids.IndexScopeLiteral(In.Scope), FTypenameId(Ids.IndexTypenameLiteral(In.Name))};
}

static FStructId IndexStruct(FLiteralIndexerBase& Ids, FTypename In)
{
	return Ids.IndexStruct(IndexType(Ids, In));
}

////////////////////////////////////////////////////////////////////////////////////////////////

struct FMidOp
{
	Op	 					Function;
	uint8					NumOptionalInputs;	// Excluding inner parameters
	uint8					NumRequiredInputs;	// Excluding inner parameters
	uint8					NumOutputs;			// Excluding inner parameters
};

struct FMidParam
{
	FParameterId			Name;
	FLooseType				Type;
	uint8					InnerSlice;
};

struct FInnerSlice
{
	int32 					ItemIdx;
	int32					NumItems;

	explicit operator bool() { return !!NumItems;}
};

template<typename T>
struct TInnerSlices
{
	uint8 AddSlice(TConstArrayView<T> Slice)
	{
		check(Slice.Num() > 0);
		Slices.Emplace(Items.Num(), Slice.Num());
		Items.Append(Slice);
		return IntCastChecked<uint8>(Slices.Num());
	}

	TConstArrayView<T> GetSlice(uint8 Handle) const
	{
		check(Handle > 0);
		FInnerSlice Slice = Slices[Handle - 1];
		check(Slice.ItemIdx + Slice.NumItems <= Items.Num());
		return MakeArrayView(Items.GetData() + Slice.ItemIdx, Slice.NumItems);
	}

	TArray<FInnerSlice, TInlineAllocator<16>>	Slices;			// todo switch to scratch or vanilla allocs
	TArray<T, TInlineAllocator<32>>				Items;			// todo switch to scratch or vanilla allocs
};

struct FOps final : IOps
{
	FOps(FDeclId Id, uint16 Ver, FLiteralIndexerBase& InIds, FLooseIndexers& Indexers)
	: Owner(Id)
	, Version(Ver)
	, Ids(InIds)
	, OwnerIndexer(Indexers.Get(Id))
	, InnerIndexers(Indexers)
	{}

	const FDeclId								Owner;
	const uint16								Version;
	FOptionalVersion							NextVersion;
	FLiteralIndexerBase&						Ids;
	FParameterIndexer&							OwnerIndexer;
	FLooseIndexers&								InnerIndexers;
	TArray<FMidOp, TInlineAllocator<8>>			Ops;			// todo switch to scratch or vanilla allocs
	TArray<FMidParam, TInlineAllocator<32>>		Parameters;		// todo switch to scratch or vanilla allocs
	TInnerSlices<FMidParam>						InnerMembers;
	TInnerSlices<FEnumeratorId>					InnerEnumerators;
	FOptionalDeclId								Redirect;
	int32										NumInputs = 0;
	int32										NumOptionalInputs = 0;
	int32										NumOutputsFlat = 0;
	int32										NumDecompositions = 0;
	int32										NumStructDecompositions = 0;

	int32										NumMatches() const				{ return NumInputs + NumDecompositions; }
	int32										NumRecompositions() const		{ return InnerEnumerators.Slices.Num(); }
	int32										NumRecompositionIds() const		{ return InnerEnumerators.Items.Num(); }

	uint8 IndexEnumerators(FEnumId Id, std::initializer_list<const char*> Enumerators)
	{
		FEnumeratorIndexer& InnerIndexer = InnerIndexers.Get(Id);

		TArray<FEnumeratorId, TInlineAllocator<64>> Slice;
		for (FAnsiStringView Enumerator : Enumerators)
		{
			Slice.Add(InnerIndexer.Index(Ids.IndexEnumeratorLiteral(Enumerator)));
		}

		return InnerEnumerators.AddSlice(Slice);
	}

	uint8 IndexMembers(FStructId Id, TConstArrayView<FParameter> Members)
	{
		// opt: reuse slices, special case iota slices

		FParameterIndexer& InnerIndexer = InnerIndexers.Get(FDeclId(Id));
		TArray<FMidParam, TInlineAllocator<64>> Slice;
		Slice.Reserve(Members.Num());
		for (FParameter Member : Members)
		{
			Slice.Add(IndexParameter(Member, InnerIndexer));
		}
		return InnerMembers.AddSlice(Slice);
	}

	virtual FEnumId NameEnum(FTypename Enum) override
	{
		return Ids.IndexEnum(IndexType(Ids, Enum));
	}

	virtual FStructId NameStruct(FTypename Struct) override
	{
		return IndexStruct(Ids, Struct);
	}
	
	virtual FEnumeratorNames NameEnumerators(FEnumId Id, std::initializer_list<const char*> Enumerators) override
	{
		return { Id, IndexEnumerators(Id, Enumerators) };
	}

	virtual FEnumFlagNames NameFlags(FEnumId Id, std::initializer_list<const char*> Flags) override
	{
		return { Id, IndexEnumerators(Id, Flags) };
	}

	virtual FStructMemberNames NameMembers(FStructId Id, TConstArrayView<FParameter> Members) override
	{
		return { Id, IndexMembers(Id, Members) };
	}

	bool IsOpen() const
	{
		return !NextVersion;
	}
	
	virtual void Bump(uint16 Next) override
	{
		check(IsOpen());
		NextVersion = Next;
	}

	virtual void Rename(FTypename To, uint16 ToVersion) override
	{
		check(IsOpen());
		Redirect = FDeclId(IndexStruct(Ids, To));
		NextVersion = ToVersion;
	}

	uint32 CountFlat(TConstArrayView<FMidParam> Params) const
	{
		uint32 Out = Params.Num();
		for (FMidParam Param : Params)
		{
			if (uint8 Slice = Param.InnerSlice)
			{
				if (Param.Type.IsStruct())
				{
					Out += CountFlat(InnerMembers.GetSlice(Slice));
				}
			}
		}
		return Out;
	}

	virtual void Add(TConstArrayView<FParameter> Ins, TConstArrayView<FParameter> Outs, Op Func) override
	{
		check(IsOpen());

		uint8 NumOptionalIns = IntCastChecked<uint8>(Algo::CountIf(Ins, [](FParameter P){ return P.bOptional; }));
		uint8 NumRequiredIns = IntCastChecked<uint8>(Ins.Num() - NumOptionalIns);
		Ops.Emplace(Func, NumOptionalIns, NumRequiredIns, IntCastChecked<uint8>(Outs.Num()));
		IndexParameters(Ins);
		CountDecompositions(MakeArrayView(Parameters).Right(Ins.Num()));
		NumInputs += Ins.Num();
		NumOptionalInputs += NumOptionalIns;
		IndexParameters(Outs);
		NumOutputsFlat += CountFlat(MakeArrayView(Parameters).Right(Outs.Num()));
	}

	void IndexParameters(TConstArrayView<FParameter> Params)
	{
		Parameters.AddUninitialized(Params.Num());
		FMidParam* OutIt = Parameters.GetData() + Parameters.Num() - Params.Num();
		for (FParameter Param : Params)
		{
			*OutIt++ = IndexParameter(Param, OwnerIndexer);
		}
	}

	FMidParam IndexParameter(FParameter In, FParameterIndexer& StructIndexer) const
	{
		check(In.NumRanges <= MaxRangeNesting);
		check(In.InnermostHandle.Id || In.InnermostHandle.Version == 0);

		FMemberId Member = Ids.IndexMemberLiteral(FAnsiStringView(In.Name, In.NameLen));
		FParameterId Name = StructIndexer.Index(Member);
		FLooseType Type = { In.bOptional, In.NumRanges, In.InnermostType, In.InnermostHandle.Version, In.InnermostHandle.Id };

		return {Name, Type, In.InnermostHandle.Slice};
	}

	void CountDecompositions(TConstArrayView<FMidParam> Inputs)
	{
		for (FMidParam Input : Inputs)
		{
			if (Input.InnerSlice)
			{
				++NumDecompositions;
				NumStructDecompositions += Input.Type.IsStruct();
			}
		}
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////

const FLooseSchema* FSchemaCache::Obtain(FDeclId Id, const IDeclarations& Types, FParameterIndexer& ParamIds)
{
	if (const FLooseSchemaPtr* Out = Cache.Find(Id))
	{
		return Out->Get();
	}
	if (const FStructDeclaration* Struct = Types.Find(Id))
	{
		return Cache.Emplace(Id, CreateTargetSchema(*Struct, Types, ParamIds)).Get();
	}

	// Don't cache missing declarations for now
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////////////////////

static FEnumeratorId GetInnerName(FEnumeratorId Inner) { return Inner; }
static FParameterId GetInnerName(FMidParam Inner) { return Inner.Name; }

template <typename InnerType>
static bool IsDeclarationOrder(TConstArrayView<InnerType> Order)
{
	uint32 N = 0;
	for (InnerType Inner : Order)
	{
		if (GetInnerName(Inner).Idx != N++)
		{
			return false;
		}

	}
	return true;
}

static TSet<FParameterId> CreateInputOrder(TConstArrayView<FMidParam> Order)
{
	if (IsDeclarationOrder(Order))
	{
		return {};
	}
		
	TSet<FParameterId> Out;
	Out.Reserve(Order.Num());
	for (FMidParam Param : Order)
	{
		Out.Add(Param.Name);
	}
	return Out;
}

static TSet<FEnumeratorId> CreateInputOrder(TConstArrayView<FEnumeratorId> Order)
{
	return IsDeclarationOrder(Order) ? TSet<FEnumeratorId>() : TSet<FEnumeratorId>(Order);
}

////////////////////////////////////////////////////////////////////////////////////////////////

static FLooseSchemaPtr CreateInputSchema(uint16 Version, TConstArrayView<FMidParam> Order)
{
	const uint32 Size = offsetof(FLooseSchema, Members) + Order.Num() * sizeof(FLooseParameter);
	FLooseSchema Header = {Version, IntCastChecked<uint16>(Order.Num())};
	FLooseSchema* Out = new (FMemory::MallocZeroed(Size, alignof(FLooseSchema))) FLooseSchema(Header);

	FLooseParameter* It = Out->Members;
	for (FMidParam Param : Order)
	{
		new (It++) FLooseParameter{Param.Name, Param.Type};
	}

	return FLooseSchemaPtr(Out);
}

////////////////////////////////////////////////////////////////////////////////////////////////

static uint64 CopyFlat(FOutParameter* Dst, TConstArrayView<FMidParam> Params, const FOps& Ops)
{
	FOutParameter* It = Dst;
	for (FMidParam Param : Params)
	{
		FOutParameter& Out = *It++;
		Out = {Param.Name, 0, 0, Param.Type};

		if (uint8 Slice = Param.InnerSlice)
		{
			if (Param.Type.InnermostType.Kind == EInnermostKind::Struct)
			{
				TConstArrayView<FMidParam> Inners = Ops.InnerMembers.GetSlice(Slice);
				Out.NumInners = static_cast<uint8>(Inners.Num());
				It += CopyFlat(It, Inners, Ops);
			}
			else
			{
				check(Param.Type.InnermostType.Kind == EInnermostKind::Enum);
				TConstArrayView<FEnumeratorId> Inners = Ops.InnerEnumerators.GetSlice(Slice);
				Out.NumInners = static_cast<uint8>(Inners.Num());
				Out.RecompIdx = Slice - 1;
			}
		}
	}

	return static_cast<uint64>(It - Dst);
}

static void CompileDecomposition(uint8*& InOutIt, FMidParam Input, TInnerSlices<FMidParam> Members, TInnerSlices<FEnumeratorId> Enumerators)
{
	FInnerId Id = Input.Type.InnermostId.Get();
	uint16 Version = Input.Type.InnermostVersion;
	if (Input.Type.InnermostIsStruct())
	{
		TConstArrayView<FMidParam> Inners = Members.GetSlice(Input.InnerSlice);
		new (InOutIt) FStructDecomposition{ Id.AsStructDeclId(), Version,  CreateInputOrder(Inners), /* set in Link */ nullptr, 
											CreateInputSchema(Input.Type.InnermostVersion, Inners) };
		InOutIt += sizeof(FStructDecomposition);
	}
	else
	{
		check(Input.Type.InnermostIsEnum());
		TConstArrayView<FEnumeratorId> Inners = Enumerators.GetSlice(Input.InnerSlice);
		new (InOutIt) FEnumDecomposition{ CreateInputOrder(Inners) };
		InOutIt += sizeof(FEnumDecomposition);
	}
}

static int32 CompileRecomposition(uint16& OutJump, FEnumeratorId* Dst, TConstArrayView<FEnumeratorId> Src)
{				
	// The same FNamedEnumerators/FNamedFlags slice can be reused for multiple outputs -- copy enumerators once
	if (OutJump)
	{
		return 0;
	}

	static_assert(sizeof(OutJump) == sizeof(*Dst), "Jump offset is relative to sel and assumes jump offset and FEnumeratorId have same size");
	OutJump = IntCastChecked<uint16>(reinterpret_cast<uint16*>(Dst) - &OutJump);
	FMemory::Memcpy(Dst, Src.GetData(), Src.NumBytes());

	return Src.Num();
}

static void CompileOps(FVersionUpgrades& Out, const FOps& In)
{
	using MemberMatches = TArray<FMemberMatch, TInlineAllocator<1>>;
	TArray<MemberMatches> AllMatches;
	AllMatches.SetNum(In.OwnerIndexer.Num());

	TConstArrayView<FMidParam> Params = In.Parameters;
	FOutParameter* Outputs = const_cast<FOutParameter*>(Out.GetOutputs());
	FOp* OutOps = const_cast<FOp*>(Out.GetOps());
	uint32* DecompOffsets = const_cast<uint32*>(Out.GetDecompOffsets());
	FLooseParameter* OptionalInputs = const_cast<FLooseParameter*>(Out.GetOptionalInputs());
	uint16* RecompJumps = const_cast<uint16*>(Out.GetRecompJumps());
	FEnumeratorId* RecompIdIt = reinterpret_cast<FEnumeratorId*>(RecompJumps + Out.NumRecompositions);
	uint8* DecompDataIt = Align(reinterpret_cast<uint8*>(Outputs + In.NumOutputsFlat), alignof(FStructDecomposition));
	uint16 DecompIdx = 0;
	uint16 OptionalInputIdx = 0;
	uint32 OutputIdx = 0;
	TConstArrayView<FMidOp> InOps = In.Ops;
	for (int32 OpIdx = 0; OpIdx < InOps.Num(); ++OpIdx)
	{
		const FMidOp Op = InOps[OpIdx];

		// Compile inputs into member matches
		check(Op.NumOptionalInputs + Op.NumRequiredInputs <= 0xFF);
		uint8 InputIdx = 0;
		for (FMidParam Input : PopFront(Params, Op.NumOptionalInputs + Op.NumRequiredInputs))
		{
			// Generate input decomposition match
			if (Input.InnerSlice)
			{
				FMatchIdx Idx = { .Decomp = DecompIdx };
				FLooseType Type = ClearDecomposed(Input.Type);
				AllMatches[Input.Name.Idx].Add({ .Type = Type, .Idx = Idx, .InputIdx = InputIdx, .Recipient = FMemberMatch::DecomposeRecipient });

				// Write offset and generate decomposition
				DecompOffsets[DecompIdx] = IntCastChecked<uint32>(DecompDataIt - reinterpret_cast<uint8*>(&Out));
				CompileDecomposition(/* in-out */ DecompDataIt, Input, In.InnerMembers, In.InnerEnumerators);
				++DecompIdx;
			}

			// Copy optional input metadata
			if (Input.Type.OptionalParameter)
			{
				OptionalInputs[OptionalInputIdx++] = {Input.Name, Input.Type};
			}

			// Copy normal input match
			FMatchIdx Idx = { .Op = static_cast<uint16>(OpIdx) };
			AllMatches[Input.Name.Idx].Add({ .Type = Input.Type, .Idx = Idx, .InputIdx = InputIdx, .Recipient = 0 });

			++InputIdx;
		}

		// Copy out parameters
		TConstArrayView<FMidParam> OutParams = PopFront(/* in-out */ Params, Op.NumOutputs);
		uint64 NumOutputsFlat = CopyFlat(Outputs + OutputIdx, OutParams, In);
		check(NumOutputsFlat == In.CountFlat(OutParams));

		// Generate enum recompositions
		for (FMidParam Output : OutParams)
		{
			if (Output.InnerSlice && Output.Type.InnermostType.Kind == EInnermostKind::Enum)
			{
				uint8 RecompIdx = Output.InnerSlice - 1;
				RecompIdIt += CompileRecomposition(/* out */ RecompJumps[RecompIdx], RecompIdIt, In.InnerEnumerators.GetSlice(Output.InnerSlice));
			}
		}

		// Copy FMidOp -> FOp
		OutOps[OpIdx] = {	.Function = Op.Function,
							.Outputs = { OutputIdx, IntCastChecked<uint8>(NumOutputsFlat) },
							.NumOptionalInputs = Op.NumOptionalInputs,
							.NumRequiredInputs = Op.NumRequiredInputs,
							.OptionalInputIdx = IntCastChecked<uint16>(OptionalInputIdx - Op.NumOptionalInputs) };

		OutputIdx += NumOutputsFlat;
	}
	check(OutputIdx == In.NumOutputsFlat);
	check(OptionalInputIdx == Out.NumOptionalInputs);
	check(RecompIdIt == reinterpret_cast<FEnumeratorId*>(RecompJumps + Out.NumRecompositions) + In.InnerEnumerators.Items.Num());
	check(Out.GetOutputs() == AlignPtr<FOutParameter>(RecompIdIt));

	// Copy matches
	FMiniSlice* NameMatchIt = Out.NameMatches;
	FMemberMatch* Matches = const_cast<FMemberMatch*>(Out.GetMatches());
	uint32 MatchIdx = 0;
	for (const MemberMatches& NameMatches : AllMatches)
	{
		FMemory::Memcpy(Matches + MatchIdx, NameMatches.GetData(), NameMatches.NumBytes());
		new (NameMatchIt++) FMiniSlice(MatchIdx, NameMatches.Num());
		MatchIdx += NameMatches.Num(); 
	}
	check(MatchIdx == Out.NumMatches);
}

static void CompileVersion(FVersionUpgrades& Out, const FOps& In)
{
	Out.NumNames = In.OwnerIndexer.Num();
	Out.NumMatches = static_cast<uint32>(In.NumMatches());
	Out.NumOps = IntCastChecked<uint16>(In.Ops.Num());
	Out.NumDecompositions = IntCastChecked<uint16>(In.NumDecompositions);
	Out.NumRecompositions = IntCastChecked<uint16>(In.NumRecompositions());
	Out.NumOptionalInputs = IntCastChecked<uint16>(In.NumOptionalInputs);
	const FEnumeratorId* RecompIds = AlignPtr<FEnumeratorId>(Out.GetRecompJumps() + In.NumRecompositions());
	const FOutParameter* Outputs = AlignPtr<FOutParameter>(RecompIds + In.NumRecompositionIds());
	Out.OutputsOffset = IntCastChecked<uint32>((const uint8*)Outputs - (uint8*)&Out);
	Out.NextVersion = In.NextVersion;

	CompileOps(Out, In);
}

static uint32 CalculateVersionUpgradesSize(const FOps& Ops)
{
	uint32 Out = offsetof(FVersionUpgrades, NameMatches) + Ops.OwnerIndexer.Num() * sizeof(FMiniSlice);
	Out = Align(Out + Ops.NumMatches() * sizeof(FMemberMatch), alignof(FMemberMatch));
	Out = Align(Out + Ops.Ops.Num() * sizeof(FOp), alignof(FOp));
	Out = Align(Out + Ops.NumDecompositions * sizeof(uint32), alignof(uint32));
	Out = Align(Out + Ops.NumOptionalInputs * sizeof(FLooseParameter), alignof(FLooseParameter));
	Out = Align(Out + Ops.NumRecompositions() * sizeof(uint16), alignof(uint16));
	Out = Align(Out + Ops.NumRecompositionIds() * sizeof(FEnumeratorId), alignof(FEnumeratorId));
	Out = Align(Out + Ops.NumOutputsFlat * sizeof(FOutParameter), alignof(FOutParameter));
	static_assert(alignof(FStructDecomposition) == alignof(FEnumDecomposition));
	Out = Align(Out + Ops.NumStructDecompositions * sizeof(FStructDecomposition), alignof(FStructDecomposition));
	Out = Align(Out + (Ops.NumDecompositions - Ops.NumStructDecompositions) * sizeof(FEnumDecomposition), alignof(FEnumDecomposition));

	return Out;
}

void FHistory::CompileStruct(TArrayView<FOpsPtr> AllOps)
{
	const FDeclId Id = FDeclId(AllOps[0]->Owner);
	const uint32 StructSize = sizeof(FStructUpgrades) + AllOps.Num() * sizeof(FVersionUpgrades*);

	uint32 VersionOffset = StructSize;
	TArray<uint32, TInlineAllocator<32>> VersionOffsets;
	for (const FOpsPtr& Ops : AllOps)
	{
		VersionOffset = Align(VersionOffset, alignof(FVersionUpgrades));
		VersionOffsets.Add(VersionOffset);
		VersionOffset += CalculateVersionUpgradesSize(*Ops);
	}

	FOptionalDeclId Redirect = AllOps.Last()->Redirect;
	FOptionalVersion RedirectVersion = AllOps.Last()->NextVersion;
	FStructUpgrades Header = { .Id = Id, .FinalId = Redirect.GetOr(Id), .FirstVersion = AllOps[0]->Version, .NumVersions = IntCastChecked<uint16>(AllOps.Num()) };
	Header.FinalVersion = RedirectVersion ? RedirectVersion.Get() : Header.FirstVersion + Header.NumVersions - 1;
	// Zeroed memory expected by CompileRecomposition
	FStructUpgrades* Out = new (FMemory::MallocZeroed(VersionOffset, alignof(FStructUpgrades))) FStructUpgrades(Header);

	for (int32 Idx = 0, Num = AllOps.Num(); Idx < Num; ++Idx)
	{
		FVersionUpgrades* Version = reinterpret_cast<FVersionUpgrades*>(reinterpret_cast<uint8*>(Out) + VersionOffsets[Idx]);
		Out->Versions[Idx] = Version;
		CompileVersion(*Version, *AllOps[Idx]);
	}
	
	CompiledUpgrades.Emplace(Id, Out);
}

static TArrayView<FOpsPtr> GrabOwnerRange(FOpsPtr*& It, FOpsPtr* End)
{
	check(*It != nullptr);
	FOpsPtr* First = It++;
	for (FStructId Owner = (**First).Owner; It != End && *It && (**It).Owner == Owner; ++It);
	int32 Num = It - First;
	for (; It != End && *It == nullptr; ++It);
	return MakeArrayView(First, Num);
}

void FHistory::Compile()
{
	if (CommitQueue.IsEmpty())
	{
		return;
	}

	FOpsPtr* It = CommitQueue.FindByPredicate([=](FOpsPtr& Ops){ return Ops != nullptr; });
	for (FOpsPtr* End = GetEnd(CommitQueue); It != End;)
	{
		CompileStruct(GrabOwnerRange(/* in-out */ It, End));
	}

	Link(CommitQueue);

	CommitQueue.Empty();
	CommitScratch.Reset();
}

void FHistory::Erase(FStructId Owner)
{
	// todo: track inverse link dependencies and drop/disable/whatever compiled upgrades linked against owner

	LatestSchemas.Cache.Remove(FDeclId(Owner));

	if (CompiledUpgrades.Remove(FDeclId(Owner)) ||
		CommitQueue.IsEmpty())
	{
		return;
	}

	// Don't reorder, CompileStruct requires Ops to be sorted by version
	if (FOpsPtr* It = CommitQueue.FindByPredicate([=](FOpsPtr& Ops){ return Ops && Ops->Owner == Owner; }))
	{
		for (FOpsPtr* End = GetEnd(CommitQueue); It != End && *It && (**It).Owner == Owner; ++It)
		{
			It->Reset();
		}
	}
	
	while (!CommitQueue.IsEmpty() && CommitQueue.Last() == nullptr)
	{
		CommitQueue.Pop(EAllowShrinking::No);
	}

	if (CommitQueue.IsEmpty())
	{
		CommitScratch.Reset();
	}
}

void FHistory::Link(TConstArrayView<FOpsPtr> Committed)
{
	FOptionalDeclId Last;
	for (const FOpsPtr& Ops : Committed)
	{
		if (Ops && Ops->Owner != Last)
		{
			Link(Ops->Owner);
			Last = Ops->Owner;
		}
	}
}

template<class T>
T& EditDecomposition(const FVersionUpgrades& Version, uint16 DecompIdx)
{
	check(DecompIdx < Version.NumDecompositions);
	uint32 Offset = Version.GetDecompOffsets()[DecompIdx];
	return const_cast<T&>(Version.At<T>(Offset));
}

void FHistory::Link(FDeclId Compiled)
{
	FStructUpgrades& Upgrades = *CompiledUpgrades.FindChecked(Compiled).Get();
	for (const FVersionUpgrades* Version : ViewVersions(Upgrades))
	{
		for (FMemberMatch Match : ViewMatches(*Version))
		{
			if (Match.Decomposes())
			{
				if (Match.Type.IsStruct())
				{
					Link(EditDecomposition<FStructDecomposition>(*Version, Match.Idx.Decomp));
				}
			}
		}
	}
}

static bool CompareParameters(FLooseParameter A, FLooseParameter B)
{
	return A.Name == B.Name && A.Type.Compare(B.Type);
}

// Temp checks that latest declaration matches the input order exactly until overriding is implemented
static void TemporaryInputSchemaCheck(const FLooseSchema& Declared, const FLooseSchema& Input)
{
	check(Declared.Version == Input.Version);
	check(Declared.NumMembers >= Input.NumMembers);

	TConstArrayView<FLooseParameter> Expecteds = Declared.GetMembers();
	for (FLooseParameter Actual : Input.GetMembers())	
	{
		const FLooseParameter* Expected = Expecteds.FindByPredicate([=](FLooseParameter Param) { return CompareParameters(Actual, Param); });
		check(Expected);
		Expecteds.RightChopInline(Expected == Expecteds.GetData());
	}
}

void FHistory::Link(FStructDecomposition& Dependency)
{
	if (const FStructUpgradesPtr* Upgrades = CompiledUpgrades.Find(Dependency.Id))
	{
		// todo: override Upgrades with Recipient 1

		if (const FLooseSchema* DeclaredSchema = LatestSchemas.Obtain(Dependency.Id, Latest, LooseIds.Get(Dependency.Id)))
		{
			TemporaryInputSchemaCheck(*DeclaredSchema, *Dependency.InputSchema);
		}
		
		Dependency.Upgrades = Upgrades->Get();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////

FHistoryPtr MakeHistory(IDeclarations& Latest, FLiteralIndexerBase& Ids)
{
	return FHistoryPtr(new FHistory{Latest, {}, Ids, Latest});
}

void FHistoryDeleter::operator()(FHistory* Ptr) const
{
	delete Ptr;
}

void EraseChroniclesOf(FHistory& History, FStructId Owner)
{
	History.Erase(Owner);
}

////////////////////////////////////////////////////////////////////////////////////////////////

IOps* FChronicles::CreateVersion(FTypename Struct, uint16 Ver)
{
	FDeclId Id(IndexStruct(History.Ids, Struct));
	return new (History.CommitScratch) FOps(Id, Ver, History.Ids, History.LooseIds);
}

void FChronicles::CommitVersion(IOps* InOps)
{
	FOps* Ops = static_cast<FOps*>(InOps);
	checkf(uint16(PriorVersion + uint16(1)) == Ops->Version, TEXT("Commit versions in contiguous order")); // Overflows first time
	checkf(!bFinalVersion, TEXT("Final version already committed, version %d failed to Bump()"), PriorVersion);
	PriorVersion = Ops->Version;
	bFinalVersion = Ops->Redirect || !Ops->NextVersion;

	History.CommitQueue.Emplace(Ops);
}

}} // namespace PlainProps::Upgrade
