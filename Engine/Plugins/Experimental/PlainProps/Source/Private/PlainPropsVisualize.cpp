// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsVisualize.h"
#include "PlainPropsInternalBuild.h"
#include "PlainPropsInternalFormat.h"
#include "Misc/Guid.h"
#include "Modules/VisualizerDebuggingState.h"
#include <type_traits>

#define PP_DBGVIS_VERIFY_TYPE(Type) \
	static_assert(sizeof(Type) == sizeof(DbgVis::Type)); \
	static_assert(alignof(Type) == alignof(DbgVis::Type))

#define PP_DBGVIS_VERIFY_MEMBER(Type, Member) \
	static_assert(offsetof(Type, Member) == offsetof(DbgVis::Type, Member)); \
	static_assert(std::is_same_v<decltype(Type::Member),decltype(DbgVis::Type::Member)>);

#define PP_DBGVIS_VERIFY_BITFIELD(Type, Member) \
	static_assert(std::is_integral_v<decltype(Type::Member)> == std::is_integral_v<decltype(DbgVis::Type::Member)>); \
	static_assert(std::is_enum_v<decltype(Type::Member)> == std::is_enum_v<decltype(DbgVis::Type::Member)>);

#define PP_DBGVIS_VERIFY_ENUMERATOR(Enum, Constant) \
	static_assert((uint64)Enum::Constant == (uint64)DbgVis::Enum::Constant);


namespace PlainProps
{

// Verify layouts for duplicated debug types for InternalBuild.h

PP_DBGVIS_VERIFY_TYPE(FBuiltStruct);
PP_DBGVIS_VERIFY_MEMBER(FBuiltStruct, NumMembers);
PP_DBGVIS_VERIFY_MEMBER(FBuiltStruct, Members);

PP_DBGVIS_VERIFY_TYPE(FBuiltRange);
PP_DBGVIS_VERIFY_MEMBER(FBuiltRange, Num);
PP_DBGVIS_VERIFY_MEMBER(FBuiltRange, Data);


// Verify layouts for duplicated debug types for InternalFormat.h

PP_DBGVIS_VERIFY_TYPE(FSchemaBatch);
PP_DBGVIS_VERIFY_MEMBER(FSchemaBatch, NumNestedScopes);
PP_DBGVIS_VERIFY_MEMBER(FSchemaBatch, NestedScopesOffset);
PP_DBGVIS_VERIFY_MEMBER(FSchemaBatch, NumParametricTypes);
PP_DBGVIS_VERIFY_MEMBER(FSchemaBatch, NumSchemas);
PP_DBGVIS_VERIFY_MEMBER(FSchemaBatch, NumStructSchemas);
PP_DBGVIS_VERIFY_MEMBER(FSchemaBatch, SchemaOffsets);

PP_DBGVIS_VERIFY_TYPE(ESuper);
PP_DBGVIS_VERIFY_ENUMERATOR(ESuper, No);
PP_DBGVIS_VERIFY_ENUMERATOR(ESuper, Unused);
PP_DBGVIS_VERIFY_ENUMERATOR(ESuper, Used);
PP_DBGVIS_VERIFY_ENUMERATOR(ESuper, Reused);

PP_DBGVIS_VERIFY_TYPE(FStructSchema);
PP_DBGVIS_VERIFY_MEMBER(FStructSchema, Type);
PP_DBGVIS_VERIFY_MEMBER(FStructSchema, NumMembers);
PP_DBGVIS_VERIFY_MEMBER(FStructSchema, NumRangeTypes);
PP_DBGVIS_VERIFY_MEMBER(FStructSchema, NumInnerSchemas);
PP_DBGVIS_VERIFY_MEMBER(FStructSchema, Version);
PP_DBGVIS_VERIFY_BITFIELD(FStructSchema, Inheritance);
PP_DBGVIS_VERIFY_BITFIELD(FStructSchema, IsDense);
PP_DBGVIS_VERIFY_MEMBER(FStructSchema, Footer);

PP_DBGVIS_VERIFY_TYPE(FEnumSchema);
PP_DBGVIS_VERIFY_MEMBER(FEnumSchema, Type);
PP_DBGVIS_VERIFY_BITFIELD(FEnumSchema, FlagMode);
PP_DBGVIS_VERIFY_BITFIELD(FEnumSchema, ExplicitConstants);
PP_DBGVIS_VERIFY_MEMBER(FEnumSchema, Width);
PP_DBGVIS_VERIFY_MEMBER(FEnumSchema, Num);
PP_DBGVIS_VERIFY_MEMBER(FEnumSchema, Footer);

// Verify layouts for duplicated debug types for Load.cpp

// PP_DBGVIS_VERIFY_TYPE(FLoadStructPlan);
// PP_DBGVIS_VERIFY_MEMBER(FLoadStructPlan, Handle);
//
// PP_DBGVIS_VERIFY_TYPE(FLoadBatch);
// PP_DBGVIS_VERIFY_MEMBER(FLoadBatch, BatchId);
// PP_DBGVIS_VERIFY_MEMBER(FLoadBatch, NumReadSchemas);
// PP_DBGVIS_VERIFY_MEMBER(FLoadBatch, NumPlans);
// PP_DBGVIS_VERIFY_MEMBER(FLoadBatch, Plans);

////////////////////////////////////////////////////////////////////////////////

namespace DbgVis
{

// D4B455B7-7BAB-4F1D-A944-98EC086FB4AB => d4b455b77bab4f1da94498ec086fb4ab
static constexpr FGuid GIdVisualizerGuid = FGuid(0xD4B455B7, 0x7BAB4F1D, 0xA94498EC, 0x086FB4AB);
// 0A05D5A9-DE4E-492D-989E-7F936CC1C843 => 0a05d5a9de4e492d989e7f936cc1c843
static constexpr FGuid GBatchVisualizerGuid = FGuid(0x0A05D5A9, 0xDE4E492D, 0x989E7F93, 0x6CC1C843);

void FIdScope::AssignDebuggingState()
{
	FIdScope::Global = &Current;
	(void)::UE::Core::FVisualizerDebuggingState::Assign(GIdVisualizerGuid, Global);
}

FIdScope::~FIdScope()
{
	FIdScope::Global = Previous;
	(void)::UE::Core::FVisualizerDebuggingState::Assign(GIdVisualizerGuid, Global);
}

void AssignReadSchemasDebuggingState(FSchemaBatch** Slots)
{
	(void)::UE::Core::FVisualizerDebuggingState::Assign(GBatchVisualizerGuid, Slots);
}

} // namespace DbgVis

} // namespace PlainProps
