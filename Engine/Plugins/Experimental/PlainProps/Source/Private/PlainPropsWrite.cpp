// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsWrite.h"
#include "PlainPropsBuildSchema.h"
#include "PlainPropsIndex.h"
#include "PlainPropsInternalBuild.h"
#include "PlainPropsInternalFormat.h"
#include "Serialization/VarInt.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Misc/ScopeExit.h"

namespace PlainProps
{

// Maps declared / built ids to write ids
// 
// Rewrite as a more compact data structure once we get a large number of ids
struct FWriteIds
{
	FWriteIds(const FIdIndexerBase& DeclaredIds, const IDeclarations& Decls, const FBuiltSchemas& Schemas, ESchemaFormat Format);

	bool								HasStableNames() const { return !Names.IsEmpty(); }

	const IDeclarations&				Declarations;

	TArray<FOptionalNameId>				Names;
	TArray<FOptionalNestedScopeId>		NestedScopes;
	TArray<FOptionalParametricTypeId>	ParametricTypes;
	TArray<FOptionalStructSchemaId>		Structs;
	TArray<FOptionalEnumSchemaId>		Enums;

	uint32								NumKeptSchemas;
	uint32								NumKeptStructSchemas;
	TArray<FNestedScope>				KeptScopes;
	TArray<FParametricType>				KeptParametrics;
	TArray<FType>						KeptParameters;
	TArray<FNameId>						KeptNames;

	FDebugIds							Debug;

	FNameId								Remap(FNameId Old) const				{ return Names[Old.Idx].Get(); }
	FMemberId							Remap(FMemberId Old) const				{ return { Remap(Old.Id) }; }
	FFlatScopeId						Remap(FFlatScopeId Old) const			{ return { Remap(Old.Name) }; }
	FNestedScopeId						Remap(FNestedScopeId Old) const			{ return NestedScopes[Old.Idx].Get(); }
	FScopeId							Remap(FScopeId Old) const				{ return Old.IsFlat() ? FScopeId(Remap(Old.AsFlat())) : Old ? FScopeId(Remap(Old.AsNested())) : Old; }
	FConcreteTypenameId					Remap(FConcreteTypenameId Old) const	{ return { Remap(Old.Id) }; }	
	FParametricTypeId					Remap(FParametricTypeId Old) const		{ return ParametricTypes[Old.Idx].Get(); }
	FTypenameId							Remap(FTypenameId Old) const			{ return Old.IsConcrete() ? FTypenameId(Remap(Old.AsConcrete())) : FTypenameId(Remap(Old.AsParametric())); }
	FType								Remap(FType Old) const					{ return { Remap(Old.Scope), Remap(Old.Name) }; }
	FStructSchemaId						RemapStruct(FStructId Old) const;
	FEnumSchemaId						RemapEnum(FEnumId Old) const			{ return Enums[Old.Idx].Get(); }

	template<typename T>
	TOptionalId<T>						Remap(TOptionalId<T> Old) const			{ return Old ? ToOptional(Remap(Old.Get())) : Old; }
};

static TConstArrayView<FNameId> GetUsedNames(const FBuiltStructSchema& Used)
{
	static_assert(sizeof(FMemberId) == sizeof(FNameId));
	return reinterpret_cast<const TArray<FNameId>&>(Used.MemberNames);
}

static TConstArrayView<FNameId> GetUsedNames(const FBuiltEnumSchema& Used)
{
	return Used.Names;
}

struct FUsedIds
{
	const FIdIndexerBase& Ids;
	TBitArray<> Names;
	TBitArray<> NestedScopes;
	TBitArray<> ParametricTypes;
	TArray<FNameId> KeptNames;
	TArray<FNestedScopeId> KeptNestedScopes;
	TArray<FParametricTypeId> KeptParametricTypes;


	explicit FUsedIds(const FIdIndexerBase& InIds)
	: Ids(InIds)
	, Names(false, Ids.NumNames())
	, NestedScopes(false, Ids.GetNestedScopes().Num())
	, ParametricTypes(false, Ids.GetParametricTypes().Num())
	{
		KeptNames.Reserve(16 * (Ids.NumEnums() + Ids.NumStructs()));
		KeptNestedScopes.Reserve(Ids.GetNestedScopes().Num());
		KeptParametricTypes.Reserve(Ids.GetParametricTypes().Num());
	}

	template<class PartialSchemaType>
	void DetectUsage(const TArray<PartialSchemaType>& Schemas)
	{
		for (const PartialSchemaType& Schema : Schemas)
		{
			MarkUsed(Schema.Type);

			for (FNameId Name : GetUsedNames(Schema))
			{
				MarkUsed(Name);
			}
		}
	}

	void MarkUsed(FNameId Name)
	{
		FBitReference Used = Names[Name.Idx];
		if (!Used)
		{
			Used = true;
			KeptNames.Add(Name);
		}
	}
	
	void MarkUsed(FOptionalConcreteTypenameId Name)
	{
		if (Name)
		{
			MarkUsed(Name.Get().Id);
		}
	}
	
	void MarkUsed(FType Type)
	{
		MarkUsed(Type.Scope);
		MarkUsed(Type.Name);
	}

	void MarkUsed(FScopeId Scope)
	{
		if (Scope.IsFlat())
		{
			MarkUsed(Scope.AsFlat().Name);
		}
		else if (Scope)
		{
			FBitReference Used = NestedScopes[Scope.AsNested().Idx];
			if (!Used)
			{
				Used = true;
				KeptNestedScopes.Add(Scope.AsNested());

				FNestedScope Nested = Ids.Resolve(Scope.AsNested());
				MarkUsed(Nested.Outer);
				MarkUsed(Nested.Inner.Name);
			}
		}
	}

	void MarkUsed(FTypenameId Typename)
	{
		if (Typename.IsConcrete())
		{
			MarkUsed(Typename.AsConcrete().Id);
		}
		else
		{
			FBitReference Used = ParametricTypes[Typename.AsParametric().Idx];
			if (!Used)
			{
				Used = true;

				FParametricTypeView ParametricType = Ids.Resolve(Typename.AsParametric());
				MarkUsed(ParametricType.Name);
				for (FType Parameter : ParametricType.GetParameters())
				{
					MarkUsed(Parameter);
				}
				KeptParametricTypes.Add(Typename.AsParametric());
			}
		}
	}
};

template<typename IdType>
static void MakeIdRemapping(TArray<TOptionalId<IdType>>& Out, const TArray<IdType>& Kept, int32 NumDeclared)
{
	uint32 Idx = 0;
	Out.SetNum(NumDeclared);
	for (IdType Id : Kept)
	{
		Out[Id.Idx] = FromIdx<IdType>(Idx++);
	}
}

static uint32 MakeIdRemapping(
	TArray<FOptionalParametricTypeId>& Out,
	const TArray<FParametricTypeId>& Kept,
	const FParametricTypeIndexer& Declared)
{
	uint32 SumParameters = 0;
	uint32 Idx = 0;
	Out.SetNum(Declared.Num());
	for (FParametricTypeId Id : Kept)
	{
		uint8 NumParameters = static_cast<uint8>(Declared.At(Id.Idx).Parameters.NumParameters);
		Out[Id.Idx] = FParametricTypeId(NumParameters, Idx++);
		SumParameters += NumParameters;
	}
	return SumParameters;
}


inline static FNestedScope Resolve(FNestedScopeId Id, const FNestedScopeIndexer& Indexer)
{
	return Indexer.Resolve(Id);
}

inline static FParametricType Resolve(FParametricTypeId Id, const FParametricTypeIndexer& Indexer)
{
	return Indexer.At(Id.Idx);
}

template<typename ItemType, typename IdType, typename IndexerType>
static void CopyUsed(TArray<ItemType>& Out, const TArray<IdType>& Kept, IndexerType&& Indexer)
{
	uint32 Idx = 0;
	Out.SetNumUninitialized(Kept.Num());
	for (IdType Id : Kept)
	{
		Out[Idx++] = Resolve(Id, Indexer);
	}
}

FWriteIds::FWriteIds(const FIdIndexerBase& Ids, const IDeclarations& Decls, const FBuiltSchemas& Schemas, ESchemaFormat Format)
: Declarations(Decls)
, Debug(Ids)
{
	Structs.Init(NoId, Ids.NumStructs());
	Enums.Init(NoId, Ids.NumEnums());

	// Generate new struct and enum schema indices
	uint32 NewSchemaIdx = 0;
	for (const FBuiltStructSchema& Struct : Schemas.Structs)
	{
		Structs[Struct.Id.Idx] = FStructSchemaId{NewSchemaIdx++};
	}
	NumKeptStructSchemas = NewSchemaIdx;
	for (const FBuiltEnumSchema& Enum : Schemas.Enums)
	{
		Enums[Enum.Id.Idx] = FEnumSchemaId{NewSchemaIdx++};
	}
	NumKeptSchemas = NewSchemaIdx;

	if (Format == ESchemaFormat::StableNames)
	{
		// Generate new name, nested scope and parametric type indices
		// in the deterministic order of traversing the built schemas
		FUsedIds Used(Ids);
		Used.DetectUsage(Schemas.Structs);
		Used.DetectUsage(Schemas.Enums);

		// Remap ids and copy used names
		MakeIdRemapping(/* out */ Names, Used.KeptNames, Ids.NumNames());
		KeptNames = MoveTemp(Used.KeptNames);

		// Remap ids and copy used nested scopes
		MakeIdRemapping(/* out */ NestedScopes, Used.KeptNestedScopes, Ids.GetNestedScopes().Num());
		CopyUsed(/* out */ KeptScopes, Used.KeptNestedScopes, Ids.GetNestedScopes());

		// Remap copied nested scopes
		for (FNestedScope& KeptScope : KeptScopes)
		{
			KeptScope.Inner = Remap(KeptScope.Inner);
			KeptScope.Outer = Remap(KeptScope.Outer);
		}

		// Remap ids and copy used parametric types
		uint32 SumParams = MakeIdRemapping(/*out*/ ParametricTypes, Used.KeptParametricTypes, Ids.GetParametricTypes());
		CopyUsed(/* out */ KeptParametrics, Used.KeptParametricTypes, Ids.GetParametricTypes());

		// Remap copied parametric types and copy parameters
		TConstArrayView<FType> AllParameters = Ids.GetParametricTypes().GetAllParameters();
		KeptParameters.Reserve(SumParams);
		for (FParametricType& KeptType : KeptParametrics)
		{
			FParameterIndexRange OldParameters = KeptType.Parameters;
			KeptType.Name = Remap(KeptType.Name);
			KeptType.Parameters.Idx = KeptParameters.Num();
			KeptParameters.Append(AllParameters.Slice(OldParameters.Idx, OldParameters.NumParameters));
		}

		// Remap copied parameters
		for (FType& KeptParameter : KeptParameters)
		{
			KeptParameter = Remap(KeptParameter);
		}
	}

	check(HasStableNames() == (Format == ESchemaFormat::StableNames));
}

FStructSchemaId FWriteIds::RemapStruct(FStructId OldId) const
{
	if (FOptionalStructSchemaId WriteDeclId = Structs[OldId.Idx])
	{
		return WriteDeclId.Get();
	}

	// Could optimize by caching Structs[OldBindId.Idx] here
	FDeclId OldDeclId = Declarations.Lower(FBindId(OldId));
	return Structs[OldDeclId.Idx].Get();
}

//////////////////////////////////////////////////////////////////////////

static TArray<FMemberType> GetMemberTypes(const FBuiltStructSchema& Struct)
{
	TArray<FMemberType> Out;
	Out.Reserve(Struct.MemberSchemas.Num());
	for (const FMemberSchema* Schema : Struct.MemberSchemas)
	{
		Out.Add(Schema->Type);
	}
	return Out;
}

static TArray<FMemberType> GetInnerRangeTypes(const FBuiltStructSchema& Struct)
{
	TArray<FMemberType> Out;
	for (const FMemberSchema* Schema : Struct.MemberSchemas)
	{
		Out.Append(Schema->GetInnerRangeTypes());
	}
	return Out;
}

static FOptionalSchemaId GetStaticInnerSchema(const FMemberSchema& Schema, const FWriteIds& NewIds)
{
	if (FOptionalInnerId InnerSchema = Schema.InnerSchema)
	{
		FMemberType InnermostType = Schema.GetInnermostType();
		check(IsStructOrEnum(InnermostType));
		if (InnermostType.IsLeaf())
		{
			return { NewIds.RemapEnum(InnerSchema.Get().AsEnum()) };
		}
		else if (!InnermostType.AsStruct().IsDynamic)
		{
			return { NewIds.RemapStruct(InnerSchema.Get().AsStructBindId()) };
		}
	}
	return {};
}

static TArray<FSchemaId> GetInnerSchemas(const FBuiltStructSchema& Struct, const FWriteIds& NewIds, ESuper Inheritance)
{
	TArray<FSchemaId> Out;	
	if (Inheritance != ESuper::No && Inheritance != ESuper::Reused)
	{
		Out.Add(NewIds.RemapStruct(FBindId(Struct.Super.Get())));
	}

	for (const FMemberSchema* Schema : Struct.MemberSchemas)
	{
		if (FOptionalSchemaId InnerSchema = GetStaticInnerSchema(*Schema, NewIds))
		{
			Out.Add(InnerSchema.Get());
		}
	}
	return Out;
}

template<typename IdType>
static TArray<IdType> RemapIds(const FWriteIds& NewIds, const TArray<IdType>& Names)
{
	TArray<IdType> Out = Names;
	for (IdType& Name : Out)
	{
		Name = NewIds.Remap(Name);
	}
	return Out;
}

static ESuper GetInheritance(FOptionalStructId Super, TConstArrayView<const FMemberSchema*> Members)
{
	if (Super)
	{
		if (Members.IsEmpty() || !IsSuper(Members[0]->Type))
		{
			return ESuper::Unused;
		}

		if (Super != ToOptionalStruct(Members[0]->InnerSchema))
		{
			return ESuper::Used;
		}

		check(!Members[0]->Type.AsStruct().IsDynamic);
		return ESuper::Reused;
	}

	return ESuper::No;
}

static void WriteSchema(TArray64<uint8>& Out, const FBuiltStructSchema& Struct, const FWriteIds& NewIds)
{
	ESuper Inheritance = GetInheritance(Struct.Super, Struct.MemberSchemas);
	TArray<FMemberType> MemberTypes = GetMemberTypes(Struct);
	TArray<FMemberType> InnerRangeTypes = GetInnerRangeTypes(Struct);
	TArray<FMemberId> MemberNames = NewIds.HasStableNames() ? RemapIds(NewIds, Struct.MemberNames) : Struct.MemberNames;
	TArray<FSchemaId> InnerSchemas = GetInnerSchemas(Struct, NewIds, Inheritance);

	checkf(MemberNames.Num() + UsesSuper(Inheritance) == MemberTypes.Num(), 
		TEXT("'%s' has %d member names and %d unnamed super but %d types"), 
		*NewIds.Debug.Print(Struct.Type), MemberNames.Num(), UsesSuper(Inheritance), MemberTypes.Num());

	// Zero init for determinism
	alignas(FStructSchema) uint8 HeaderData[offsetof(FStructSchema, Footer)] = {};
	FStructSchema& BinaryHeader = *reinterpret_cast<FStructSchema*>(HeaderData);
	BinaryHeader.Type = NewIds.HasStableNames() ? NewIds.Remap(Struct.Type) : Struct.Type;
	BinaryHeader.Inheritance = Inheritance;
	BinaryHeader.IsDense = Struct.bDense;
	BinaryHeader.NumMembers = IntCastChecked<uint16>(MemberTypes.Num());
	BinaryHeader.NumRangeTypes = IntCastChecked<uint16>(InnerRangeTypes.Num());
	BinaryHeader.NumInnerSchemas = IntCastChecked<uint16>(InnerSchemas.Num());

	const int64 HeaderPos = Out.Num(); (void)HeaderPos;
	check(IsAligned(HeaderPos, alignof(FStructSchema)));
	WriteData(Out, HeaderData, sizeof(HeaderData));
	WriteArray(Out, MemberTypes);
	WriteArray(Out, InnerRangeTypes);
	WriteAlignedArray(Out, MakeArrayView(MemberNames));
	WriteAlignedArray(Out, MakeArrayView(InnerSchemas));
	check(IsAligned(Out.Num(), alignof(FStructSchema)));
	check(Out.Num() - HeaderPos == CalculateSize(*reinterpret_cast<FStructSchema*>(&Out[HeaderPos])));
}

static bool IsFlatSequence(TConstArrayView<uint64> Constants)
{
	uint64 Expected = 0;
	for (uint64 Constant : Constants)
	{
		if (Constant != Expected)
		{
			return false;
		}
		++Expected;
	}
	return true;
}
static bool IsFlagSequence(TConstArrayView<uint64> Constants)
{
	check(Constants.Num() <= 64);
	uint64 Expected = 1;
	for (uint64 Constant : Constants)
	{
		if (Constant != Expected)
		{
			return false;
		}
		Expected <<= 1;
	}
	return true;
}

template<typename IntType>
static void WriteEnumConstantsAs(TArray64<uint8>& Out, TConstArrayView<uint64> Constants)
{
	TArray<IntType, TInlineAllocator<64>> Tmp;
	for (uint64 Constant : Constants)
	{
		Tmp.Add(IntCastChecked<IntType>(Constant));
	}
	WriteArray(Out, Tmp);
}

static void WriteEnumConstants(TArray64<uint8>& Out, ELeafWidth Width, TConstArrayView<uint64> Constants)
{
	switch (Width)
	{
		case ELeafWidth::B8:	WriteEnumConstantsAs<uint8 >(Out, Constants); break;
		case ELeafWidth::B16:	WriteEnumConstantsAs<uint16>(Out, Constants); break;
		case ELeafWidth::B32:	WriteEnumConstantsAs<uint32>(Out, Constants); break;
		case ELeafWidth::B64:	WriteArray(Out, Constants); break;
	}
}

static void WriteSchema(TArray64<uint8>& Out, const FBuiltEnumSchema& Enum, const FWriteIds& NewIds)
{
	bool bIsSequence = Enum.Mode == EEnumMode::Flag ? IsFlagSequence(Enum.Constants) : IsFlatSequence(Enum.Constants);
	TArray<FNameId> Names = NewIds.HasStableNames() ? RemapIds(NewIds, Enum.Names) : Enum.Names;

	FEnumSchema BinaryHeader = { NewIds.HasStableNames() ? NewIds.Remap(Enum.Type) : Enum.Type };
	BinaryHeader.FlagMode = Enum.Mode == EEnumMode::Flag;
	BinaryHeader.ExplicitConstants = !bIsSequence;
	BinaryHeader.Width = Enum.Width;
	BinaryHeader.Num = IntCastChecked<uint16>(Names.Num());

	const int64 HeaderPos = Out.Num(); (void)HeaderPos;
	check(IsAligned(HeaderPos, alignof(FEnumSchema)));
	WriteData(Out, &BinaryHeader, sizeof(BinaryHeader));
	WriteArray(Out, Names);
	if (BinaryHeader.ExplicitConstants)
	{
		WriteEnumConstants(Out, Enum.Width, Enum.Constants);
	}
	WriteAlignmentPadding<FEnumSchema>(Out);
	check(Out.Num() - HeaderPos == CalculateSize(*reinterpret_cast<FEnumSchema*>(&Out[HeaderPos])));
}

template<typename T>
void AppendBinary(TArray64<uint8>& Dst, const TArray<T>& Src)
{
	if (uint32 NumBytes = Src.Num() * sizeof(T))
	{
		Dst.AddUninitialized(NumBytes);
		FMemory::Memcpy(Dst.GetData() + Dst.Num() - NumBytes, Src.GetData(), NumBytes);
	}
}

static void WriteSchemasImpl(TArray64<uint8>& Out, const FBuiltSchemas& Schemas, const FWriteIds& NewIds)
{
	// @see FReadSchemaBatch

	WriteAlignmentPadding<uint32>(Out);
	const int64 HeaderPos = Out.Num();
	const uint32 NumSchemas = NewIds.NumKeptSchemas;
	Out.AddUninitialized(sizeof(FSchemaBatch) + NumSchemas * sizeof(uint32));

	TArray<uint32> SchemaOffsets;
	SchemaOffsets.Reserve(NumSchemas);
	auto WritePartialSchemas = [&Out, &SchemaOffsets, HeaderPos, &NewIds](auto& PartialSchemas)
	{
		for (auto& PartialSchema : PartialSchemas)
		{
			SchemaOffsets.Add(IntCastChecked<uint32>(Out.Num() - HeaderPos));
			WriteSchema(Out, PartialSchema, NewIds);
		}
	};

	WritePartialSchemas(Schemas.Structs);
	WritePartialSchemas(Schemas.Enums);
	check(SchemaOffsets.Num() == NumSchemas);

	// Write header 
	const int64 NestedScopePos = Out.Num();
	FSchemaBatch Header{0};
	Header.NumNestedScopes = NewIds.KeptScopes.Num();
	Header.NestedScopesOffset = IntCastChecked<uint32>(NestedScopePos - HeaderPos);
	Header.NumParametricTypes = NewIds.KeptParametrics.Num();
	Header.NumSchemas = NewIds.NumKeptSchemas;
	Header.NumStructSchemas = NewIds.NumKeptStructSchemas;
	FMemory::Memcpy(&Out[HeaderPos], &Header, sizeof(FSchemaBatch));

	// Write schema offsets
	FMemory::Memcpy(&Out[HeaderPos] + sizeof(FSchemaBatch), SchemaOffsets.GetData(), NumSchemas * sizeof(uint32));

	AppendBinary(Out, NewIds.KeptScopes);
	AppendBinary(Out, NewIds.KeptParametrics);
	AppendBinary(Out, NewIds.KeptParameters);
}

//////////////////////////////////////////////////////////////////////////

uint64 WriteSkippableSlice(TArray64<uint8>& Out, TConstArrayView64<uint8> Slice)
{
	if (uint64 Num = IntCastChecked<uint64>(Slice.Num()))
	{
		uint32 VarIntBytes = MeasureVarUInt(Num);
		int64 VarIntPos = Out.AddUninitialized(VarIntBytes + Slice.Num());
		WriteVarUInt(Num, &Out[VarIntPos]);
		FMemory::Memcpy(&Out[VarIntPos + VarIntBytes], Slice.GetData(), Slice.Num());
		return VarIntBytes + Num;
	}

	constexpr uint8 ZeroVarUInt = 0; 
	Out.Add(ZeroVarUInt);
	return 1;
}

//////////////////////////////////////////////////////////////////////////

class FBitCacheWriter
{
	static constexpr int64 Unused = -1;
	static constexpr int64 Finished = -2;

	uint8 Bits = 0;
	uint32 NumLeft = 0;
	TArray64<uint8>& Dest;
	int64 DestIdx = Unused;

	void DoFlush()
	{
		if (DestIdx >= 0)
		{
			Dest[DestIdx] = Bits;
		}
	}

public:
	explicit FBitCacheWriter(TArray64<uint8>& Out) : Dest(Out) {}
	~FBitCacheWriter()
	{
		check(DestIdx == Finished);
	}

	void WriteBit(bool Bit)
	{
		if (NumLeft == 0)
		{
			check(DestIdx != Finished);
			DoFlush();
			
			DestIdx = Dest.Num();
			Dest.Add(0);

			Bits = Bit;
			NumLeft = 7;
		}
		else
		{
			Bits |= uint8(Bit) << (8 - NumLeft);
			--NumLeft;
		}
	}

	void Flush()
	{
		DoFlush();
		NumLeft = 0;
		DestIdx = Finished;
	}
};

//////////////////////////////////////////////////////////////////////////

class FMemberWriter
{
public:
	FMemberWriter(TArray64<uint8>& Out, const FBuiltSchemas& InSchemas, const FWriteIds& InNewIds, const FDebugIds& InDebug)
	: Bytes(Out)
	, Bits(Out)
	, Schemas(InSchemas)
	, NewIds(InNewIds)
	, Debug(InDebug)
	{}

	FStructSchemaId WriteMembers(FStructId BuiltId, const FBuiltStruct& Struct)
	{
		FStructSchemaId WriteId = NewIds.RemapStruct(FBindId(BuiltId));
		const FBuiltStructSchema& Schema = Schemas.Structs[WriteId.Idx];
		const TArray<FMemberId>& Order = Schema.MemberNames;
		const int32 NumSuper = Schema.MemberSchemas.Num() - Schema.MemberNames.Num();
		check(NumSuper == 0 || NumSuper == 1 && IsSuper(Schema.MemberSchemas[0]->Type));
		check(Struct.NumMembers <= Schema.MemberSchemas.Num());

		int32 Idx = 0;
		if (Schema.bDense)
		{
			for (const FBuiltMember& Member : MakeArrayView(Struct.Members, Struct.NumMembers))
			{
				checkf(Member.Name == NoId || Order[Idx - NumSuper] == Member.Name.Get(), TEXT("Member '%s' in '%s' %s %s"),
					*Debug.Print(Member.Name), *Debug.Print(Schema.Type),
					Order.Contains(Member.Name) ? TEXT("appeared before missing member") : TEXT("is undeclared"),
					Order.Contains(Member.Name) ? *Debug.Print(Order[Idx]) : TEXT(""));

				WriteMember(Schema.MemberSchemas[Idx]->GetInnermostType(), Member.Schema, Member.Value);
				++Idx;
			}
		}
		else
		{
			for (const FBuiltMember& Member : MakeArrayView(Struct.Members, Struct.NumMembers))
			{
				for (bool bSkip = true; bSkip; ++Idx) 
				{
					checkf(Idx - NumSuper < Order.Num(), TEXT("Member '%s' in '%s' %s"), *Debug.Print(Member.Name), *Debug.Print(Schema.Type), 
						   Order.Contains(Member.Name) ? TEXT("appeared in non-declared order") : TEXT("is undeclared"));

					bSkip = Member.Name && (Idx < NumSuper || Order[Idx - NumSuper] != Member.Name.Get());
					Bits.WriteBit(bSkip);
				}

				WriteMember(Schema.MemberSchemas[Idx - 1]->GetInnermostType(), Member.Schema, Member.Value);
			}

			// Skip remaining missing members
			for (int32 Num = Schema.MemberSchemas.Num(); Idx < Num; ++Idx)
			{
				Bits.WriteBit(true);
			}
		}

		Bits.Flush();

		return WriteId;
	}

private:
	TArray64<uint8>& Bytes;
	FBitCacheWriter Bits;
	TArray64<uint8> Tmp;
	const FBuiltSchemas& Schemas;
	const FWriteIds& NewIds;
	const FDebugIds& Debug;

	// Tricky! InnermostType comes from FBuiltStructSchema and it's IsDynamic is decided during noting.
	// Schema.GetInnermostType() IsDynamic is false, it's from immutable FBuiltMember built before noting.
	void WriteMember(FMemberType InnermostType, const FMemberSchema& Schema, const FBuiltValue& Value)
	{
		if (InnermostType.IsStruct() && InnermostType.AsStruct().IsDynamic)
		{
			FSchemaId WriteId = NewIds.RemapStruct(FBindId(Schema.InnerSchema.Get().AsStruct()));
			Bytes.Append(reinterpret_cast<uint8*>(&WriteId.Idx), sizeof(WriteId.Idx));
		}

		switch (Schema.Type.GetKind())
		{
		case EMemberKind::Leaf:		WriteLeaf(Schema.Type.AsLeaf(), Value.Leaf); break;
		case EMemberKind::Range:	WriteRange(Schema.Type.AsRange().MaxSize, Schema.GetInnerRangeTypes(), Schema.InnerSchema, Value.Range); break;
		case EMemberKind::Struct:	WriteStruct(Schema.Type.AsStruct(), Schema.InnerSchema.Get().AsStruct(), *Value.Struct); break;
		}
	}

	void WriteLeaf(FLeafType Leaf, uint64 LeafValue)
	{
		if (Leaf.Type == ELeafType::Bool)
		{
			check(LeafValue <= 1);
			Bits.WriteBit(!!LeafValue);
		}
		else
		{
			WriteUnsigned(LeafValue, SizeOf(Leaf.Width));
		}
	}
		
	void WriteStruct(FStructType StructType, FStructId Id, const FBuiltStruct& Struct)
	{
		Tmp.Reserve(1024);
		FMemberWriter(Tmp, Schemas, NewIds, Debug).WriteMembers(Id, Struct);
		WriteSkippableSlice(Bytes, Tmp);
		Tmp.Reset();
	}

	void WriteRange(ESizeType NumType, TConstArrayView<FMemberType> Types, FOptionalInnerId InnermostSchema, const FBuiltRange* Range)
	{
		check(Types.Num() > 0);
		check((Types.Num() > 1) == (Types[0].GetKind() == EMemberKind::Range));
		check(!Range || Range->Num > 0 && Range->Num <= Max(NumType));

		// Write Num
		uint64 Num = Range ? Range->Num : 0;
		if (NumType == ESizeType::Uni)
		{
			Bits.WriteBit(Num == 1);
		}
		else
		{
			WriteUnsigned(Num, SizeOf(NumType));
		}
		
		// Write Data
		if (Range)
		{
			switch (Types[0].GetKind())
			{
			case EMemberKind::Leaf:		WriteLeaves(Types[0].AsLeaf(), *Range);	break;
			case EMemberKind::Range:	WriteRanges(Types[0].AsRange().MaxSize, Types.RightChop(1), InnermostSchema, Range->AsRanges()); break;
			case EMemberKind::Struct:	WriteStructs(Types[0].AsStruct(), InnermostSchema.Get().AsStruct(), Range->AsStructs()); break;
			}
		}
	}

	void WriteLeaves(FLeafType Leaf, const FBuiltRange& Range)
	{
		if (Leaf.Type == ELeafType::Bool)
		{
			FBitCacheWriter BitArray(/* Out */ Bytes);
			for (uint8 Bool : TConstArrayView64<uint8>(Range.Data, Range.Num))
			{
				check(Bool <= 1);
				BitArray.WriteBit(!!Bool);
			}
			BitArray.Flush();
		}
		else
		{
			WriteData(Bytes, Range.Data, GetLeafRangeSize(Range.Num, Leaf));
		}
	}

	template<typename ItemType, typename WriteFn>
	void WriteSkippableItems(TConstArrayView64<ItemType> Items, WriteFn WriteItem)
	{
		Tmp.Reserve(1024);
		FMemberWriter NestedWriter(Tmp, Schemas, NewIds, Debug);
		for (const ItemType& Item : Items)
		{
			WriteItem(NestedWriter, Item);
		}
		NestedWriter.Bits.Flush();

		WriteSkippableSlice(/* out */ Bytes, Tmp);
		Tmp.Reset();
	}

	void WriteStructs(FStructType StructType, FStructId BuiltId, TConstArrayView64<const FBuiltStruct*> Structs)
	{
		FStructSchemaId WriteId = NewIds.RemapStruct(BuiltId);
		Bytes.Append(reinterpret_cast<uint8*>(&WriteId.Idx), StructType.IsDynamic * sizeof(WriteId.Idx));
		WriteSkippableItems(Structs, [=](FMemberWriter& Out, const FBuiltStruct* Struct) { Out.WriteStruct(StructType, BuiltId, *Struct); });
	}
	
	void WriteRanges(ESizeType NumType, TConstArrayView<FMemberType> Types, FOptionalInnerId InnermostSchema, TConstArrayView64<const FBuiltRange*> Ranges)
	{	
		WriteSkippableItems(Ranges, [=](FMemberWriter& Out, const FBuiltRange* Range) { Out.WriteRange(NumType, Types, InnermostSchema, Range); });
	}

	void WriteUnsigned(uint64 Value, SIZE_T SizeOf)
	{
		check(SizeOf == 8 || (Value >> (SizeOf*8)) == 0);
		Bytes.Append(reinterpret_cast<uint8*>(&Value), SizeOf);
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FWriter::FWriter(const FIdIndexerBase& AllIds, const IDeclarations& Declarations, const FBuiltSchemas& InSchemas, ESchemaFormat Format)
: Schemas(InSchemas)
, Debug(AllIds)
, NewIds(new FWriteIds(AllIds, Declarations, InSchemas, Format))
{}

FWriter::~FWriter()
{}

TConstArrayView<FNameId> FWriter::GetUsedNames() const
{
	return NewIds->KeptNames;
}

FOptionalStructSchemaId FWriter::GetWriteId(FStructId BuiltId) const
{
	return static_cast<FOptionalStructSchemaId>(NewIds->Structs[BuiltId.Idx]);
}

void FWriter::WriteSchemas(TArray64<uint8>& Out) const
{
	WriteSchemasImpl(Out, Schemas, *NewIds);
}

FStructSchemaId FWriter::WriteMembers(TArray64<uint8>& Out, FStructId BuiltId, const FBuiltStruct& Struct) const
{
	return FMemberWriter(Out, Schemas, *NewIds, Debug).WriteMembers(BuiltId, Struct);
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace PlainProps
