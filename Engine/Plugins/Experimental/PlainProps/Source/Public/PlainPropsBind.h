// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/SparseSet.h"
#include "Memory/MemoryFwd.h"
#include "Memory/MemoryView.h"
#include "PlainPropsDeclare.h"
#include "PlainPropsRead.h"
#include "PlainPropsTypes.h"

namespace PlainProps 
{

class FBaseline;
struct FBindContext;
struct FBuiltRange;
struct FBuiltStruct;
struct FCustomBindingHandle;
struct FDiffContext;
class FIdIndexerBase;
struct FLoadBatch;
class FMemberBuilder;
struct FMemcpyLoadPlan;
class FRangeLoader;
struct FSchemaBatch;
class FScratchAllocator;
struct FStructLoadView;
class FRangeBinding;
struct FSaveContext;
template<class T> class TIdIndexer;

////////////////////////////////////////////////////////////////////////////////////////////////

enum class ELeafBindType : uint8 { Bool, IntS, IntU, Float, Hex, Enum, Unicode, BitfieldBool };

inline static constexpr ELeafBindType ToLeafBindType(ELeafType Type)
{
	return static_cast<ELeafBindType>(static_cast<uint8>(Type));
}

inline static constexpr ELeafType ToLeafType(ELeafBindType Type)
{
	return Type == ELeafBindType::BitfieldBool ? ELeafType::Bool : static_cast<ELeafType>(static_cast<uint8>(Type));
}

struct FBasicLeafBindType
{
    EMemberKind		_ : 2;
	ELeafBindType	__ : 3;
	ELeafWidth		Width : 2;
	uint8			_Pad : 1;
};

struct FBitfieldBoolBindType
{
    EMemberKind		_ : 2;
	ELeafBindType	__ : 3;
	uint8			Idx : 3;
};

union FLeafBindType
{
	constexpr explicit FLeafBindType(ELeafBindType BasicType, ELeafWidth Width) : Basic({EMemberKind::Leaf, BasicType, Width}) {}
	constexpr explicit FLeafBindType(FUnpackedLeafType In) : Basic({EMemberKind::Leaf, ToLeafBindType(In.Type), In.Width}) {}
	constexpr explicit FLeafBindType(FLeafType In) : FLeafBindType(FUnpackedLeafType(In)) {}
	constexpr explicit FLeafBindType(FBitfieldBoolBindType In) : Bitfield({EMemberKind::Leaf, ELeafBindType::BitfieldBool, In.Idx}) {}
	constexpr explicit FLeafBindType(uint8 BitfieldIdx) : Bitfield({EMemberKind::Leaf, ELeafBindType::BitfieldBool, BitfieldIdx}) {}

	struct
	{
		EMemberKind			_ : 2;
		ELeafBindType		Type : 3;
		uint8				_Pad : 3;
	}						Bind;
	FBasicLeafBindType		Basic;
	FBitfieldBoolBindType	Bitfield;
};

inline static constexpr FLeafType ToLeafType(FLeafBindType Leaf)
{
	if (Leaf.Bind.Type == ELeafBindType::BitfieldBool)
	{
		return { EMemberKind::Leaf, ELeafWidth::B8, ELeafType::Bool };
	}
	
	return { EMemberKind::Leaf,  Leaf.Basic.Width, ToLeafType(Leaf.Bind.Type) };
}

struct FRangeBindType : FRangeType {};

struct FStructBindType : FStructType {};

union FMemberBindType
{
	constexpr explicit FMemberBindType(FLeafType In) : Leaf(In) {}
	constexpr explicit FMemberBindType(FUnpackedLeafType In) : Leaf(In) {}
	constexpr explicit FMemberBindType(FLeafBindType In) : Leaf(In) {}
	constexpr explicit FMemberBindType(FBitfieldBoolBindType In) : Leaf(In) {}
	constexpr explicit FMemberBindType(FRangeType In) : Range(In) {}
	constexpr explicit FMemberBindType(ESizeType MaxSize) : Range({EMemberKind::Range, MaxSize}) {}
	constexpr explicit FMemberBindType(FStructType In) : Struct(In) {}
	
	bool					IsLeaf() const		{ return Kind == EMemberKind::Leaf; }
	bool					IsRange() const		{ return Kind == EMemberKind::Range; }
	bool					IsStruct() const	{ return Kind == EMemberKind::Struct; }
	EMemberKind				GetKind() const		{ return Kind; }
	
	FLeafBindType			AsLeaf() const		{ checkSlow(IsLeaf());		return Leaf; }
	FRangeBindType			AsRange() const		{ checkSlow(IsRange());		return Range; }
	FStructBindType			AsStruct() const	{ checkSlow(IsStruct());	return Struct; }
	uint8					AsByte() const		{ return BitCast<uint8>(*this); }

	friend inline bool operator==(FMemberBindType A, FMemberBindType B) { return A.AsByte() == B.AsByte(); }
	friend inline uint32 GetTypeHash(FMemberBindType In) { return In.AsByte(); }
private:
    EMemberKind				Kind : 2;
	FLeafBindType			Leaf;
    FRangeBindType			Range;
    FStructBindType			Struct;
};

static_assert(sizeof(FMemberBindType) == 1);

////////////////////////////////////////////////////////////////////////////////////////////////

// Members are loaded in saved FStructSchema order, not current offset order unless upgrade layer reorders
struct alignas(/*FRangeBinding*/ 8) FSchemaBinding
{
	FDeclId					DeclId;
	uint16					NumMembers;
	uint16					NumInnerIds;
	uint16					NumInnerRanges;
	FMemberBindType			Members[0];

	const FMemberBindType*	GetInnerRangeTypes() const	{ return Members + NumMembers; }
	const uint32*			GetOffsets() const			{ return AlignPtr<uint32>(GetInnerRangeTypes() + NumInnerRanges); }
	const FInnerId*			GetInnerIds() const			{ return AlignPtr<FInnerId>(GetOffsets() + NumMembers); }
	const FRangeBinding*	GetRangeBindings() const	{ return AlignPtr<FRangeBinding>(GetInnerIds() + NumInnerIds); }
	uint32					CalculateSize() const;
	bool					HasSuper() const			{ return NumInnerIds > 0 && Members[0].IsStruct() && Members[0].AsStruct().IsSuper; }
};

////////////////////////////////////////////////////////////////////////////////////////////////

struct FUnpackedLeafBindType
{
	ELeafBindType			Type;
	union
	{
		ELeafWidth			Width;
		uint8				BitfieldIdx;
	};
	
	constexpr FUnpackedLeafBindType(FLeafBindType In)
	: Type(In.Bind.Type)
	{
		if (Type != ELeafBindType::BitfieldBool)
		{
			Width = In.Basic.Width;
		}
		else
		{
			BitfieldIdx = In.Bitfield.Idx;
		}
	}

	FMemberBindType Pack() const
	{ 
		return FMemberBindType(Type == ELeafBindType::BitfieldBool ? FLeafBindType(BitfieldIdx) : FLeafBindType(Type, Width));
	}
};

inline static constexpr FUnpackedLeafType ToUnpackedLeafType(FUnpackedLeafBindType Leaf)
{
	if (Leaf.Type == ELeafBindType::BitfieldBool)
	{
		return { ELeafType::Bool, ELeafWidth::B8 };
	}
	
	return { ToLeafType(Leaf.Type), Leaf.Width };
}

// @pre Type != ELeafBindType::BitfieldBool
inline FUnpackedLeafType UnpackNonBitfield(FLeafBindType Packed)
{
	FUnpackedLeafBindType Unpacked(Packed);
	return { ToLeafType(Unpacked.Type), Unpacked.Width };
}

struct FLeafMemberBinding
{
	FUnpackedLeafBindType	Leaf;
	FOptionalEnumId			Enum;
	SIZE_T					Offset;
};

struct FRangeMemberBinding
{
	const FMemberBindType*	InnerTypes;
	const FRangeBinding*	RangeBindings;
	uint16					NumRanges; // At least 1, >1 for nested ranges
	FOptionalInnerId		InnermostId;
	SIZE_T					Offset;
};

struct FStructMemberBinding
{
	FStructType				Type;
	FBindId					Id;
	SIZE_T					Offset;
};

PLAINPROPS_API FRangeMemberBinding GetInnerRange(FRangeMemberBinding In); // @pre In.NumRanges > 1

////////////////////////////////////////////////////////////////////////////////////////////////

struct FBothStructId
{
	FBindId					BindId;
	FDeclId					DeclId;

	bool					IsLowered() const { return BindId != DeclId; }
};

// FStructId statically known to share FBindId and FDeclId, i.e. not lowered
struct FDualStructId : FStructId
{
	explicit FDualStructId(FStructId In) : FStructId(In) {}
	
	operator FBindId() const { return {Idx}; }
	operator FDeclId() const { return {Idx}; }
};

////////////////////////////////////////////////////////////////////////////////////////////////

// Helps raise lowered FDeclId by seeing which FBindId a particular named member has
struct FInnerStruct
{
	FMemberId				Name; // Of outer Range or Struct
	FBindId					Id;
};

enum ECustomLoadMethod { Construct, Assign };

/// Load/save a struct with custom code to handle:
/// * reference types
/// * private members
/// * non-default constructible types
/// * custom delta semantics
/// * other runtime representations than struct/class, e.g. serialize database
/// * optimizations of very common structs
struct ICustomBinding
{
	virtual ~ICustomBinding() {}
	virtual void	SaveCustom(FMemberBuilder& Dst, const void* Src, FBaseline Base, const FSaveContext& Ctx) = 0;
	virtual void	LoadCustom(void* Dst, FStructLoadView Src, ECustomLoadMethod Method) const = 0;
	virtual bool	DiffCustom(const void* StructA, const void* StructB, const FBindContext& Ctx) const = 0;

	// Overload to load via a single memcpy call instead of LoadCustom, Out.Offset is 0 by default
	virtual void	PlanCustom(FMemcpyLoadPlan& Out) const {}

	// Overload to track the first or all diffing members, see FDiffTree
	PLAINPROPS_API virtual bool	DiffCustom(const void* StructA, const void* StructB, FDiffContext& Ctx) const;
	
};

struct FInnersHandle
{
	uint32			Num = 0;
	uint32			Idx = 0;
};

struct FIdWindow
{
	uint32			Min = 0;
	uint32			Num = 0;
};

// Slightly optimized map from FStructId to ICustomBinding/FStructDeclaration/FInnersHandle
struct FCustomBindingMap
{
	UE_NONCOPYABLE(FCustomBindingMap);
	explicit FCustomBindingMap(FDebugIds Dbg);
	~FCustomBindingMap();

	void								Bind(FBindId Id, ICustomBinding& Binding, FStructDeclarationPtr&& Declaration, FInnersHandle LoweredInners);
	FCustomBindingHandle				Find(FBindId Id) const;
	void								Drop(FBindId Id);

	FIdWindow							Window;
	TSparseSet<FBindId>					Keys; // Need FSetElementId stability when dropping
	ICustomBinding**					Values = nullptr;
	uint32								MaxValues = 0;
	TArray<FStructDeclarationPtr>		Declarations; // Consider switch to TSparseArray or TMap
	FDebugIds							Debug;
};

class FCustomBindings
{
public:
	// @param Binding must outlive this or call DropStruct()
	PLAINPROPS_API void							BindStruct(FBindId Id, ICustomBinding& Binding, FStructDeclarationPtr&& Declaration, TConstArrayView<FInnerStruct> LoweredInners);
	PLAINPROPS_API void							BindStruct(FBindId Id, ICustomBinding& Binding, const FStructSpec& DeclSpec, TConstArrayView<FInnerStruct> LoweredInners);
	PLAINPROPS_API const ICustomBinding*		FindStruct(FBindId Id) const;
	PLAINPROPS_API const ICustomBinding*		FindStruct(FBindId Id, TConstArrayView<FInnerStruct>& OutLoweredInners) const;
	PLAINPROPS_API const ICustomBinding*		FindStruct(FBindId Id, const FStructDeclaration*& OutDeclaration) const;
	PLAINPROPS_API ICustomBinding*				FindStructToSave(FBindId Id, const FStructDeclaration*& OutDeclaration) const;
	PLAINPROPS_API const FStructDeclaration*	FindDeclaration(FBindId Id) const;
	PLAINPROPS_API void							DropStruct(FBindId Id);

	virtual FCustomBindingHandle				Find(FBindId Id) const = 0;
protected:
	PLAINPROPS_API FCustomBindings(TArray<FInnerStruct>& Inners, FDebugIds Dbg);
	PLAINPROPS_API FCustomBindings(const FCustomBindings* Under);
	PLAINPROPS_API ~FCustomBindings();

	FCustomBindingMap							Map;
	TArray<FInnerStruct>&						BottomInners;
};

class FCustomBindingsBottom final : public FCustomBindings
{
	mutable TArray<FInnerStruct>				AllInners;
	PLAINPROPS_API virtual FCustomBindingHandle	Find(FBindId Id) const override;	
public:
	explicit FCustomBindingsBottom(FDebugIds Dbg) : FCustomBindings(/* unconstructed */  AllInners, Dbg) {}
};

class FCustomBindingsOverlay final : public FCustomBindings
{
	const FCustomBindings&						Underlay;
	PLAINPROPS_API virtual FCustomBindingHandle	Find(FBindId Id) const override;
public:
	explicit FCustomBindingsOverlay(const FCustomBindings& Under) : FCustomBindings(&Under), Underlay(Under) {}
};

////////////////////////////////////////////////////////////////////////////////////////////////

// Request from loading layer to IItemRangeBinding to allocate and construct items
class FConstructionRequest
{
	friend FRangeLoader;

	void* const		Range = nullptr;
	const uint64	Num = 0;
	uint64			Index = 0;

	FConstructionRequest(void* InRange, uint64 InNum) : Range(InRange), Num(InNum) {}

public:
	template<typename T>
	T&				GetRange() const		{ return *reinterpret_cast<T*>(Range); }
	uint64			NumTotal() const		{ return Num; }
	uint64			NumMore() const			{ return Num - Index; }
	uint64			GetIndex() const		{ return Index; }
	bool			IsFirstCall() const		{ return Index == 0;}
	bool			IsFinalCall() const		{ return Index == Num;}
};

// Response from IItemRangeBinding with contiguous items ready to be loaded
//
// Non-contiguous containers provide one items one by one or use
// FLoadRangeContext::Scratch or a temp allocation to avoid that
class FConstructedItems
{
public:
	// E.g. allow hash table to rehash after all items are loaded
	void RequestFinalCall() { bNeedFinalize = true; }

	void SetUnconstructed() { bUnconstructed = true; }

	// Non-contiguous items must be set individually
	template<typename ItemType>
	void Set(ItemType* Items, uint64 NumItems)
	{
		Set(Items, NumItems, sizeof(ItemType));
	}

	void Set(void* Items, uint64 NumItems, uint32 ItemSize)
	{
		check(NumItems == 0 || Items != Data);
		Data = reinterpret_cast<uint8*>(Items);
		Num = NumItems;
		Size = ItemSize;
	}
	
	void UpdateNum(uint64 NumItems)
	{
		check(Data);
		Num = NumItems;
	}

	template<typename ItemType>
	TArrayView<ItemType> Get()
	{ 
		return MakeArrayView(reinterpret_cast<ItemType*>(Data), Num);
	}

	inline uint8* GetData()		{ return Data; }
	inline uint8* GetDataEnd()	{ return Data + NumBytes(); }

private:
	friend FRangeLoader;
	uint8*	Data = nullptr;
	uint64	Num = 0;			
	uint32	Size = 0;
	bool	bNeedFinalize = false;
	bool	bUnconstructed = false;

	uint64	NumBytes() const { return Num * Size; }
};

struct FLoadRangeContext
{
	FConstructionRequest	Request;		// Request to construct items to be loaded
	FConstructedItems		Items;			// Response from IItemRangeBinding
	uint64					Scratch[64];	// Scratch memory for IItemRangeBinding
};

// todo: switch to class
struct FGetItemsRequest
{
	template<typename T>
	const T& GetRange() const { return *reinterpret_cast<const T*>(Range); }

	bool IsFirstCall() const { return NumRead == 0;}

	const void* Range = nullptr;
	uint64 NumRead = 0;
};

struct FExistingItemSlice
{
	const void*		Data = nullptr;			
	uint64			Num = 0;

	explicit operator bool() const { return !!Num; }

	const uint8* At(uint64 Idx, uint32 Stride) const
	{
		check(Idx < Num);
		return reinterpret_cast<const uint8*>(Data) + Idx * Stride;
	}
};

struct FExistingItems
{
	uint64				NumTotal = 0;
	uint32				Stride = 0;
	FExistingItemSlice	Slice;

	void SetAll(const void* Items, uint64 Num, uint32 InStride)
	{
		NumTotal = Num;
		Stride = InStride;
		Slice = { Items, Num };
	}

	template<typename ItemType>
	void SetAll(const ItemType* Items, uint64 Num)
	{
		SetAll(Items, Num, sizeof(ItemType));
	}
};

struct FSaveRangeContext
{
	FGetItemsRequest		Request;		// Request to get items to be saved
	FExistingItems			Items;			// Response from IRangeBinding
	uint64					Scratch[8]; 	// Scratch memory for IRangeBinding
};

struct alignas(16) IItemRangeBinding
{
	virtual void ReadItems(FSaveRangeContext& Ctx) const = 0;
	virtual void MakeItems(FLoadRangeContext& Ctx) const = 0;

	// Try living without virtual destructor and trivially destructible members
	explicit IItemRangeBinding(FConcreteTypenameId RangeBindName) : BindName(RangeBindName) {}
	const FConcreteTypenameId BindName;
};

////////////////////////////////////////////////////////////////////////////////////////////////

// Possible save opt: Use paged linear allocator that only allocates on page exhaustion
class FLeafRangeAllocator
{
	FScratchAllocator&			Scratch;
	FBuiltRange*				Range = nullptr;
	const FUnpackedLeafType		Expected;

	PLAINPROPS_API void* Allocate(uint64 Num, SIZE_T LeafSize);

public:
	FLeafRangeAllocator(FScratchAllocator& InScratch, FUnpackedLeafType InExpected) : Scratch(InScratch), Expected(InExpected) {}

	template<LeafType T, typename SizeType>
	T* AllocateRange(SizeType Num)
	{
		check(ReflectArithmetic<T> == Expected);
		return Num ? static_cast<T*>(Allocate(static_cast<uint64>(Num), sizeof(T))) : nullptr;
	}

	template<typename SizeType>
	void* AllocateNonEmptyRange(SizeType Num, ELeafWidth Width)
	{
		check(Num > 0);
		check(Width == Expected.Width);
		return Allocate(static_cast<uint64>(Num), SizeOf(Width));
	}

	FBuiltRange* GetAllocatedRange() { return Range; }
};


// Specialized binding for transcoding leaf ranges
struct alignas(16) ILeafRangeBinding
{
	virtual void		SaveLeaves(const void* Range, FLeafRangeAllocator& Out) const = 0;
	virtual void		LoadLeaves(void* Range, FLeafRangeLoadView Leaves) const = 0;
	virtual bool		DiffLeaves(const void* RangeA, const void* RangeB) const = 0;

	explicit ILeafRangeBinding(FConcreteTypenameId RangeBindName) : BindName(RangeBindName) {}
	const FConcreteTypenameId BindName;

	template<typename SizeType>
	static bool Diff(SizeType NumA, SizeType NumB, const void* A, const void* B, SIZE_T ItemSize)
	{
		return NumA != NumB || (NumA > 0 && FMemory::Memcmp(A, B, static_cast<uint64>(NumA) * ItemSize) != 0);
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////

class FRangeBinding
{
	static constexpr uint64		SizeMask = 0b1111;
	static constexpr uint64		LeafMask = uint64(1) << FPlatformMemory::KernelAddressBit;
	static constexpr uint64		BindMask = ~(SizeMask | LeafMask);
	uint64						Handle;

public:
	PLAINPROPS_API FRangeBinding(const IItemRangeBinding& Binding, ESizeType SizeType);
	PLAINPROPS_API FRangeBinding(const ILeafRangeBinding& Binding, ESizeType SizeType);
	
	bool						IsLeafBinding() const					{ return !!(LeafMask & Handle); }
	const IItemRangeBinding&	AsItemBinding() const					{ check(!IsLeafBinding()); return *reinterpret_cast<IItemRangeBinding*>(Handle & BindMask); }
	const ILeafRangeBinding&	AsLeafBinding() const					{ check( IsLeafBinding()); return *reinterpret_cast<ILeafRangeBinding*>(Handle & BindMask); }
	ESizeType					GetSizeType() const						{ return static_cast<ESizeType>(Handle & SizeMask); }
	FConcreteTypenameId			GetBindName() const						{ return IsLeafBinding() ? AsLeafBinding().BindName : AsItemBinding().BindName; }
	inline bool	operator==(FRangeBinding O) const { return Handle == O.Handle; }
};

////////////////////////////////////////////////////////////////////////////////////////////////

struct FBothType
{
	FType		BindType;
	FType		DeclType;

	bool IsLowered() const { return BindType != DeclType; }
};

////////////////////////////////////////////////////////////////////////////////////////////////

struct FMemberBinding
{
	explicit FMemberBinding(uint64 InOffset = 0)
	: Offset(InOffset)
	, InnermostType(FLeafBindType(ELeafBindType::Bool, ELeafWidth::B8))
	{}

	uint64							Offset;
	FMemberBindType					InnermostType;		// Always Leaf or Struct
	FOptionalInnerId				InnermostId;		// Enum or struct bind id
	TConstArrayView<FRangeBinding>	RangeBindings;		// Non-empty -> Range

	// @pre InnermostId isn't type-erased / lowered
	PLAINPROPS_API FBothType		IndexParameterName(FIdIndexerBase& Ids) const;
};

class FSchemaBindings
{
public:
	UE_NONCOPYABLE(FSchemaBindings);
	PLAINPROPS_API explicit FSchemaBindings(FDebugIds In);
	PLAINPROPS_API ~FSchemaBindings();
	
	PLAINPROPS_API void							BindStruct(FBindId Id, TConstArrayView<FMemberBinding> Members, FStructSpec Decl);
	PLAINPROPS_API void							BindStruct(FBindId Id, TConstArrayView<FMemberBinding> Members, FStructDeclarationPtr&& Decl);
	PLAINPROPS_API const FSchemaBinding*		FindStruct(FBindId Id) const;
	PLAINPROPS_API const FSchemaBinding*		FindStruct(FBindId Id, const FStructDeclaration*& OutDeclaration) const;
	PLAINPROPS_API const FSchemaBinding&		GetStruct(FBindId Id) const;
	PLAINPROPS_API const FSchemaBinding&		GetStruct(FBindId Id, const FStructDeclaration*& OutDecl) const;
	PLAINPROPS_API void							DropStruct(FBindId Id);

	PLAINPROPS_API FDeclId						Lower(FBindId Id) const;
	PLAINPROPS_API const FStructDeclaration&	GetDeclaration(FBindId Id) const;
	PLAINPROPS_API const FStructDeclaration*	FindDeclaration(FBindId Id) const;
	FDebugIds									GetDebug() const { return Debug; }

private:
	TArray<FStructDeclarationPtr>				Declarations;
	TArray<TUniquePtr<FSchemaBinding>>			Bindings;
	FDebugIds									Debug;

	FSchemaBinding*								Bind(FBindId Id, TConstArrayView<FMemberBinding> Members, FDeclId DeclId);
	FStructDeclarationPtr&						At(FDeclId Id);
};

////////////////////////////////////////////////////////////////////////////////////////////////

// Lookup default instances when delta-saving struct ranges
struct IDefaultStructs
{
	virtual const void* Get(FBindId Id) = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////

struct FBindContext
{
	const FSchemaBindings&								Schemas;
	FCustomBindings&									Customs;

	PLAINPROPS_API const FStructDeclaration&			GetDeclaration(FBindId Id) const;
};

struct FBindDeclarations final : IDeclarations
{
	FBindDeclarations(const FEnumDeclarations& E, const FCustomBindings& C, const FSchemaBindings& S) : Enums(E), Customs(C), Schemas(S) {}

	const FEnumDeclarations&							Enums;
	const FCustomBindings&								Customs;
	const FSchemaBindings&								Schemas;
	
	PLAINPROPS_API virtual const FEnumDeclaration*		Find(FEnumId Id) const override;
	PLAINPROPS_API virtual const FStructDeclaration*	Find(FStructId Id) const override;
	PLAINPROPS_API virtual FDeclId						Lower(FBindId Id) const override;
};

//////////////////////////////////////////////////////////////////////////

// Save -> load struct ids for ESchemaFormat::InMemoryNames, alternative to side-channel with ExtractRuntimeIds()
[[nodiscard]] PLAINPROPS_API TArray<FStructId>			IndexStructRuntimeIds(const FSchemaBatch& Schemas,  FIdIndexerBase& Indexer);
[[nodiscard]] PLAINPROPS_API TArray<FInnerId>			IndexAllRuntimeIds(const FSchemaBatch& Schemas,  FIdIndexerBase& Indexer);
[[nodiscard]] PLAINPROPS_API TConstArrayView<FStructId>	GetStructRuntimeIds(TConstArrayView<FInnerId> AllIds, uint32 NumStructSchemas);

// Save -> load ids for ESchemaFormat::StableNames
struct FIdBinding
{
	TConstArrayView<FNameId>			Names;
	TConstArrayView<FNestedScopeId>		NestedScopes;
	TConstArrayView<FParametricTypeId>	ParametricTypes;
	TConstArrayView<FInnerId>			Schemas;

	FNameId								Remap(FNameId Old) const				{ return Names[Old.Idx]; }
	FMemberId							Remap(FMemberId Old) const				{ return { Remap(Old.Id) }; }
	FFlatScopeId						Remap(FFlatScopeId Old) const			{ return { Remap(Old.Name) }; }
	FNestedScopeId						Remap(FNestedScopeId Old) const			{ return NestedScopes[Old.Idx]; }
	FScopeId							Remap(FScopeId Old) const				{ return Old.IsFlat() ? FScopeId(Remap(Old.AsFlat())) : Old ? FScopeId(Remap(Old.AsNested())) : Old; }
	FConcreteTypenameId					Remap(FConcreteTypenameId Old) const	{ return { Remap(Old.Id) }; }	
	FParametricTypeId					Remap(FParametricTypeId Old) const		{ return ParametricTypes[Old.Idx]; }
	FTypenameId							Remap(FTypenameId Old) const			{ return Old.IsConcrete() ? FTypenameId(Remap(Old.AsConcrete())) : FTypenameId(Remap(Old.AsParametric())); }
	FType								Remap(FType Old) const				{ return { Remap(Old.Scope), Remap(Old.Name) }; }

	template<typename T>
	TOptionalId<T>						Remap(TOptionalId<T> Old) const			{ return Old ? ToOptional(Remap(Old.Get())) : Old; }

	TConstArrayView<FStructId>			GetStructIds(int32 NumStructs) const
	{
		// All saved struct schema ids are lower than enum schema ids
		check(NumStructs <= Schemas.Num());
		return MakeArrayView(reinterpret_cast<const FStructId*>(Schemas.GetData()), NumStructs);
	}
};

struct FIdTranslatorBase
{
	PLAINPROPS_API static uint32 CalculateTranslationSize(int32 NumSavedNames, const FSchemaBatch& Batch);
	PLAINPROPS_API static FIdBinding TranslateIds(FMutableMemoryView To, FIdIndexerBase& Indexer, TConstArrayView<FNameId> TranslatedNames, const FSchemaBatch& From);
};

// Maps saved ids -> runtime load ids for ESchemaFormat::StableNames
struct FIdTranslator : FIdTranslatorBase
{
	template<class NameType>
	FIdTranslator(TIdIndexer<NameType>& Indexer, TConstArrayView<NameType> SavedNames, const FSchemaBatch& Batch)
	{
		Allocator.SetNumUninitialized(CalculateTranslationSize(SavedNames.Num(), Batch));
		
		// Translate names
		TArrayView<FNameId> NewNames(reinterpret_cast<FNameId*>(&Allocator[0]), SavedNames.Num());
		FNameId* NameIt = NewNames.GetData();
		for (const NameType& SavedName : SavedNames)
		{
			(*NameIt++) = Indexer.MakeName(SavedName);
		}

		FMutableMemoryView OtherIds(NameIt, Allocator.Num() - NewNames.Num() * sizeof(FNameId));
		Translation = TranslateIds(/* out */ OtherIds, Indexer, NewNames, Batch);
	}
	
	FIdBinding								Translation;
	TArray<uint8, TInlineAllocator<1024>>	Allocator;
};

PLAINPROPS_API FSchemaBatch*	CreateTranslatedSchemas(const FSchemaBatch& Schemas, FIdBinding NewIds);
PLAINPROPS_API void				DestroyTranslatedSchemas(const FSchemaBatch* Schemas);

} // namespace PlainProps