// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsBind.h"
#include "PlainPropsCtti.h"
#include "PlainPropsSpecify.h"
#include "PlainPropsTypename.h"
#include "Containers/StaticArray.h"
#include "Containers/StringView.h"
#include <tuple>

namespace PlainProps 
{

inline FAnsiStringView ToAnsiView(std::string_view Str) { return FAnsiStringView(Str.data(), Str.length()); }

////////////////////////////////////////////////////////////////////////////////////////////////

template<typename T>
struct TRangeBind{ using Type = void; };

template<typename T>
using RangeBind = typename TRangeBind<T>::Type;

template<typename T>
struct TCustomBind { using Type = void; };

template<typename T, class Runtime>
struct TExternallyBound : std::false_type {};

template<class T>
struct TCustomDeltaBind { using Type = void; };

template<typename T>
using CustomBind = typename TCustomBind<T>::Type;

template<typename T>
using CustomDeltaBind = typename TCustomDeltaBind<T>::Type;

template<typename T>
struct TOccupancyOf
{
	static constexpr EMemberPresence Value = EMemberPresence::AllowSparse;
};

struct FRequireAll
{
	static constexpr EMemberPresence Value = EMemberPresence::RequireAll;
};

////////////////////////////////////////////////////////////////////////////////////////////////

template<class Ids, class Typename>
FScopeId GetOuterScope()
{
	return NoId;
}

template<class Ids, class Typename>
FScopeId IndexScope()
{
	// Opt: Make cached GetScopeId(), either via new namespace CTTI types (maybe PP_REFLECT_NAMESPACE)
	//		or some compile time string template parameters, perhaps a variadic template taking any number of chars
	return GetOuterScope<Ids, Typename>();
}

template<class Ids, class Typename>
FScopeId IndexScope() requires (Typename::Namespace.size() > 0)
{
	FScopeId Outer = GetOuterScope<Ids, Typename>();
	FFlatScopeId Namespace = Ids::IndexScope(ToAnsiView(Typename::Namespace));
	return Outer ? Ids::GetIndexer().NestFlatScope(Outer, Namespace) : FScopeId(Namespace);
}

template<ETypename Kind, class Typename>
constexpr std::string_view SelectStructName()
{	
	if constexpr (Kind == ETypename::Bind && ExplicitBindName<Typename>)
	{
		return Typename::BindName;
	}
	else
	{
		return Typename::DeclName;	
	}
}

template<class Ids, ETypename Kind, class Typename>
FType IndexStructName()
{
	FType BaseName = { IndexScope<Ids, Typename>(), 
					   FTypenameId(Ids::IndexTypename(ToAnsiView(SelectStructName<Kind, Typename>()))) };
	
	if constexpr (ParametricName<Typename>)
	{
		return IndexParametricType<Ids, Kind>(BaseName, (typename Typename::Parameters*)nullptr);
	}
	else
	{
		return BaseName;
	}
}

template<class Ids, class Typename>
FBothStructId IndexStructBothId(FType DeclName = IndexStructName<Ids, ETypename::Decl, Typename>())
{	
	FBothStructId Out;
	Out.DeclId = FDeclId(Ids::IndexStruct(DeclName));
	Out.BindId = UpCast(Out.DeclId);

	if constexpr (ExplicitBindName<Typename> || ParametricName<Typename>)
	{
		FType BindName = IndexStructName<Ids, ETypename::Bind, Typename>();
		Out.BindId = BindName != DeclName ? FBindId(Ids::IndexStruct(BindName)) : UpCast(Out.DeclId);
	}

	return Out;
}

template<class Ids, class Typename>
FDualStructId IndexStructDualId(FType Name = IndexStructName<Ids, ETypename::Decl, Typename>()) requires (!ExplicitBindName<Typename>)
{	
	return FDualStructId(Ids::IndexStruct(Name));
}

// Cached by function static
template<class Ids, typename Struct>
FDeclId GetStructDeclId()
{
	static FDeclId Id = FDeclId(Ids::IndexStruct(IndexStructName<Ids, ETypename::Decl, TTypename<Struct>>()));
	return Id;
}

// Cached by function static
template<class Ids, typename Struct>
FBindId GetStructBindId()
{
	static FBindId Id = FBindId(Ids::IndexStruct(IndexStructName<Ids, ETypename::Bind, TTypename<Struct>>()));
	return Id;
}

// Cached by function static
template<class Ids, typename Struct>
FBothStructId GetStructBothId()
{
	static FBothStructId Id = IndexStructBothId<Ids, TTypename<Struct>>();
	return Id;
}

// Cached by function static
template<class Ids, typename Struct>
FDualStructId GetStructDualId()
{
	static FDualStructId Id = IndexStructDualId<Ids, TTypename<Struct>>();
	return Id;
}

template<class Ids, Enumeration Enum>
FType IndexEnumName()
{
	using Typename = TTypename<Enum>;
	return { IndexScope<Ids, Typename>(), FTypenameId(Ids::IndexTypename(ToAnsiView(Typename::DeclName))) };
}

// Cached by function static
template<class Ids, Enumeration Enum>
FEnumId GetEnumId()
{
	static FEnumId Id = Ids::IndexEnum(IndexEnumName<Ids, Enum>());
	return Id;
}

template<class Ids, Arithmetic T>
FType IndexArithmeticName()
{
	static constexpr FUnpackedLeafType Leaf = ReflectArithmetic<T>;
	return { NoId, FTypenameId(Ids::IndexTypename(ToAnsiView(ArithmeticName<Leaf.Type, Leaf.Width>))) };
}

template<class Ids, ETypename, Arithmetic Leaf>
FType IndexParameterName()
{
	return IndexArithmeticName<Ids, Leaf>();
}

template<class Ids, ETypename, Enumeration Enum>
FType IndexParameterName()
{
	return IndexEnumName<Ids, Enum>();
}

template<class Ids, ETypename Kind, typename T>
FType IndexParameterName()
{
	using RangeBinding = RangeBind<T>;
	if constexpr (std::is_void_v<RangeBinding>)
	{
		return IndexStructName<Ids, Kind, TTypename<T>>();
	}
	else
	{
		FType ItemParam = IndexParameterName<Ids, Kind, typename RangeBinding::ItemType>();
		FType SizeParam = Ids::GetIndexer().MakeRangeParameter(RangeSizeOf(typename RangeBinding::SizeType{}));

		if constexpr (Kind == ETypename::Decl)
		{
			// Type-erase range type
			return Ids::GetIndexer().MakeAnonymousParametricType({ItemParam, SizeParam});
		}
		else
		{
			using Typename = TTypename<T>;
			FType RangeBindName = { IndexScope<Ids, Typename>(), FTypenameId(Ids::IndexTypename(ToAnsiView(RangeBinding::BindName))) };
			return Ids::GetIndexer().MakeParametricType(RangeBindName, {ItemParam, SizeParam});
		}
	}
}

template<class Ids, ETypename Kind, typename... Ts>
FType IndexParametricType(FType TemplatedType, const std::tuple<Ts...>*)
{
	FType Parameters[] = { (IndexParameterName<Ids, Kind, Ts>())... };
	return Ids::GetIndexer().MakeParametricType(TemplatedType, Parameters);
}

////////////////////////////////////////////////////////////////////////////////////////////////

// Works around C++'s lack of templated nullary constructors
template<class Ids>
struct TInit {};

using InnerStructArray = TArray<FInnerStruct, TInlineAllocator<8>>;

struct FCustomInit
{
	UE_NONCOPYABLE(FCustomInit);
	FCustomInit() = default;

	InnerStructArray				LoweredInners;

	// Only needed for inner structs that might get type-erased/lowered
	void RegisterInnerStruct(FBothStructId InnerId, TConstArrayView<FMemberId> Names)
	{
		if (InnerId.IsLowered())
		{
			for (FMemberId Name : Names)
			{
				LoweredInners.Add({Name, InnerId.BindId});
			}
		}
	}
};

// Helps custom binding constructors create ids and register types that might get type-erased 
template<class Ids>
struct TCustomInit : FCustomInit, TInit<Ids>
{};

// Constructor API for static custom bindings to create ids, register type-erased inner structs
// and specify member types, inheritance and version 
template<class Ids, uint32 N>
struct TCustomSpecifier : TCustomInit<Ids>
{
	FOptionalDeclId					Super;
	uint16							Version = 0;
	FMemberSpec						Members[N];

	template <typename... Ts>
    void SetMembers(Ts&&... Specs) requires (N == sizeof...(Specs))
    {
		FMemberSpec* It = &Members[0];
		((*It++ = FMemberSpec(Forward<Ts>(Specs))), ...);
    }

	void FillMembers(FMemberSpec Same)
	{
		for (FMemberSpec& Member : Members)
		{
			Member = Same;
		}
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////

// Workaround for clang requiring the definition, make it dependent on any other type
template<typename>
struct TDependent
{
	using Baseline = FBaseline;
	using StructLoadView = FStructLoadView;
};

// Scoped custom binding for native type
template<class T, class Runtime, typename CustomBinding = CustomBind<T>>
struct TScopedStructBinding : FBothStructId, CustomBinding
{
	using Ids = typename Runtime::Ids;
	using Typename = TCustomTypename<CustomBinding>::Type;
	static constexpr uint32 N = sizeof(CustomBinding::MemberIds) / sizeof(CustomBinding::MemberIds[0]);
	static constexpr bool bFastLoad = requires(FMemcpyLoadPlan& O) { CustomBinding::Plan(O); };

	explicit TScopedStructBinding(FType DeclName = IndexStructName<Ids, ETypename::Decl, Typename>())
	: TScopedStructBinding(IndexStructBothId<Ids, Typename>(DeclName))
	{}

	explicit TScopedStructBinding(FDualStructId Dual)
	: TScopedStructBinding(FBothStructId{Dual, Dual})
	{}

	explicit TScopedStructBinding(FBothStructId Both)
	: TScopedStructBinding(Both, TCustomSpecifier<Ids, N>())
	{}

	TScopedStructBinding(FBothStructId Both, TCustomSpecifier<Ids, N>&& Init)
	: FBothStructId(Both)
	, CustomBinding(/* out */ Init)
	{
		checkf(N > 0 && CustomBinding::MemberIds[N - 1] != FMemberId{},
			TEXT("Invalid members in custom binding for '%s' (Num=%u)"),
			*FDebugIds(Ids::GetIndexer()).Print(Both.BindId), N);
		FStructSpec Decl = { DeclId, Init.Super, Init.Version, TOccupancyOf<T>::Value, CustomBinding::MemberIds, Init.Members };
		Runtime::GetCustoms().BindStruct(BindId, *this, Decl, Init.LoweredInners);
	}

	~TScopedStructBinding()
	{
		Runtime::GetCustoms().DropStruct(BindId);
	}

	virtual void SaveCustom(FMemberBuilder& Dst, const void* Src, TDependent<Ids>::Baseline Base, const FSaveContext& Ctx) override
	{
		const void* Default = Base.Get();
		CustomBinding::Save(Dst, *static_cast<const T*>(Src), static_cast<const T*>(Default), Ctx);
	}

	virtual void LoadCustom(void* Dst, TDependent<Ids>::StructLoadView Src, ECustomLoadMethod Method) const override 
	{
		if constexpr (bFastLoad)
		{
			unimplemented();
		}
		else
		{
			CustomBinding::Load(*static_cast<T*>(Dst), Src, Method);
		}
	}
	
	virtual bool DiffCustom(const void* A, const void* B, const FBindContext& Ctx) const override
	{
		return CustomBinding::Diff(*static_cast<const T*>(A), *static_cast<const T*>(B), Ctx);
	}

	virtual bool DiffCustom(const void* A, const void* B, FDiffContext& Ctx) const override
	{
		return CustomBinding::Diff(*static_cast<const T*>(A), *static_cast<const T*>(B), Ctx);
	}

	virtual void PlanCustom(FMemcpyLoadPlan& Out) const override
	{
		if constexpr (bFastLoad)
		{
			CustomBinding::Plan(Out);
		}
	}
};

// Specialization for schemabound structs
template<class T, class Runtime>
struct TScopedStructBinding<T, Runtime, void> : FBothStructId
{
	using Ids = typename Runtime::Ids;
	using Typename = TTypename<T>;

	explicit TScopedStructBinding(FDualStructId Dual)
	: TScopedStructBinding(FBothStructId{Dual, Dual})
	{}

	TScopedStructBinding(FBothStructId Both = IndexStructBothId<Ids, Typename>() )
	: FBothStructId(Both)
	{
		BindNativeStruct<CttiOf<T>, Runtime>(/* out */ Runtime::GetSchemas(), Both, TOccupancyOf<T>::Value);
	}

	~TScopedStructBinding()
	{
		Runtime::GetSchemas().DropStruct(BindId);
	}
};

template<Enumeration Enum, EEnumMode Mode, class Runtime>
struct TScopedEnumDeclaration
{
	using Ids = typename Runtime::Ids;

	FEnumId Id;
	TScopedEnumDeclaration() : Id(DeclareNativeEnum<Enum, Mode, Ids>(Runtime::GetEnums(), LeafWidth<sizeof(Enum)>)) {}
	~TScopedEnumDeclaration() { Runtime::GetEnums().Drop(Id); }
};

template<class Struct, typename CustomBinding, class Runtime>
FBothStructId BindCustomStructOnce()
{
	static TScopedStructBinding<Struct, Runtime, CustomBinding> Instance;
	return Instance;
}

////////////////////////////////////////////////////////////////////////////////////////////////

inline constexpr FMemberBindType DefaultStructBindType = FMemberBindType(FStructType{EMemberKind::Struct, /* IsDynamic */ 0, /* IsSuper */ 0});
inline constexpr FMemberBindType SuperStructBindType = FMemberBindType(FStructType{EMemberKind::Struct, /* IsDynamic */ 0, /* IsSuper */ 1});

inline FMemberBindType BindInnerStruct(FOptionalInnerId& OutBindId, FMemberSpec& OutSpec, FBothStructId Both)
{
	OutBindId = FInnerId(Both.BindId);
	OutSpec = FMemberSpec(Both.DeclId);
	return DefaultStructBindType;
}

inline FMemberBindType BindInnerLeaf(FOptionalInnerId& OutId, FMemberSpec& OutSpec, FUnpackedLeafType Leaf, FOptionalEnumId Id)
{
	OutId = ToOptionalInner(Id);
	OutSpec = FMemberSpec(Leaf, Id);
	return FMemberBindType(Leaf);
}

template<typename Struct, class Ids>
FMemberBindType BindInnermostType(FOptionalInnerId& OutBindId, FMemberSpec& OutSpec)
{
	return BindInnerStruct(OutBindId, OutSpec, GetStructBothId<Ids, Struct>());
}

template<Arithmetic Type, class Ids>
FMemberBindType BindInnermostType(FOptionalInnerId& OutId, FMemberSpec& OutSpec)
{
	return BindInnerLeaf(OutId, OutSpec, ReflectArithmetic<Type>, NoId);
}

template<Enumeration Enum, class Ids>
FMemberBindType BindInnermostType(FOptionalInnerId& OutId, FMemberSpec& OutSpec)
{
	return BindInnerLeaf(OutId, OutSpec, ReflectEnum<Enum>, GetEnumId<Ids, Enum>());
}

template<typename RangeBinding>
constexpr uint32 CountRangeBindings()
{
	using InnerBinding = RangeBind<typename RangeBinding::ItemType>;
	return 1 + CountRangeBindings<InnerBinding>();
}

template<>
constexpr uint32 CountRangeBindings<void>()
{
	return 0;
}

template<typename RangeBinding, uint32 NestLevel>
struct TInnermostType
{
	using InnerType = typename RangeBinding::ItemType;
	using Type = TInnermostType<RangeBind<InnerType>, NestLevel - 1>::Type;
};

template<typename RangeBinding>
struct TInnermostType<RangeBinding, 1>
{
	using Type = typename RangeBinding::ItemType;
};

template<typename RangeBinding, typename Ids, uint32 N>
TConstArrayView<FRangeBinding> GetRangeBindings() requires (N > 0)
{
	struct FOnce
	{
		FOnce(FAnsiStringView RangeBindName)
		: Instance(Ids::IndexTypename(RangeBindName))
		, Binding(Instance, RangeSizeOf(typename RangeBinding::SizeType{}))
		{}
		RangeBinding Instance;
		FRangeBinding Binding;
	};

	if constexpr (N == 1)
	{
		static FOnce Static(ToAnsiView(RangeBinding::BindName));
		return MakeArrayView(&Static.Binding, N);
	}
	else
	{
		using InnerType = typename RangeBinding::ItemType;
		using InnerRangeBinding = RangeBind<InnerType>;

		struct FNestedOnce : FOnce
		{
			FNestedOnce()
			: FOnce(ToAnsiView(InnerRangeBinding::BindName))
			{
				FMemory::Memcpy(NestedBindings, GetRangeBindings<InnerRangeBinding, Ids,  N - 1>().GetData(), sizeof(NestedBindings));
			}
			
			uint8 NestedBindings[sizeof(FRangeBinding) * (N - 1)] = {};
		};
		static_assert(std::is_trivially_destructible_v<FRangeBinding>);
		static_assert(offsetof(FNestedOnce, Binding) + sizeof(FRangeBinding) == offsetof(FNestedOnce, NestedBindings));	

		static FNestedOnce Static;
		return MakeArrayView(&Static.Binding, N);
	}
}

template<LeafType Type, class Runtime>
FMemberBinding BindMember(uint64 Offset, FMemberSpec& OutSpec)
{
	FMemberBinding Out(Offset);
	Out.InnermostType = BindInnermostType<Type, typename Runtime::Ids>(Out.InnermostId, OutSpec);
	return Out;
}

template<typename Type, class Runtime>
FMemberBinding BindMember(uint64 Offset, FMemberSpec& OutSpec)
{
	using Ids = typename Runtime::Ids;
	using CustomBinding = typename Runtime::template CustomBindings<Type>::Type;

	FMemberBinding Out(Offset);
	if constexpr (!std::is_void_v<CustomBinding>)
	{
		FBothStructId Both;  
		if constexpr (TExternallyBound<Type, Runtime>::value)
		{
			using Typename = typename TCustomTypename<CustomBinding>::Type;
			Both = IndexStructBothId<Ids, Typename>();
			check(Runtime::GetCustoms().FindStruct(Both.BindId) != nullptr);
		}
		else
		{
			Both = BindCustomStructOnce<Type, CustomBinding, Runtime>();
		}
		Out.InnermostType = BindInnerStruct(Out.InnermostId, OutSpec, Both);
	}
	else
	{
		using RangeBinding = RangeBind<Type>;
		if constexpr (!std::is_void_v<RangeBinding>)
		{
			constexpr uint32 NumRangeBindings = CountRangeBindings<RangeBinding>();
			using InnermostType = typename TInnermostType<RangeBinding, NumRangeBindings>::Type;

			Out.RangeBindings = GetRangeBindings<RangeBinding, Ids, NumRangeBindings>();
			Out.InnermostType = BindInnermostType<InnermostType, Ids>(Out.InnermostId, OutSpec);
			
			for (FRangeBinding Range : Out.RangeBindings)
			{
				OutSpec.RangeWrap(Range.GetSizeType());
			}
		}
		else
		{
			FBothStructId Both = GetStructBothId<typename Runtime::Ids, Type>();
			Out.InnermostType = BindInnerStruct(Out.InnermostId, OutSpec, Both);
		}
	}
	
	return Out;
}

////////////////////////////////////////////////////////////////////////////////////////////////

template<Enumeration Enum, EEnumMode Mode, class Ids>
FEnumId DeclareNativeEnum(FEnumDeclarations& Out, ELeafWidth Width)
{
	using Ctti = CttiOf<Enum>;
	// Never sign extend negative enum constant values of width B8, B16 and B32
	using UnsignedUnderlyingType = std::make_unsigned_t<std::underlying_type_t<typename Ctti::Type>>;

	FType Type = IndexEnumName<Ids, Enum>();
	FEnumId Id = Ids::IndexEnum(Type);
	FEnumerator Enumerators[Ctti::NumEnumerators];
	for (FEnumerator& Enumerator : Enumerators)
	{
		Enumerator.Name = Ids::IndexName(ToAnsiView(Ctti::Names[&Enumerator - Enumerators]));
		Enumerator.Constant = static_cast<uint64>(static_cast<UnsignedUnderlyingType>(Ctti::Constants[&Enumerator - Enumerators]));
	}
	Out.Declare(Id, Type, Mode, Width, Enumerators, EEnumAliases::Fail);

	return Id;
}

////////////////////////////////////////////////////////////////////////////////////////////////

template<class Ctti, class Runtime>
void BindNativeStruct(FSchemaBindings& Out, FBothStructId Both, EMemberPresence Occupancy)
{
	using Ids = Runtime::Ids;
	using SuperType = typename Ctti::Super;
	FOptionalDeclId SuperId;
	if constexpr (!std::is_void_v<SuperType>)
	{
		SuperId = GetStructDeclId<Ids, SuperType>();
	}

	if constexpr (Ctti::NumVars) // Temp workaround til MakeArrayView handles zero-sized arrays
	{
		FMemberId MemberNames[Ctti::NumVars];
		FMemberSpec MemberTypes[Ctti::NumVars];
		FMemberBinding MemberBindings[Ctti::NumVars];
		ForEachVar<Ctti>([&]<class Var>()
		{
			MemberNames[Var::Index] = Ids::IndexMember(Var::Name);
			FMemberSpec& OutSpec = MemberTypes[Var::Index];
			MemberBindings[Var::Index] = BindMember<typename Var::Type, Runtime>(Var::Offset, OutSpec);
		});
	
		const FStructSpec Spec = { Both.DeclId, SuperId, /* no CTTI version yet */ 0, Occupancy, MemberNames, MemberTypes };
		Out.BindStruct(Both.BindId, MemberBindings, Spec);
	}
	else
	{
		const FStructSpec Spec = { Both.DeclId, SuperId, /* no CTTI version yet */ 0, Occupancy, {}, {} };
		Out.BindStruct(Both.BindId, {}, Spec);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////

struct FAuxMember
{
	FMemberId			Name;
	FMemberBinding		Binding;
	FMemberSpec			Spec;
};

template<class Ctti, class Runtime>
TStaticArray<FAuxMember, Ctti::NumVars> BindAuxMembers()
{
	static_assert(std::is_void_v<typename Ctti::Super>, "Inheritance support unimplemented");

	TStaticArray<FAuxMember, Ctti::NumVars> Out;
	ForEachVar<Ctti>([&]<class Var>()
	{
		FAuxMember& Aux = Out[Var::Index];
		Aux.Name = Runtime::Ids::IndexMember(Var::Name);
		Aux.Binding = BindMember<typename Var::Type, Runtime>(Var::Offset, /* out */ Aux.Spec);
	});
	return Out;
}

} // namespace PlainProps
