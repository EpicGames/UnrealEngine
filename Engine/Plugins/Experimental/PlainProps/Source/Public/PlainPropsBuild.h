// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsDeclare.h"
#include "PlainPropsTypes.h"
#include "Algo/Compare.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Memory/MemoryView.h"
#include "Templates/UniquePtr.h"

namespace PlainProps
{

struct FBuiltMember;
struct FBuiltStruct;
struct FBuiltRange;
struct FUnpackedLeafType;

//////////////////////////////////////////////////////////////////////////

/// Single-threaded scratch allocator for intermediate built representation
class FScratchAllocator
{
	struct FPage
	{
		FPage*			PrevPage;
		uint8			Data[0];
	};

	static constexpr uint32 PageSize = 65536;
	static constexpr uint32 DataSize = PageSize - offsetof(FPage, Data);
	
	uint8*					Cursor = nullptr;
	uint8*					PageEnd = nullptr;
	FPage*					LastPage = nullptr;

	PLAINPROPS_API uint8* AllocateInNewPage(SIZE_T Size, uint32 Alignment);

public:
	UE_NONCOPYABLE(FScratchAllocator);
	FScratchAllocator() = default;
	~FScratchAllocator() { Reset(); }

	inline void* Allocate(SIZE_T Size, uint32 Alignment)
	{
		uint8* Out = Align(Cursor, Alignment);
		if (Out + Size <= PageEnd)
		{
			Cursor = Out + Size;
			return Out;
		}

		return AllocateInNewPage(Size, Alignment);
	}

	inline void* AllocateZeroed(SIZE_T Size, uint32 Alignment)
	{
		void* Out = Allocate(Size, Alignment);
		FMemory::Memzero(Out, Size);
		return Out;
	}

	template<typename T, typename... Ts>
	inline T* AllocateArray(uint64 Num, Ts... ConstructorParams)
	{
		T* Out = static_cast<T*>(Allocate(Num * sizeof(T), alignof(T)));
		for (uint64 Idx = 0; Idx < Num; ++Idx)
		{
			new (Out + Idx) T(ConstructorParams...);
		}
		return Out;
	}
	
	template<typename T>
	inline T* CloneArray(uint64 Num, const T* In)
	{
		T* Out = static_cast<T*>(Allocate(Num * sizeof(T), alignof(T)));
		for (uint64 Idx = 0; Idx < Num; ++Idx)
		{
			new (Out + Idx) T(In[Idx]);
		}
		return Out;
	}

	PLAINPROPS_API void Reset();
};

//////////////////////////////////////////////////////////////////////////

struct FMemberSchema
{
	FMemberType						Type;
	FMemberType						InnerRangeType;
	uint16							NumInnerRanges;
	FOptionalInnerId				InnerSchema;
	const FMemberType*				NestedRangeTypes;

	TConstArrayView<FMemberType> GetInnerRangeTypes() const
	{
		return MakeArrayView(NestedRangeTypes ? NestedRangeTypes : &InnerRangeType, NumInnerRanges);
	}

	FMemberType GetInnermostType() const
	{
		return NumInnerRanges ? GetInnerRangeTypes().Last() : Type;
	}

	[[nodiscard]] PLAINPROPS_API FMemberType& EditInnermostType(FScratchAllocator& Scratch);

	void CheckInvariants()
	{
		check(Type.IsRange() == !!NumInnerRanges);
		check(!!NestedRangeTypes == (NumInnerRanges > 1));
	}
};

inline bool operator==(FMemberSchema A, FMemberSchema B)
{
	if (FMemory::Memcmp(&A, &B, sizeof(FMemberSchema)) == 0)
	{
		return true;
	}

	return A.Type == B.Type && A.InnerSchema == B.InnerSchema && Algo::Compare(A.GetInnerRangeTypes(), B.GetInnerRangeTypes());
}
//////////////////////////////////////////////////////////////////////////

template<Arithmetic T>
inline T Normalize(T Value)					{ return Value; }
inline float Normalize(float Value)			{ return Value; } // todo
inline double Normalize(double Value)		{ return Value; } // todo

//////////////////////////////////////////////////////////////////////////

inline uint64 ValueCast(bool Value)			{ return static_cast<uint64>(Value); }
inline uint64 ValueCast(int8 Value)			{ return static_cast<uint8>(Value); }
inline uint64 ValueCast(int16 Value)		{ return static_cast<uint16>(Value); }
inline uint64 ValueCast(int32 Value)		{ return static_cast<uint32>(Value); }
inline uint64 ValueCast(int64 Value)		{ return static_cast<uint64>(Value); }
inline uint64 ValueCast(uint8 Value)		{ return Value; }
inline uint64 ValueCast(uint16 Value)		{ return Value; }
inline uint64 ValueCast(uint32 Value)		{ return Value; }
inline uint64 ValueCast(uint64 Value)		{ return Value; }
uint64 ValueCast(float Value);
uint64 ValueCast(double Value);
inline uint64 ValueCast(char8_t Value)		{ return static_cast<uint8>(Value); }
inline uint64 ValueCast(char16_t Value)		{ return static_cast<uint16>(Value); }
inline uint64 ValueCast(char32_t Value)		{ return static_cast<uint32>(Value); }

//////////////////////////////////////////////////////////////////////////

struct FTypedRange
{
	FMemberSchema Schema;
	FBuiltRange* Values = nullptr;
};

template<Arithmetic T>
FMemberSchema MakeLeafRangeSchema(ESizeType MaxSize)
{
	return { FMemberType(MaxSize), ReflectArithmetic<T>.Pack(), 1, NoId, nullptr };
}

template<Arithmetic T, typename SizeType>
FMemberSchema MakeLeafRangeSchema()
{
	return MakeLeafRangeSchema<T>(RangeSizeOf(SizeType{}));
}

template<UnsignedIntegral T>
FMemberSchema MakeHexRangeSchema(ESizeType MaxSize)
{
	return { FMemberType(MaxSize), FMemberType(ELeafType::Hex, ReflectArithmetic<T>.Width), 1, NoId, nullptr };
}

template<Enumeration T>
FMemberSchema MakeEnumRangeSchema(FEnumId Id, ESizeType MaxSize)
{
	return { FMemberType(MaxSize), ReflectEnum<T>.Pack(), 1, FInnerId(Id), nullptr };
}

template<Enumeration T, typename SizeType>
FMemberSchema MakeEnumRangeSchema(FEnumId Id)
{
	return MakeEnumRangeSchema<T>(Id, RangeSizeOf(SizeType{}));
}

template<UnsignedIntegral T>
FMemberSchema MakeEnumRangeSchema(FEnumId Id, ESizeType MaxSize)
{
	return { FMemberType(MaxSize), FMemberType(ELeafType::Enum, ReflectArithmetic<T>.Width), 1, FInnerId(Id), nullptr };
}

inline constexpr FMemberType DefaultStructType =	FMemberType(FStructType{EMemberKind::Struct, /* IsDynamic */ 0, /* IsSuper */ 0});
inline constexpr FMemberType SuperStructType =		FMemberType(FStructType{EMemberKind::Struct, /* IsDynamic */ 0, /* IsSuper */ 1});
inline constexpr FMemberType DynamicStructType =	FMemberType(FStructType{EMemberKind::Struct, /* IsDynamic */ 1, /* IsSuper */ 0});


inline FMemberSchema MakeStructRangeSchema(ESizeType SizeType, FStructId Id)
{
	return { FMemberType(SizeType), DefaultStructType, 1, FInnerId(Id), nullptr };
}

inline FMemberSchema MakeDynamicStructRangeSchema(ESizeType SizeType)
{
	return { FMemberType(SizeType), DynamicStructType, 1, NoId, nullptr };
}

PLAINPROPS_API FMemberSchema MakeNestedRangeSchema(FScratchAllocator& Scratch, ESizeType SizeType, FMemberSchema InnerRangeSchema);

//////////////////////////////////////////////////////////////////////////

[[nodiscard]] PLAINPROPS_API FBuiltRange* CloneLeaves(FScratchAllocator& Scratch, uint64 Num, const void* Data, SIZE_T LeafSize);

template<Arithmetic T>
[[nodiscard]] FTypedRange BuildLeafRange(FScratchAllocator& Scratch, ESizeType SizeType, const T* Values, uint64 Num)
{
	// todo: detect invalid floats
	return { MakeLeafRangeSchema<T>(SizeType), CloneLeaves(Scratch, Num, Values, sizeof(T)) };
}

template<Arithmetic T, typename SizeType>
[[nodiscard]] FTypedRange BuildLeafRange(FScratchAllocator& Scratch, const T* Values, SizeType Num)
{
	return BuildLeafRange(Scratch, RangeSizeOf(SizeType{}), Values, Num);
}

template<Arithmetic T>
[[nodiscard]] FTypedRange BuildLeafRange(FScratchAllocator& Scratch, ESizeType SizeType, TConstArrayView<T> Values)
{
	return BuildLeafRange(Scratch, SizeType, Values.GetData(), Values.Num());
}

template<Arithmetic T, typename SizeType>
[[nodiscard]] FTypedRange BuildLeafRange(FScratchAllocator& Scratch, TConstArrayView<T, SizeType> Values)
{
	return BuildLeafRange(Scratch, RangeSizeOf(SizeType{}), Values.GetData(), Values.Num());
}

template<UnsignedIntegral T, typename SizeType>
[[nodiscard]] FTypedRange BuildHexRange(FScratchAllocator& Scratch, TConstArrayView<T, SizeType> Values)
{
	return { MakeHexRangeSchema<T>(RangeSizeOf(SizeType{})), CloneLeaves(Scratch, Values.Num(), Values.GetData(), sizeof(T)) };
}

template<UnsignedIntegral T>
[[nodiscard]] FTypedRange BuildHexRange(FScratchAllocator& Scratch, ESizeType SizeType, TConstArrayView<T> Values)
{
	return { MakeHexRangeSchema<T>(SizeType), CloneLeaves(Scratch, Values.Num(), Values.GetData(), sizeof(T)) };
}

template<Enumeration T, typename SizeType>
[[nodiscard]] FTypedRange BuildEnumRange(FScratchAllocator& Scratch, FEnumId Enum, TConstArrayView<T, SizeType> Values)
{
	return { MakeEnumRangeSchema<T, SizeType>(Enum), CloneLeaves(Scratch, Values.Num(), Values.GetData(), sizeof(T)) };
}

template<UnsignedIntegral T>
[[nodiscard]] FTypedRange BuildEnumRange(FScratchAllocator& Scratch, FEnumId Enum, ESizeType SizeType, TConstArrayView<T> Values)
{
	return { MakeEnumRangeSchema<T>(Enum, SizeType), CloneLeaves(Scratch, Values.Num(), Values.GetData(), sizeof(T)) };
}

[[nodiscard]] inline FTypedRange MakeStructRange(FStructId Id, ESizeType SizeType, FBuiltRange* Values )
{
	return { MakeStructRangeSchema(SizeType, Id), Values };
}

//////////////////////////////////////////////////////////////////////////

union FBuiltValue
{
	uint64			Leaf;
	FBuiltStruct*	Struct;
	FBuiltRange*	Range;
};

struct FTypedValue
{
	FMemberSchema	Schema;
	FBuiltValue		Value;
};

struct FBuiltMember
{
	FBuiltMember(FMemberId InName, FTypedValue In) : FBuiltMember(InName, In.Schema, In.Value) {}
	FBuiltMember(FOptionalMemberId N, FMemberSchema S, FBuiltValue V) : Name(N), Schema(MoveTemp(S)), Value(V) {}
	PLAINPROPS_API FBuiltMember(FMemberId Name, FUnpackedLeafType Leaf, FOptionalEnumId Id, uint64 Value);
	PLAINPROPS_API FBuiltMember(FMemberId Name, FTypedRange Range);
	PLAINPROPS_API FBuiltMember(FMemberId Name, FStructId Id, FBuiltStruct* Value);
	PLAINPROPS_API static FBuiltMember MakeSuper(FStructId Id, FBuiltStruct* Value);

	FOptionalMemberId		Name;
	FMemberSchema			Schema;
	FBuiltValue				Value;
};

//////////////////////////////////////////////////////////////////////////

// Builds an ordered list of properties to be saved
class FMemberBuilder
{
public:
	template<Arithmetic T>
	void Add(FMemberId Name, T Value)
	{
		AddLeaf(Name, ReflectArithmetic<T>, NoId, ValueCast(Value));
	}

	template<Enumeration T>
	void AddEnum(FMemberId Name, FEnumId Id, T Value)
	{
		AddLeaf(Name, ReflectEnum<T>, ToOptional(Id), ValueCast(static_cast<__underlying_type(T)>(Value)));
	}
	
	template<UnsignedIntegral T>
	void AddEnum(FMemberId Name, FEnumId Id, T Value)
	{
		AddLeaf(Name, {ELeafType::Enum, ReflectArithmetic<T>.Width}, ToOptional(Id), Value);
	}
	
	template<UnsignedIntegral T>
	void AddHex(FMemberId Name, T Value)
	{
		AddLeaf(Name, {ELeafType::Hex, ReflectArithmetic<T>.Width}, NoId, Value);
	}

	void AddLeaf(FMemberId Name, FUnpackedLeafType Leaf, FOptionalEnumId Enum, uint64 Value)	{ Members.Emplace(Name, Leaf, Enum, Value); }
	void AddRange(FMemberId Name, FTypedRange Range)											{ Members.Emplace(Name, Range); }
	// Add already built nested struct, must not be null
	void AddStruct(FMemberId Name, FStructId Id, FBuiltStruct* Struct)							{ check(Struct); Members.Emplace(Name, Id, Struct); }
	void Add(FMemberId Name, FTypedValue TypedValue)											{ Members.Emplace(Name, TypedValue); }

	// Add already built super struct, must not be null and must be the first member added
	PLAINPROPS_API void AddSuperStruct(FStructId SuperSchema, FBuiltStruct* SuperStruct);
	
	// Build members into a single nested super struct member, no-op if no non-super members has been added
	PLAINPROPS_API void BuildSuperStruct(FScratchAllocator&	Scratch, const FStructDeclaration& Super, const FDebugIds& Debug);

	[[nodiscard]] PLAINPROPS_API FBuiltStruct* BuildAndReset(FScratchAllocator& Scratch, const FStructDeclaration& Declared, const FDebugIds& Debug);

	bool IsEmpty() const { return Members.IsEmpty(); }

private:
	using FBuiltMemberArray = TArray<FBuiltMember, TInlineAllocator<16>>;
	
	FBuiltMemberArray		Members;
	
	//template<typename T>
	//void NormalizeLeafRange(T*, uint64) {}
	//PLAINPROPS_API void NormalizeLeafRange(float*, uint64 Num);
	//PLAINPROPS_API void NormalizeLeafRange(double*, uint64 Num);
};

// Rough API draft
struct FDenseMemberBuilder
{
	FScratchAllocator& Scratch;
	const FDebugIds& Debug;

	template<typename T, typename... Ts>
	[[nodiscard]] FBuiltStruct* BuildHomogeneous(const FStructDeclaration& Declaration, T Head, Ts... Tail) const
	{
		// Todo: Handle enums, ranges and structs
		FBuiltValue Values[] = { {.Leaf = ValueCast(Head)}, {.Leaf = (ValueCast(Tail))}...  };
		return BuildHomo(Declaration, FMemberType(ReflectArithmetic<T>.Pack()), Values);
	}

private:
	[[nodiscard]] PLAINPROPS_API FBuiltStruct* BuildHomo(const FStructDeclaration& Declaration, FMemberType Leaf, TConstArrayView<FBuiltValue> Values) const;
};

// Helper class for building struct ranges
class FStructRangeBuilder
{
public:
	FStructRangeBuilder(uint64 Num, ESizeType InSizeType)
	: SizeType(InSizeType)
	{
		Structs.SetNum(Num);
	}

	template<typename IntType>
	explicit FStructRangeBuilder(IntType Num)
	: FStructRangeBuilder(static_cast<uint64>(Num), RangeSizeOf(Num))
	{}
	 
	FMemberBuilder& operator[](uint64 Idx) { return Structs[Idx]; }

	FTypedRange BuildAndReset(FScratchAllocator& Scratch, const FStructDeclaration& Declared, const FDebugIds& Debug);

private:
	TArray64<FMemberBuilder> Structs; 
	ESizeType SizeType;
};


// Helper class for building nested ranges
class FNestedRangeBuilder
{
public:
	FNestedRangeBuilder(FMemberSchema InSchema, int64 InitialReserve)
	: Schema(InSchema)
	{
		Ranges.Reserve(InitialReserve);
	}

	~FNestedRangeBuilder();

	void Add(FTypedRange Range)
	{
		check(Range.Values == nullptr || Range.Schema == Schema);
		Ranges.Add(Range.Values);
	}

	[[nodiscard]] FTypedRange BuildAndReset(FScratchAllocator& Scratch, ESizeType SizeType);

private:
	TArray64<FBuiltRange*> Ranges; 
	FMemberSchema Schema;
};

} // namespace PlainProps


inline void* operator new(std::size_t Size, PlainProps::FScratchAllocator& Scratch)
{
	return Scratch.Allocate(Size, MIN_ALIGNMENT);
}

inline void* operator new(std::size_t Size, std::align_val_t Align, PlainProps::FScratchAllocator& Scratch)
{
	return Scratch.Allocate(Size, static_cast<uint32>(Align));
}
