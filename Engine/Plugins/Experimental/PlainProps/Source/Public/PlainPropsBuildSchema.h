// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsBuild.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/PagedArray.h"
#include "Containers/Set.h"

namespace PlainProps
{

struct FMemberSchema;
struct FEnumSchemaBuilder;
struct FStructSchemaBuilder;

struct FBuiltStructSchema
{
	FType							Type;
	FStructId						Id;
	FOptionalStructId				Super;
	bool							bDense = false;
	TArray<FMemberId>				MemberNames;
	TArray<const FMemberSchema*>	MemberSchemas;
};

struct FBuiltEnumSchema
{
	FType							Type;
	FEnumId							Id;
	EEnumMode						Mode = EEnumMode::Flat;
	ELeafWidth						Width = ELeafWidth::B8;
	TArray<FNameId>					Names;
	TArray<uint64>					Constants;
};

struct FBuiltSchemas
{
	TArray<FBuiltStructSchema>		Structs; // Same size as number of declared structs
	TArray<FBuiltEnumSchema>		Enums; // Same size as number of declared enums
};

class FSchemasBuilder
{
public:

	PLAINPROPS_API FSchemasBuilder(const FIds& Names, const IDeclarations& Types, FScratchAllocator& InScratch, ESchemaFormat InFormat);
	PLAINPROPS_API ~FSchemasBuilder();

	PLAINPROPS_API FEnumSchemaBuilder&			NoteEnum(FEnumId Id);
	PLAINPROPS_API FStructSchemaBuilder&		NoteStruct(FStructId Id);
	PLAINPROPS_API void							NoteStructAndMembers(FStructId Id, const FBuiltStruct& Struct);
	PLAINPROPS_API FBuiltSchemas				Build();
	
	FScratchAllocator&							GetScratch() const { return Scratch; }
	const FIds&									GetIds() const { return Ids; }
	FDebugIds									GetDebug() const { return Debug; }

private:
	const IDeclarations&						Declarations;
	TSet<FStructId>								StructIndices;
	TSet<FEnumId>								EnumIndices;
	const FIds&									Ids;
	ESchemaFormat								Format;
	TPagedArray<FStructSchemaBuilder, 4096>		Structs;		// TPagedArray for stable references
	TPagedArray<FEnumSchemaBuilder, 4096>		Enums;			// TPagedArray for stable references
	FScratchAllocator&							Scratch;
	FDebugIds									Debug;
	bool										bBuilt = false;

	void										NoteInheritanceChains();
};

// Extract runtime ids to side-channel for loading to avoid reindexing with IndexRuntimeIds()
[[nodiscard]] PLAINPROPS_API TArray<FStructId> ExtractRuntimeIds(const FBuiltSchemas& In);

} // namespace PlainProps
