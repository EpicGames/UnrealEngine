// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsStringUtil.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMath.h"
#include <tuple>

namespace PlainProps 
{

template<typename T> struct TCttiOf { using Type = decltype(CttiOfPtr((T*)nullptr)); };
template<typename T> using CttiOf = typename TCttiOf<T>::Type;

namespace Private
{
template<class Ctti>
constexpr bool ValidateEnumFlags()
{
	using EnumType = Ctti::Type;
	using UnsignedUnderlyingType = std::make_unsigned_t<std::underlying_type_t<EnumType>>;
	for (EnumType Constant : Ctti::Constants)
	{
		if (FPlatformMath::CountBits64(static_cast<UnsignedUnderlyingType>(Constant)) > 1)
		{
			return false;
		}
	}
	return true;
}
} // namespace Private

//////////////////////////////////////////////////////////////////////////
// Private PP_REFLECT_XYZ helpers

#define _PP_EXPAND(X) X

#define _PP_NUM_ARGS_(_1,_2,_3,_4,_5,_6,_7,_8,_9,_10,_11,_12,_13,_14,_15,_16, N, ...) N
#define _PP_NUM_ARGS(...) _PP_NUM_ARGS_(__VA_ARGS__,16,15,14,13,12,11,10,9,8,7,6,5,4,3,2,1)

#define _PP_REFLECT_1( N, MACRO, NS, T, M)				MACRO(N-1, NS, T, M)
#define _PP_REFLECT_2( N, MACRO, NS, T, M, ...)			MACRO(N-2, NS, T, M)  _PP_REFLECT_1(N, MACRO, NS, T, __VA_ARGS__)
#define _PP_REFLECT_3( N, MACRO, NS, T, M, ...)			MACRO(N-3, NS, T, M)  _PP_REFLECT_2(N, MACRO, NS, T, __VA_ARGS__)
#define _PP_REFLECT_4( N, MACRO, NS, T, M, ...)			MACRO(N-4, NS, T, M)  _PP_REFLECT_3(N, MACRO, NS, T, __VA_ARGS__)
#define _PP_REFLECT_5( N, MACRO, NS, T, M, ...)			MACRO(N-5, NS, T, M)  _PP_REFLECT_4(N, MACRO, NS, T, __VA_ARGS__)
#define _PP_REFLECT_6( N, MACRO, NS, T, M, ...)			MACRO(N-6, NS, T, M)  _PP_REFLECT_5(N, MACRO, NS, T, __VA_ARGS__)
#define _PP_REFLECT_7( N, MACRO, NS, T, M, ...)			MACRO(N-7, NS, T, M)  _PP_REFLECT_6(N, MACRO, NS, T, __VA_ARGS__)
#define _PP_REFLECT_8( N, MACRO, NS, T, M, ...)			MACRO(N-8, NS, T, M)  _PP_REFLECT_7(N, MACRO, NS, T, __VA_ARGS__)
#define _PP_REFLECT_9( N, MACRO, NS, T, M, ...)			MACRO(N-9, NS, T, M)  _PP_REFLECT_8(N, MACRO, NS, T, __VA_ARGS__)
#define _PP_REFLECT_10(N, MACRO, NS, T, M, ...)			MACRO(N-10, NS, T, M) _PP_REFLECT_9(N, MACRO, NS, T, __VA_ARGS__)
#define _PP_REFLECT_11(N, MACRO, NS, T, M, ...)			MACRO(N-11, NS, T, M) _PP_REFLECT_10(N, MACRO, NS, T, __VA_ARGS__)
#define _PP_REFLECT_12(N, MACRO, NS, T, M, ...)			MACRO(N-12, NS, T, M) _PP_REFLECT_11(N, MACRO, NS, T, __VA_ARGS__)
#define _PP_REFLECT_13(N, MACRO, NS, T, M, ...)			MACRO(N-13, NS, T, M) _PP_REFLECT_12(N, MACRO, NS, T, __VA_ARGS__)
#define _PP_REFLECT_14(N, MACRO, NS, T, M, ...)			MACRO(N-14, NS, T, M) _PP_REFLECT_13(N, MACRO, NS, T, __VA_ARGS__)
#define _PP_REFLECT_15(N, MACRO, NS, T, M, ...)			MACRO(N-15, NS, T, M) _PP_REFLECT_14(N, MACRO, NS, T, __VA_ARGS__)
#define _PP_REFLECT_16(N, MACRO, NS, T, M, ...)			MACRO(N-16, NS, T, M) _PP_REFLECT_15(N, MACRO, NS, T, __VA_ARGS__)
#define _PP_REFLECT_THINGS(N, MACRO, ...)				_PP_EXPAND(_PP_REFLECT_##N(N, MACRO, __VA_ARGS__))

#define _PP_REFLECT_ENUM_INNER(NS, T, S, ...)			struct T##_Ctti \
	{ \
	inline static constexpr char Namespace[] = #NS; \
	inline static constexpr char Name[] = #T; \
	using Type = NS :: T; \
	using enum NS :: T; \
	inline static constexpr std::string_view AllNames = S; \
	inline static constexpr auto Names = ::PlainProps::Private::ParseArgs<AllNames>; \
	static constexpr NS :: T Constants[] = { __VA_ARGS__ }; \
	static constexpr uint32 NumEnumerators = sizeof(Constants)/sizeof(NS :: T); \
	static_assert(NumEnumerators == static_cast<uint32>(Names.size())); \
	}; \
	T##_Ctti CttiOfPtr(NS :: T*);

#define _PP_REFLECT_STRUCT(N, NS, T, S, ...)			PP_REFLECT_STRUCT_ONLY(N, NS, T, S)				_PP_EXPAND(_PP_REFLECT_THINGS(N, PP_REFLECT_MEMBER, NS, T, __VA_ARGS__))
#define _PP_REFLECT_STRUCT_TEMPLATE(N, NS, T, S, ...)	PP_REFLECT_STRUCT_TEMPLATE_ONLY(N, NS, T, S)	_PP_EXPAND(_PP_REFLECT_THINGS(N, PP_REFLECT_TEMPLATE_MEMBER, NS, T, __VA_ARGS__))

//////////////////////////////////////////////////////////////////////////

#define PP_REFLECT_ENUM(NS, T, ...)						_PP_REFLECT_ENUM_INNER(NS, T, #__VA_ARGS__, __VA_ARGS__)
#define PP_REFLECT_FLAG_ENUM(NS, T, ...)				_PP_REFLECT_ENUM_INNER(NS, T, #__VA_ARGS__, __VA_ARGS__); static_assert(::PlainProps::Private::ValidateEnumFlags<T##_Ctti>(), "Flag enum " #T " must use one bit per enumerator")
#define PP_REFLECT_STRUCT(NS, T, S, ...)				_PP_REFLECT_STRUCT(_PP_NUM_ARGS(__VA_ARGS__), NS, T, S, __VA_ARGS__)
#define PP_REFLECT_STRUCT_TEMPLATE(NS, T, S, ...)		_PP_REFLECT_STRUCT_TEMPLATE(_PP_NUM_ARGS(__VA_ARGS__), NS, T, S, __VA_ARGS__)
#define PP_NAME_STRUCT(NS, T)							struct T##_Ctti { inline static constexpr char Name[] = #T; using Type = NS :: T; inline static constexpr char Namespace[] = #NS; }; T##_Ctti CttiOfPtr(T*);
#define PP_NAME_STRUCT_TEMPLATE(NS, T)					template<class... Ts> struct T##_Ctti { inline static constexpr char Name[] = #T; using Type = NS :: T<Ts...>; using TemplateArgs = std::tuple<Ts...>; inline static constexpr char Namespace[] = #NS; }; template<class... Ts> T##_Ctti<Ts...> CttiOfPtr(T<Ts...>*);
		
// Alternate set of macros to reflect classes/structs with bitfield bool members, e.g. uint8 bOol : 1;

#define PP_REFLECT_STRUCT_ONLY(N, NS, T, S)				struct T##_Ctti { inline static constexpr char Name[] = #T; using Type = NS :: T; using Super = S; static constexpr int NumVars = N; template<int> struct Var; inline static constexpr char Namespace[] = #NS; }; T##_Ctti CttiOfPtr(T*);
#define PP_REFLECT_STRUCT_TEMPLATE_ONLY(N, NS, T, S)	template<class... Ts> struct T##_Ctti { inline static constexpr char Name[] = #T; using Type = NS :: T<Ts...>; using Super = S; using TemplateArgs = std::tuple<Ts...>; static constexpr int NumVars = N; template<int, int> struct Var_; template<int I> using Var = Var_<I, 0>; inline static constexpr char Namespace[] = #NS; }; template<class... Ts> T##_Ctti<Ts...> CttiOfPtr(T<Ts...>*);
#define PP_REFLECT_MEMBER(N, NS, T, M)					template<> struct T##_Ctti::Var<N> { inline static constexpr char Name[] = #M; using Type = decltype(NS :: T :: M); static constexpr std::size_t Offset = offsetof(NS :: T, M); static constexpr auto Pointer = &NS :: T :: M; static constexpr int Index = N; };
#define PP_REFLECT_BIT_MEMBER(N, NS, T, M, BYTEOFFSET, BITOFFSET) template<> struct T##_Ctti::Var<N> { inline static constexpr char Name[] = #M; using Type = decltype(NS :: T :: M); static constexpr std::size_t Offset = BYTEOFFSET; static constexpr int BitIndex = BITOFFSET; static constexpr int Index = N; };
#define PP_REFLECT_TEMPLATE_MEMBER(N, NS, T, M)			template<class... Ts> template <int _> struct T##_Ctti<Ts...>::Var_<N, _> { inline static constexpr char Name[] = #M; using Type = decltype(NS :: T<Ts...> :: M); static constexpr std::size_t Offset = offsetof(NS :: T<Ts...>, M); static constexpr auto Pointer = &NS :: T<Ts...> :: M; static constexpr int Index = N; };
#define PP_REFLECT_TEMPLATE_BIT_MEMBER					todo

//////////////////////////////////////////////////////////////////////////

// E.g. ForEachVar<CttiOf<MyClass>>([]<class Var>() { printf(Var::Name); });
template <class Ctti, typename Fn>
static constexpr void ForEachVar(Fn&& Visitor)
{
    constexpr int N = Ctti::NumVars;
    static_assert(N <= 16, "Member overflow");
    if constexpr (N > 0) Visitor.template operator()<typename Ctti::template Var<0>>();
    if constexpr (N > 1) Visitor.template operator()<typename Ctti::template Var<1>>();
    if constexpr (N > 2) Visitor.template operator()<typename Ctti::template Var<2>>();
    if constexpr (N > 3) Visitor.template operator()<typename Ctti::template Var<3>>();
    if constexpr (N > 4) Visitor.template operator()<typename Ctti::template Var<4>>();
    if constexpr (N > 5) Visitor.template operator()<typename Ctti::template Var<5>>();
    if constexpr (N > 6) Visitor.template operator()<typename Ctti::template Var<6>>();
    if constexpr (N > 7) Visitor.template operator()<typename Ctti::template Var<7>>();
    if constexpr (N > 8) Visitor.template operator()<typename Ctti::template Var<8>>();
    if constexpr (N > 9) Visitor.template operator()<typename Ctti::template Var<9>>();
    if constexpr (N > 10) Visitor.template operator()<typename Ctti::template Var<10>>();
    if constexpr (N > 11) Visitor.template operator()<typename Ctti::template Var<11>>();
    if constexpr (N > 12) Visitor.template operator()<typename Ctti::template Var<12>>();
    if constexpr (N > 13) Visitor.template operator()<typename Ctti::template Var<13>>();
    if constexpr (N > 14) Visitor.template operator()<typename Ctti::template Var<14>>();
    if constexpr (N > 15) Visitor.template operator()<typename Ctti::template Var<15>>();
}

//////////////////////////////////////////////////////////////////////////

template<class Ctti>
concept Templated = requires { typename Ctti::TemplateArgs; };

} // namespace PlainProps
