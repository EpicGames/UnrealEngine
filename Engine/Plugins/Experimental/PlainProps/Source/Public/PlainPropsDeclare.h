// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsTypes.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Templates/UniquePtr.h"
#include <atomic>

template<typename T> class TRefCountPtr;

namespace PlainProps
{
struct FStructSpec;

////////////////////////////////////////////////////////////////////////////////////////////////

struct FEnumerator
{
	FNameId							Name;
	uint64							Constant;

	friend bool operator==(FEnumerator, FEnumerator) = default;
};

enum class EEnumMode : uint8 { Flat, Flag };

struct FEnumDeclaration
{
	FType							Type;			// Could be removed
	EEnumMode						Mode;
	ELeafWidth						Width;
	uint16							NumEnumerators;
	FEnumerator						Enumerators[0];	// Constants must be unique, no aliases allowed

	TConstArrayView<FEnumerator>	GetEnumerators() const		{ return MakeArrayView(Enumerators, NumEnumerators);	}
};

////////////////////////////////////////////////////////////////////////////////////////////////

// Enum values are stored as integers. Aliased enum values are illegal, including composite flags.
// Aliases can be automatically removed on declaration or detected and fail hard.
enum class EEnumAliases { Strip, Fail };

class FEnumDeclarations
{
public:
	UE_NONCOPYABLE(FEnumDeclarations);
	explicit FEnumDeclarations(FDebugIds In) : Debug(In) {}

	PLAINPROPS_API const FEnumDeclaration&	Declare(FEnumId Id, FType Type, EEnumMode Mode, ELeafWidth Width, TConstArrayView<FEnumerator> Enumerators, EEnumAliases Policy);
	void									Drop(FEnumId Id)				{ Check(Id); Declarations[Id.Idx].Reset(); }
	const FEnumDeclaration&					Get(FEnumId Id) const			{ Check(Id); return *Declarations[Id.Idx].Get(); }
	const FEnumDeclaration*					Find(FEnumId Id) const			{ return Id.Idx < (uint32)Declarations.Num() ? Declarations[Id.Idx].Get() : nullptr; }

	FDebugIds								GetDebug() const				{ return Debug; }

protected:
	TArray<TUniquePtr<FEnumDeclaration>>	Declarations;
	FDebugIds								Debug;

#if DO_CHECK
	PLAINPROPS_API void						Check(FEnumId Id) const;
#else
	void									Check(FEnumId) const {}
#endif
};

////////////////////////////////////////////////////////////////////////////////////////////////

enum class EMemberPresence : uint8 { RequireAll, AllowSparse };

struct FStructDeclaration
{
	mutable std::atomic<int32>		RefCount = {};
	FDeclId							Id;				// Could be removed, might allow declaration dedup among templated types
	FOptionalDeclId					Super;
	uint16							Version;
	uint16							NumMembers;
	uint16							NumInnerRanges;
	uint16							NumInnerIds;
	EMemberPresence					Occupancy;
	FMemberId						MemberNames[0];

	const FInnerId*					GetInnerIds() const			{ return AlignPtr<FInnerId>(MemberNames + NumMembers); }
	const FMemberType*				GetTypes() const			{ return AlignPtr<FMemberType>(GetInnerIds() + NumInnerIds); }
	const FMemberType*				GetInnerRangeTypes() const	{ return GetTypes() + NumMembers; }
	TConstArrayView<FMemberId>		GetMemberOrder() const		{ return MakeArrayView(MemberNames, NumMembers); }
	uint32							CalculateSize() const;
	
	void							AddRef() const				{ RefCount.fetch_add(1, std::memory_order_relaxed);	}
	PLAINPROPS_API bool				Release() const;

	inline static constexpr uint16 MaxMembers = 0xFFFF;
};

////////////////////////////////////////////////////////////////////////////////////////////////

using FStructDeclarationPtr = ::TRefCountPtr<const FStructDeclaration>;

PLAINPROPS_API FStructDeclarationPtr Declare(FStructSpec Spec);

////////////////////////////////////////////////////////////////////////////////////////////////

struct IDeclarations
{
	virtual ~IDeclarations() {}
	virtual FDeclId								Lower(FBindId BindId) const = 0;
	virtual const FEnumDeclaration*				Find(FEnumId Id) const = 0;
	virtual const FStructDeclaration*			Find(FStructId Id) const = 0;
	
	PLAINPROPS_API const FEnumDeclaration&		Get(FEnumId Id) const;
	PLAINPROPS_API const FStructDeclaration&	Get(FStructId Id) const;
};

} // namespace PlainProps
