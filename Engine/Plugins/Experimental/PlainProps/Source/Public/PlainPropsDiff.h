// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsBind.h"

namespace PlainProps 
{

union FDiffMetadata
{
	FOptionalEnumId			Leaf;
	FRangeBinding			Range;
	FBindId					Struct;
};

// Currently lacking range indices
struct FDiffNode
{
	FMemberBindType			Type;
	FOptionalMemberId		Name;
	FDiffMetadata			Meta;
	const void*				A;
	const void*				B;
	int32					PrecedingSibling = INDEX_NONE; // Index of the previous sibling of the node in the tree chain.
};

// Flat tree of nodes. Nodes are linked via the index to a previous sibling.
// A tree can contain multiple diff paths.
struct FDiffTree : public TArray<FDiffNode, TInlineAllocator<16>> {};

enum EDiffGather { First, All };

// Tracks diff path for diff tools unlike the const FBindContext& overloads for delta saving
struct FDiffContext : FBindContext
{
	EDiffGather		GatherMode = EDiffGather::First;
	FDiffTree		Diffs;
	int32			LastSibling = INDEX_NONE;

	PLAINPROPS_API int32 AddDiff(FDiffNode&& Node);
	PLAINPROPS_API void Reset();
};

inline bool StopOnFirstDiff(const FDiffContext& Ctx)
{
	return (Ctx.GatherMode == EDiffGather::First);
}

inline bool StopOnFirstDiff(const FBindContext& Ctx)
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////
// Tracking and non-tracking methods to diff member leaves/ranges/structs
//
// FDiffContext& overloads track inner FDiffTree, caller must add outermost FDiffNode

[[nodiscard]] PLAINPROPS_API bool DiffStructs(const void* A, const void* B, FBindId Id, const FBindContext& Ctx);
[[nodiscard]] PLAINPROPS_API bool DiffStructs(const void* A, const void* B, FBindId Id, FDiffContext& Ctx);

[[nodiscard]] PLAINPROPS_API bool DiffLeaves(float A, float B);
[[nodiscard]] PLAINPROPS_API bool DiffLeaves(double A, double B);
template<Arithmetic T>		 bool DiffLeaves(T A, T B) { return A != B; }

[[nodiscard]] PLAINPROPS_API bool DiffRanges(const void* A, const void* B, const IItemRangeBinding& Binding, FUnpackedLeafType ItemType);
[[nodiscard]] PLAINPROPS_API bool DiffRanges(const void* A, const void* B, const IItemRangeBinding& Binding, FBindId ItemType, const FBindContext& Ctx);
[[nodiscard]] PLAINPROPS_API bool DiffRanges(const void* A, const void* B, const IItemRangeBinding& Binding, FBindId ItemType, FDiffContext& Ctx);
[[nodiscard]] PLAINPROPS_API bool DiffRanges(const void* A, const void* B, const IItemRangeBinding& Binding, FRangeMemberBinding ItemType, const FBindContext& Ctx);
[[nodiscard]] PLAINPROPS_API bool DiffRanges(const void* A, const void* B, const IItemRangeBinding& Binding, FRangeMemberBinding ItemType, FDiffContext& Ctx);

////////////////////////////////////////////////////////////////////////////////

struct FReadDiffNode
{
	FMemberType				Type;
	FOptionalStructSchemaId	Struct;
	FOptionalMemberId		Name;
	uint64					RangeIdx = ~uint64(0); 
};

struct FReadDiffPath : public TArray<FReadDiffNode, TInlineAllocator<16>> {};

bool PLAINPROPS_API DiffStruct(FStructView A, FStructView B, FReadDiffPath& Out);
bool PLAINPROPS_API DiffSchemas(FSchemaBatchId A, FSchemaBatchId B);

////////////////////////////////////////////////////////////////////////////////

// Struct for converting diff results to a less efficient but easier to
// process structure.
// Nodes in the path are stored in forward order. Sibling links are invalidated.
struct FDiffPath : public TArray<FDiffNode> 
{
	PLAINPROPS_API friend uint32 GetTypeHash(const FDiffPath& Path);
};

PLAINPROPS_API bool operator==(const FDiffPath& A, const FDiffPath& B);

} // namespace PlainProps
