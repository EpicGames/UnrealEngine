// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsTypes.h"
#include "Containers/Set.h"
#include "Containers/StringView.h"

namespace PlainProps
{

class FNestedScopeIndexer
{
public:
	PLAINPROPS_API FNestedScopeId		Index(FNestedScope Scope);
	FNestedScopeId						Index(FScopeId Outer, FFlatScopeId Inner)			{ return Index({Outer, Inner}); }
	PLAINPROPS_API FNestedScope			Resolve(FNestedScopeId Id) const;
	int32								Num() const											{ return Scopes.Num(); }

	auto								begin() const										{ return Scopes.begin(); }
	auto								end() const											{ return Scopes.end(); }
private:
	TSet<FNestedScope> Scopes;
};

//////////////////////////////////////////////////////////////////////////

class FParametricTypeIndexer
{
public:
	PLAINPROPS_API ~FParametricTypeIndexer();

	PLAINPROPS_API FParametricTypeId	Index(FParametricTypeView View);
	/// @return View invalidated by calling Index() (switch Parameters to TPagedArray to avoid)
	PLAINPROPS_API FParametricTypeView	Resolve(FParametricTypeId Id) const;
	FParametricType						At(int32 Idx) const									{ return Types[Idx]; }
	int32								Num() const											{ return Types.Num(); }
	TConstArrayView<FParametricType>	GetAllTypes() const									{ return Types; }
	TConstArrayView<FType>				GetAllParameters() const							{ return Parameters; }

private:
	uint32								NumSlots = 0;
	uint32*								Slots = nullptr;
	TArray<FParametricType>				Types;
	TArray<FType>						Parameters;
};

//////////////////////////////////////////////////////////////////////////

// Allows indexes higher level ids from existing parts
class FIdIndexerBase : public FIds
{
public:
	PLAINPROPS_API FScopeId				NestFlatScope(FScopeId Outer, FFlatScopeId Inner);
	PLAINPROPS_API FScopeId				NestReversedScopes(TConstArrayView<FFlatScopeId> Inners);
	PLAINPROPS_API FParametricTypeId	MakeParametricTypeId(FOptionalConcreteTypenameId Name, TConstArrayView<FType> Params);
	PLAINPROPS_API FType				MakeParametricType(FType Type, TConstArrayView<FType> Params);
	PLAINPROPS_API FType				MakeAnonymousParametricType(TConstArrayView<FType> Params);
	FType								MakeLeafParameter(FUnpackedLeafType Leaf)			{ return { NoId, FTypenameId(Leaves[uint8(Leaf.Type)][uint8(Leaf.Width)]) }; }
	FType								MakeRangeParameter(ESizeType SizeType)				{ return { NoId, FTypenameId(Ranges[uint8(SizeType)]) }; }

	PLAINPROPS_API FEnumId				IndexEnum(FType Type);
	PLAINPROPS_API FStructId			IndexStruct(FType Type);
	FDeclId								IndexDeclId(FType Type)								{ return static_cast<FDeclId>(IndexStruct(Type)); }
	FBindId								IndexBindId(FType Type)								{ return static_cast<FBindId>(IndexStruct(Type)); }
	
	virtual uint32						NumNestedScopes() const override final				{ return IntCastChecked<uint32>(NestedScopes.Num()); }
	virtual uint32						NumParametricTypes() const override final			{ return IntCastChecked<uint32>(ParametricTypes.Num()); }
	virtual uint32						NumEnums() const override final						{ return Enums.Num(); }
	virtual uint32						NumStructs() const override final					{ return Structs.Num(); }
	
	virtual FNestedScope				Resolve(FNestedScopeId Id) const override final		{ return NestedScopes.Resolve(Id); }
	virtual FParametricTypeView			Resolve(FParametricTypeId Id) const override final	{ return ParametricTypes.Resolve(Id); }
	PLAINPROPS_API virtual FType		Resolve(FEnumId Id) const override final;
	PLAINPROPS_API virtual FType		Resolve(FStructId Id) const override final;
	
	const FNestedScopeIndexer&			GetNestedScopes() const								{ return NestedScopes; }
	const FParametricTypeIndexer&		GetParametricTypes() const							{ return ParametricTypes; }
protected:
	FNestedScopeIndexer					NestedScopes;
	FParametricTypeIndexer				ParametricTypes;
	TSet<FType>							Enums;
	TSet<FType>							Structs;
	FConcreteTypenameId					Leaves[8][4];
	FConcreteTypenameId					Ranges[9];

	PLAINPROPS_API void					InitParameterNames();
	FConcreteTypenameId					InitParameterName(FAnsiStringView Name)				{ return {IndexLiteral(Name)}; }

private:
	friend class FLiteralIndexerBase;
	virtual FNameId						IndexLiteral(FAnsiStringView Name) = 0;
};

// Also allows indexing new names from string literals
class FLiteralIndexerBase : public FIdIndexerBase
{
public:
	// Override to split scope into parts, default implementation assumes a single flat scope
	virtual FScopeId					IndexScopeLiteral(FAnsiStringView Scope)			{ return Scope.Len() ? FScopeId(FFlatScopeId{IndexLiteral(Scope)}) : NoId; }
	FConcreteTypenameId					IndexTypenameLiteral(FAnsiStringView Type)			{ return {IndexLiteral(Type)}; }
	FMemberId							IndexMemberLiteral(FAnsiStringView Member)			{ return {IndexLiteral(Member)}; }
	FNameId								IndexEnumeratorLiteral(FAnsiStringView Flag)		{ return IndexLiteral(Flag); }
};

template<class NameType>
void AppendString(FUtf8StringBuilderBase& Out, const NameType& Str);

// Indexes FNameId from unique NameT instances or NameT-constructible types
template<class NameT>
class TIdIndexer : public FLiteralIndexerBase
{
public:
	using FIdIndexerBase::AppendString;

	TIdIndexer() { InitParameterNames(); }

	template<int N>	FNameId						MakeName(const char (&Name)[N])			{ return MakeName(FAnsiStringView(Name, N - 1)); }
	template<typename T> FNameId				MakeName(T&& Name)						{ return { static_cast<uint32>(Names.Add(NameT(Forward<T>(Name))).AsInteger()) }; }
	template<typename T> FMemberId				NameMember(T&& Name)					{ return { MakeName(Forward<T>(Name)) }; }
	template<typename T> FConcreteTypenameId	NameType(T&& Name)						{ return { MakeName(Forward<T>(Name)) }; }
	template<typename T> FFlatScopeId			NameScope(T&& Name)						{ return { MakeName(Forward<T>(Name)) }; }
	template<typename T> FScopeId				MakeScope(T&& Name)						{ return FScopeId(NameScope(Forward<T>(Name))); }
	template<typename T> FScopeId				NestScope(FScopeId Outer, T&& Inner)	{ return NestFlatScope(Outer, NameScope(Forward<T>(Inner))); }
	template<typename T> FTypenameId			MakeTypename(T&& Name)					{ return FTypenameId(NameType(Forward<T>(Name))); }
	template<typename T> FType					MakeType(T&& Scope, T&& Name)			{ return { MakeScope(Forward<T>(Scope)), MakeTypename(Forward<T>(Name)) }; }

	NameT										ResolveName(FNameId Id)	const			{ return Names.Get(FSetElementId::FromInteger(Id.Idx)); }
	virtual uint32								NumNames() const override final			{ return static_cast<uint32>(Names.Num()); }

	FConcreteTypenameId							IndexRangeBindName(FAnsiStringView Str) { return InitParameterName(Str); }

protected:
	TSet<NameT>									Names;

	virtual void	AppendString(FUtf8Builder& Out, FNameId Id) const override final	{ PlainProps::AppendString(Out, ResolveName(Id));}
	virtual FNameId	IndexLiteral(FAnsiStringView Name) override final					{ return MakeName(NameT(Name)); }
};


} // namespace PlainProps
