// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsRead.h"
#include "Containers/ArrayView.h"
#include "Memory/MemoryFwd.h"
#include "Templates/UniquePtr.h"

namespace PlainProps 
{
	
class FCustomBindings;
class FSchemaBindings;
struct FLoadBatch;

struct FLoadBatchDeleter
{
	PLAINPROPS_API void operator()(FLoadBatch* Ptr) const;
};
using FLoadBatchPtr = TUniquePtr<FLoadBatch, FLoadBatchDeleter>;


[[nodiscard]] PLAINPROPS_API FLoadBatchPtr	CreateLoadPlans(FSchemaBatchId BatchId,const FCustomBindings& Customs, const FSchemaBindings& Schemas, TConstArrayView<FStructId> RuntimeIds, ESchemaFormat Format);
PLAINPROPS_API void							UpdateLoadPlans(FLoadBatch& Batch, const FCustomBindings& Customs, const FSchemaBindings& Schemas, TConstArrayView<FStructId> RuntimeIds);
PLAINPROPS_API bool							HasLoadPlan(FStructSchemaId LoadId, const FLoadBatch& Batch);
PLAINPROPS_API void							LoadStruct(void* Dst, FByteReader Src, FStructSchemaId LoadId, const FLoadBatch& Batch);
PLAINPROPS_API void							ConstructAndLoadStruct(void* Dst, FByteReader Src, FStructSchemaId LoadId, const FLoadBatch& Batch);

} // namespace PlainProps
