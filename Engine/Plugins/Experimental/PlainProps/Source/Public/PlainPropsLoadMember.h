// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsLoad.h"
#include <type_traits>

namespace PlainProps 
{

class FNestedRangeLoadIterator;
class FRangeLoader;
class FRangeBinding;
class FStructRangeLoadIterator;

struct FMemcpyLoadPlan
{
	uint32								Size = 0;
	uint32								Offset = 0;
};

struct FSchemaLoadHandle
{
	FStructSchemaId						LoadId;
	const FLoadBatch&					Batch;

	// Experimental API to help bypass FMemberLoader overhead for dense struct ranges
	//
	// @pre Out.Num() must equal number of members
	PLAINPROPS_API void	GetInnerLoadIds(TArrayView<FOptionalSchemaId> Out) const;
};

// Usable via FMemberLoader or [ConstructAnd]LoadStruct()
struct FStructLoadView
{
	FSchemaLoadHandle					Schema;
	FByteReader							Values;
};

PLAINPROPS_API void	LoadStruct(void* Dst, FStructLoadView Src);
PLAINPROPS_API void	ConstructAndLoadStruct(void* Dst, FStructLoadView Src);

// Load single struct member faster than LoadStruct(Dst, FMemberLoader(Src).GrabStruct())
PLAINPROPS_API void LoadSoleStruct(void* Dst, FStructLoadView Src);

// Load single leaf member faster than FMemberLoader(Src).GrabLeaf().As<T>()
template<LeafType T>
inline void LoadSole(void* Dst, FStructLoadView Src);

template<>
inline void LoadSole<bool>(void* Dst, FStructLoadView Src);

template<LeafType T>
inline T LoadSole(FStructLoadView Src)
{
	T Out;
	LoadSole<T>(&Out, Src);
	return Out;
}

//////////////////////////////////////////////////////////////////////////

struct FRangeLoadSchema
{
	FMemberType						ItemType;
	FOptionalSchemaId				InnermostId;
	const FMemberType*				NestedItemTypes; // For nested ranges, can be out-of-bounds otherwise
	const FLoadBatch&				Batch;
};

using FStructRangeLoadView = TStructuralRangeView<FStructRangeLoadIterator>;
using FNestedRangeLoadView = TStructuralRangeView<FNestedRangeLoadIterator>;

class FRangeLoadView
{
public:
	FRangeLoadView(FRangeLoadSchema S, uint64 N, FMemoryView V) : Schema(S), NumItems(N), Values(V)  {}

	uint64								Num() const				{ return NumItems;}
	bool								IsEmpty() const			{ return NumItems == 0; }
	bool								IsLeafRange() const		{ return Schema.ItemType.GetKind() == EMemberKind::Leaf; }
	bool								IsStructRange() const	{ return Schema.ItemType.GetKind() == EMemberKind::Struct; }
	bool								IsNestedRange() const	{ return Schema.ItemType.GetKind() == EMemberKind::Range; }

	FLeafRangeLoadView					AsLeaves() const;		// @pre IsLeafRange()
	PLAINPROPS_API FStructRangeLoadView	AsStructs() const;		// @pre IsStructRange()
	PLAINPROPS_API FNestedRangeLoadView	AsRanges() const;		// @pre IsNestedRange()

private:
	friend FRangeLoader;

	FRangeLoadSchema				Schema;
	uint64							NumItems;
	FMemoryView						Values;
};

PLAINPROPS_API void	LoadRange(void* Dst, FRangeLoadView Src, TConstArrayView<FRangeBinding> InnerBindings);
// Experimental low-level API bypassing FMemberLoader, internal use only
PLAINPROPS_API void	LoadRange(void* Dst, FByteReader& SrcBytes, FBitCacheReader& SrcBits, ESizeType MaxSize, FRangeLoadSchema Schema, TConstArrayView<FRangeBinding> InnerBindings);

//////////////////////////////////////////////////////////////////////////

// Hides internal representation to enable future format changes, 
// e.g. store zeroes or 1.0f in some compact fashion or var int encodings
class FLeafRangeLoadView
{
	const void*				Data;
	uint64					NumItems;
	FUnpackedLeafType		Leaf;

public:
	FLeafRangeLoadView(const void* InData, uint64 InNum, FUnpackedLeafType InLeaf)
	: Data(InData)
	, NumItems(InNum)
	, Leaf(InLeaf)
	{}

	uint64 Num() const { return NumItems; }
	
	template<typename T>
	auto As() const
	{
		check(Leaf == ReflectLeaf<T>);
		return TRangeView<T>(static_cast<const T*>(Data), NumItems);
	}

	template<>
	auto As<bool>() const
	{
		check(Leaf.Type == ELeafType::Bool && Leaf.Width == ELeafWidth::B8);
		return FBoolRangeView(static_cast<const uint8*>(Data), NumItems);
	}

	template<typename T>
	auto AsBitCast() const requires (std::is_unsigned_v<T>)
	{
		if constexpr (std::is_same_v<T, bool>)
		{
			return As<bool>();
		}
		else
		{
			check(Leaf.Type != ELeafType::Bool);
			check(SizeOf(Leaf.Width) == sizeof(T));
			return TRangeView<T>(reinterpret_cast<const T*>(Data), NumItems);
		}
	}
	
	template<ELeafType Type, ELeafWidth Width>
	auto AsBitCast() const
	{
		if constexpr (Type == ELeafType::Bool)			return As<bool>();
		else if constexpr (Width == ELeafWidth::B8)		return AsBitCast<uint8>();
		else if constexpr (Width == ELeafWidth::B16)	return AsBitCast<uint16>();
		else if constexpr (Width == ELeafWidth::B32)	return AsBitCast<uint32>();
		else if constexpr (Width == ELeafWidth::B64)	return AsBitCast<uint64>();
	}
};

inline FLeafRangeLoadView FRangeLoadView::AsLeaves() const		{ return { Values.GetData(), NumItems, Schema.ItemType.AsLeaf() }; }
inline FLeafRangeLoadView FLeafRangeView::AsLoadView() const	{ return { Values, NumItems, {Type, Width} }; }

//////////////////////////////////////////////////////////////////////////

class FNestedRangeLoadIterator
{
	friend FNestedRangeLoadView;
	FRangeLoadSchema		Schema;
	FByteReader				ByteIt;
	FBitCacheReader			BitIt;

public:
	FNestedRangeLoadIterator(const FRangeLoadSchema& InSchema, FMemoryView Data)
	: Schema(InSchema) 
	, ByteIt(Data)
	{}

	PLAINPROPS_API FRangeLoadView	operator*() const;
	bool							operator!=(const FNestedRangeLoadIterator& Rhs) const { return ByteIt.Peek() != Rhs.ByteIt.Peek(); };
	PLAINPROPS_API void				operator++();
};

class FStructRangeLoadIterator
{
	friend FStructRangeLoadView;
	FSchemaLoadHandle		Schema;
	FByteReader				ByteIt;

public:
	FStructRangeLoadIterator(const FSchemaLoadHandle& InSchema, FMemoryView Data)
	: Schema(InSchema) 
	, ByteIt(Data)
	{}

	FStructLoadView operator*() const { return { Schema, FByteReader(ByteIt.PeekSkippableSlice()) }; }
	bool operator!=(const FStructRangeLoadIterator& Rhs) const { return ByteIt.Peek() != Rhs.ByteIt.Peek(); };
	void operator++() { (void)ByteIt.GrabSkippableSlice(); }
};

//////////////////////////////////////////////////////////////////////////

class FMemberLoader
{
public:
	PLAINPROPS_API explicit FMemberLoader(FStructLoadView Struct);

	bool							HasMore() const				{ return Reader.HasMore(); }
	bool							HasSuper() const			{ return Reader.HasSuper(); }
	
	FOptionalMemberId				PeekName() const			{ return Reader.PeekName(); }
	FMemberId						PeekNameUnchecked() const	{ return Reader.PeekNameUnchecked(); }
	EMemberKind						PeekKind() const			{ return Reader.PeekKind(); }
	FMemberType						PeekType() const			{ return Reader.PeekType(); }

	FLeafView						GrabLeaf()					{ return Reader.GrabLeaf(); }
	PLAINPROPS_API FRangeLoadView	GrabRange();				// @pre PeekKind() == EMemberKind::Range
	PLAINPROPS_API FStructLoadView	GrabStruct();				// @pre PeekKind() == EMemberKind::Struct
private:
	FMemberReader			Reader;
	const FStructSchemaId*	LoadIdIt;
	const FLoadBatch&		Batch;
};

// Grab and load super if it exists, call before grabbing from Src
inline void	TryLoadSuper(void* Dst, /* in-out */ FMemberLoader& Src)
{
	if (Src.HasSuper())
	{
		LoadStruct(Dst, Src.GrabStruct());
	}
}

//////////////////////////////////////////////////////////////////////////

template<LeafType T>
void LoadSole(void* Dst, FStructLoadView Src)
{
	Src.Values.CheckSize(sizeof(T));
	checkSlow(ReflectLeaf<T> == FMemberLoader(Src).GrabLeaf().Leaf);
	FMemory::Memcpy(Dst, Src.Values.Peek(), sizeof(T));
}

template<> inline
void LoadSole<bool>(void* Dst, FStructLoadView Src)
{
	Src.Values.CheckSize(sizeof(bool));
	checkSlow(ReflectLeaf<bool> == FMemberLoader(Src).GrabLeaf().Leaf);
	FBitCacheReader Bits;
	*static_cast<bool*>(Dst) = Bits.GrabNext(Src.Values);
}

} // namespace PlainProps