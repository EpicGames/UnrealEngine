// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsTypes.h"
#include "Memory/MemoryFwd.h"
#include "Misc/Optional.h"

namespace PlainProps
{
class FLooseMember;
struct FLooseNestedRange;
struct FLooseValue;

////////////////////////////////////////////////////////////////////////////////////////////////

enum class EInnermostKind : uint8 { Struct, Enum, Bool, S8, S16, S32, S64, U8, U16, U32, U64, F32, F64, UTF8, UTF16, UTF32 };

struct FInnermostStructType
{
	EInnermostKind			_ : 5;
	uint8					IsDynamic : 1;
	uint8					IsSuper : 1;
	uint8					IsDecomposed : 1;
};

struct FInnermostEnumType
{
	EInnermostKind			_ : 5;
	uint8					FlagMode : 1;
	uint8					Unused : 1;
	uint8					IsDecomposed : 1;
};

union FInnermostType
{
	EInnermostKind			Kind : 5;
	FInnermostStructType	Struct;
	FInnermostEnumType		Enum;
};

inline const bool operator==(FInnermostType A, FInnermostType B) { return BitCast<uint8>(A) == BitCast<uint8>(B); }

inline static constexpr FInnermostType InnermostFlatEnum = { .Enum = { ._ = EInnermostKind::Enum, .FlagMode = 0 } };
inline static constexpr FInnermostType InnermostFlagEnum = { .Enum = { ._ = EInnermostKind::Enum, .FlagMode = 1 } };
inline static constexpr FInnermostType InnermostStaticStruct = { .Struct = { ._ = EInnermostKind::Struct, .IsDynamic = 0 } };
inline static constexpr FInnermostType InnermostDynamicStruct = { .Struct = { ._ = EInnermostKind::Struct, .IsDynamic = 1 } };

template<typename T>
constexpr FInnermostType IllegalInnermost()
{
	static_assert(!sizeof(T), "Unsupported leaf type");
	return InnermostFlatEnum;
}

template<Arithmetic T>
inline constexpr FInnermostType ReflectInnermost = IllegalInnermost<T>();

template<> inline constexpr FInnermostType ReflectInnermost<bool>		= { .Kind = EInnermostKind::Bool };
template<> inline constexpr FInnermostType ReflectInnermost<int8>		= { .Kind = EInnermostKind::S8 };
template<> inline constexpr FInnermostType ReflectInnermost<int16>		= { .Kind = EInnermostKind::S16 };
template<> inline constexpr FInnermostType ReflectInnermost<int32>		= { .Kind = EInnermostKind::S32 };
template<> inline constexpr FInnermostType ReflectInnermost<int64>		= { .Kind = EInnermostKind::S64 };
template<> inline constexpr FInnermostType ReflectInnermost<uint8>		= { .Kind = EInnermostKind::U8 };
template<> inline constexpr FInnermostType ReflectInnermost<uint16>		= { .Kind = EInnermostKind::U16 };
template<> inline constexpr FInnermostType ReflectInnermost<uint32>		= { .Kind = EInnermostKind::U32 };
template<> inline constexpr FInnermostType ReflectInnermost<uint64>		= { .Kind = EInnermostKind::U64 };
template<> inline constexpr FInnermostType ReflectInnermost<float>		= { .Kind = EInnermostKind::F32 };
template<> inline constexpr FInnermostType ReflectInnermost<double>		= { .Kind = EInnermostKind::F64 };
template<> inline constexpr FInnermostType ReflectInnermost<char8_t>	= { .Kind = EInnermostKind::UTF8 };
template<> inline constexpr FInnermostType ReflectInnermost<char16_t>	= { .Kind = EInnermostKind::UTF16 };
template<> inline constexpr FInnermostType ReflectInnermost<char32_t>	= { .Kind = EInnermostKind::UTF32 };

//template<Enumeration>
//inline constexpr FInnermostType ReflectInnermost = { .Kind = EInnermostKind::Enum };
//
//template<typename T>
//inline constexpr FInnermostType ReflectInnermost requires (std::is_class_v<T>) = { .Kind = EInnermostKind::Struct };

////////////////////////////////////////////////////////////////////////////////////////////////

template<Arithmetic T>	uint64	ValueCast(T Value)						{ return static_cast<std::make_unsigned_t<T>>(Value); }
template<> inline		uint64	ValueCast(float Value)					{ return BitCast<uint32>(Value); }
template<> inline		uint64	ValueCast(double Value)					{ return BitCast<uint64>(Value); }

template<Arithmetic T>	T		ArithmeticCast(uint64 Value)			{ return static_cast<T>(IntCastChecked<std::make_unsigned_t<T>>(Value)); }
template<> inline		bool	ArithmeticCast<bool>(uint64 Value)		{ return IntCastChecked<bool>(Value); }
template<> inline		float	ArithmeticCast<float>(uint64 Value)		{ return BitCast<float>(IntCastChecked<uint32>(Value)); }
template<> inline		double	ArithmeticCast<double>(uint64 Value)	{ return BitCast<double>(Value); }

////////////////////////////////////////////////////////////////////////////////////////////////

static constexpr uint32 LooseRangeBits = 6;
static constexpr uint8 MaxRangeNesting = (uint8(1) << LooseRangeBits) - 1; // Todo: Enforce on build side too

struct FLooseType
{
	uint8				OptionalParameter : 1 = 0;
	uint8				NumRanges : LooseRangeBits + /* reserved */ 1 = 0;
	FInnermostType		InnermostType = InnermostStaticStruct;
	uint16				InnermostVersion = 0;
	FOptionalInnerId	InnermostId;

	bool				IsRange() const					{ return NumRanges != 0; }
	bool				IsStruct() const				{ return NumRanges == 0 && InnermostType.Kind == EInnermostKind::Struct; }
	bool				IsLeaf() const					{ return NumRanges == 0 && InnermostType.Kind != EInnermostKind::Struct; }
	bool				IsEnum() const					{ return NumRanges == 0 && InnermostType.Kind == EInnermostKind::Enum; }
	bool				InnermostIsStruct() const		{ return InnermostType.Kind == EInnermostKind::Struct; } 
	bool				InnermostIsEnum() const			{ return InnermostType.Kind == EInnermostKind::Enum; } 

	bool				Compare(FLooseType O) const		{ return WithoutMetadata().AsInt() == O.WithoutMetadata().AsInt(); }
	bool				CompareAll(FLooseType O) const	{ return AsInt() == O.AsInt(); }
	FLooseType			WithoutMetadata() const			{ return {0, NumRanges, InnermostType, InnermostVersion, InnermostId}; }
private:
	uint64				AsInt() const					{ return BitCast<uint64>(*this); }
};

template<Arithmetic T>
inline constexpr FLooseType ReflectLoose = { 0, 0, ReflectInnermost<T>, 0, NoId };

//////////////////////////////////////////////////////////////////////////////////////////////////

// Per struct-indexed member id - allows TArray<T> lookup instead of TMap<FMemberId, T>
struct FParameterId
{
	uint16 Idx = 0xFFFF;

	bool operator==(FParameterId O) const { return Idx == O.Idx; }
	friend uint32 GetTypeHash(FParameterId Id) { return Id.Idx; };
};

// Per enum-indexed enumerator name
struct FEnumeratorId
{
	uint16 Idx = 0xFFFF;

	bool operator==(FEnumeratorId O) const { return Idx == O.Idx; }
	friend uint32 GetTypeHash(FEnumeratorId Id) { return Id.Idx; };
};

// Set of FEnumeratorId, each matching an enum flag name
struct FEnumeratorIdSet
{
	uint64				Bits = 0;		// <= 64 FEnumeratorId flag names are inlined
};

union FEnumeratorData
{
	FEnumeratorId		Flat = {};		// Loose enumerator name (without integer value)
	FEnumeratorIdSet	Flags;			// Loose set of enumerator flag names without integer value
};

// Temporary format used to access individual named enumerators during Op execution
struct FEnumeratorIndex
{
	uint16				Index = 0xFFFF;
};

// Temporary format used to access individual named enum flags during Op execution
struct FEnumeratorIndexSet
{
	uint64				Bits = 0;
};

// Temporary format used to access individual enum members during Op execution
union FEnumIndexData
{
	FEnumeratorIndex	Flat = {};			// Enumerator index from NameEnumerators order
	FEnumeratorIndexSet	Flags;				// Bitset of flag indices from NameFlags order
};

union FEnumLooseData
{
	FEnumeratorData		Enumerator = {};	// FEnumeratorId::Idx order from FEnumeratorIndexer
	FEnumIndexData		Decomposed;			// Index order from NameEnumerators/NameFlags
};

union FEnumData
{
	uint64				Source = 0;			// Integer source value matching the innermost enum schema
	FEnumLooseData		Loose;
};

//////////////////////////////////////////////////////////////////////////////////////////////////

union FEnumRange
{
	const void*				Source = nullptr;		// Integers source values matching the innermost enum schema
	FEnumLooseData*			Loose;				
};

union FStructRange
{
	const void*				Source = nullptr;		// Skippable slice of the FIRST source FStructView, not the range of structs
	FLooseValue*			Loose;
};

union FArithmeticRange
{
	const void*				Source = nullptr;
	void*					Loose;
};

union FRangeData
{
	FArithmeticRange		Arithmetics = {};
	FEnumRange				Enums;
	FStructRange			Structs;
	FLooseNestedRange*		Ranges; 
};

struct FLooseNestedRange
{
	uint64					Num = 0;
	FRangeData				Data;
};

//////////////////////////////////////////////////////////////////////////////////////////////////

union FStructData
{
	const uint8*		Source = nullptr;	// Skippable slice from a source FStructView
	FLooseMember*		Loose;
};

//////////////////////////////////////////////////////////////////////////////////////////////////

union FLooseData
{
	uint64				Arithmetic = 0;
	FEnumData			Enum;
	FRangeData			Range;
	FStructData			Struct;
};

//////////////////////////////////////////////////////////////////////////////////////////////////

enum class EMismatch : uint8 { No, Inners, Inputs, Type, Duplicate, Surplus, Missing };

struct FLooseMetadata
{
	uint8				IsSet = 0;
	uint8				IsSourceRange : 1 = 0;
	uint8				IsFullyUpgraded : 1 = 0;
	EMismatch			Mismatch : 6 = EMismatch::No;	// Only need 4 bits currently but want all bits initialized
	FParameterId		Name;
	FOptionalSchemaId	InnermostSchema;				// Set if innermost is source struct/enum
	uint64				Num = 0;						// Range items, loose struct members or inner members / enumerators during op execution
};

struct FLooseValue
{
	FLooseMetadata		Meta;
	FLooseData			Data;

	explicit operator	bool() const			{ return !!Meta.IsSet; }
	bool				IsMismatch() const		{ return Meta.Mismatch != EMismatch::No; }
	void				SetNamed(FLooseValue In);				// Assign but keep name
	void				SetFirstMismatch(EMismatch Mismatch);
};

inline void FLooseValue::SetNamed(FLooseValue In)
{
	In.Meta.Name = Meta.Name;
	*this = In;
}

inline void FLooseValue::SetFirstMismatch(EMismatch Mismatch)
{
	if (Meta.Mismatch == EMismatch::No)
	{
		Meta.Mismatch = Mismatch;
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////

// Typed and named optional value
class FLooseMember
{
public:
	FLooseMember(FLooseType T, FLooseValue V) : Type(T), Value(V) {}
	
	explicit			operator bool() const			{ return !!Value.Meta.IsSet; }
	bool				IsMismatch() const				{ return Value.IsMismatch(); }
	bool				IsOptional() const				{ return Type.OptionalParameter; }
	FLooseType			GetType() const					{ return Type; }
	FLooseValue			GetValue() const				{ return Value; }
	FParameterId		GetName() const					{ return Value.Meta.Name; }
	void				Rename(FParameterId Name)		{ Value.Meta.Name = Name; }
	void				SetTypechecked(FLooseValue V)	{ Value = V; } 
	FLooseValue&		EditTypechecked()				{ return Value; }
	void				RecordDynamicType(FLooseType T);
	void				MarkEnumRecomposed()			{ Type.InnermostType.Enum.IsDecomposed = 0; }
	void				MarkStructRecomposed()			{ Type.InnermostType.Struct.IsDecomposed = 0; }

protected:
	FLooseType			Type;
	FLooseValue			Value;
};

} // namespace PlainProps