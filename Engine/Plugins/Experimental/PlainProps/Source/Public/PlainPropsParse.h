// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringView.h"
#include "PlainPropsTypes.h"

namespace PlainProps
{

template<typename T> TOptional<T> Parse(FUtf8StringView String)
{
	static_assert(!sizeof(T), "Unsupported type for Parse");
}

template<> PLAINPROPS_API TOptional<ESizeType>			Parse(FUtf8StringView String);
template<> PLAINPROPS_API TOptional<ELeafWidth>			Parse(FUtf8StringView String);

template<> PLAINPROPS_API TOptional<FUnpackedLeafType>	Parse(FUtf8StringView String);

template<> PLAINPROPS_API TOptional<bool>				Parse(FUtf8StringView String);
template<> PLAINPROPS_API TOptional<int8> 				Parse(FUtf8StringView String);
template<> PLAINPROPS_API TOptional<int16>				Parse(FUtf8StringView String);
template<> PLAINPROPS_API TOptional<int32>				Parse(FUtf8StringView String);
template<> PLAINPROPS_API TOptional<int64>				Parse(FUtf8StringView String);
template<> PLAINPROPS_API TOptional<uint8> 				Parse(FUtf8StringView String);
template<> PLAINPROPS_API TOptional<uint16>				Parse(FUtf8StringView String);
template<> PLAINPROPS_API TOptional<uint32>				Parse(FUtf8StringView String);
template<> PLAINPROPS_API TOptional<uint64>				Parse(FUtf8StringView String);
template<> PLAINPROPS_API TOptional<float>				Parse(FUtf8StringView String);
template<> PLAINPROPS_API TOptional<double>				Parse(FUtf8StringView String);
template<> PLAINPROPS_API TOptional<char>				Parse(FUtf8StringView String);
template<> PLAINPROPS_API TOptional<char8_t>			Parse(FUtf8StringView String);
template<> PLAINPROPS_API TOptional<char16_t>			Parse(FUtf8StringView String);
template<> PLAINPROPS_API TOptional<char32_t>			Parse(FUtf8StringView String);

template<typename T> TOptional<T> ParseHex(FUtf8StringView String)
{
	static_assert(!sizeof(T), "Unsupported type for ParseHex");
}
template<> PLAINPROPS_API TOptional<uint8>				ParseHex(FUtf8StringView String);
template<> PLAINPROPS_API TOptional<uint16>				ParseHex(FUtf8StringView String);
template<> PLAINPROPS_API TOptional<uint32>				ParseHex(FUtf8StringView String);
template<> PLAINPROPS_API TOptional<uint64>				ParseHex(FUtf8StringView String);

///////////////////////////////////////////////////////////////////////////////

void PLAINPROPS_API ParseYamlBatch(TArray64<uint8>& OutBinary, FUtf8StringView Yaml);
void PLAINPROPS_API ParseJsonBatch(TArray64<uint8>& OutBinary, FUtf8StringView Json);

} // namespace PlainProps
