// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsTypes.h"
#include "Containers/StringFwd.h"

namespace PlainProps
{

class FBatchIds;
class FSchemaBatchId;
struct FDiffPath;
struct FMemberSchema;
struct FReadDiffPath;
struct FStructView;

PLAINPROPS_API FAnsiStringView ToString(ELeafWidth Width);
PLAINPROPS_API FAnsiStringView ToString(EMemberKind Kind);
PLAINPROPS_API FAnsiStringView ToString(ESizeType Width);
PLAINPROPS_API FAnsiStringView ToString(ESchemaFormat Format);
PLAINPROPS_API FAnsiStringView ToString(FMemberType Member);
PLAINPROPS_API FAnsiStringView ToString(FStructType Struct);
PLAINPROPS_API FAnsiStringView ToString(FUnpackedLeafType Leaf);
PLAINPROPS_API FAnsiStringView ToString(FRangeType Range);

///////////////////////////////////////////////////////////////////////////////

template<typename T> inline void Print(FUtf8Builder& Out, T Value)
{
	static_assert(!sizeof(T), "Unsupported type for Print");
}

template<> PLAINPROPS_API void Print(FUtf8Builder& Out, ESizeType Value);
template<> PLAINPROPS_API void Print(FUtf8Builder& Out, ELeafWidth Value);

template<> PLAINPROPS_API void Print(FUtf8Builder& Out, bool Value);
template<> PLAINPROPS_API void Print(FUtf8Builder& Out, int8 Value);
template<> PLAINPROPS_API void Print(FUtf8Builder& Out, int16 Value);
template<> PLAINPROPS_API void Print(FUtf8Builder& Out, int32 Value);
template<> PLAINPROPS_API void Print(FUtf8Builder& Out, int64 Value);
template<> PLAINPROPS_API void Print(FUtf8Builder& Out, uint8 Value);
template<> PLAINPROPS_API void Print(FUtf8Builder& Out, uint16 Value);
template<> PLAINPROPS_API void Print(FUtf8Builder& Out, uint32 Value);
template<> PLAINPROPS_API void Print(FUtf8Builder& Out, uint64 Value);
template<> PLAINPROPS_API void Print(FUtf8Builder& Out, float Value);
template<> PLAINPROPS_API void Print(FUtf8Builder& Out, double Value);
template<> PLAINPROPS_API void Print(FUtf8Builder& Out, char8_t Value);
template<> PLAINPROPS_API void Print(FUtf8Builder& Out, char16_t Value);
template<> PLAINPROPS_API void Print(FUtf8Builder& Out, char32_t Value);

template<typename T> inline void PrintHex(FUtf8Builder& Out, T Value)
{
	static_assert(!sizeof(T), "Unsupported type for PrintHex");
}
template<> PLAINPROPS_API void PrintHex(FUtf8Builder& Out, uint8 Value);
template<> PLAINPROPS_API void PrintHex(FUtf8Builder& Out, uint16 Value);
template<> PLAINPROPS_API void PrintHex(FUtf8Builder& Out, uint32 Value);
template<> PLAINPROPS_API void PrintHex(FUtf8Builder& Out, uint64 Value);

///////////////////////////////////////////////////////////////////////////////

PLAINPROPS_API void PrintMemberSchema(FUtf8Builder& Out, const FIds& Ids, FMemberSchema Member);
PLAINPROPS_API void PrintYamlBatch(FUtf8Builder& Out, const FBatchIds& Ids, TConstArrayView<FStructView> Objects);
PLAINPROPS_API void PrintJsonBatch(FUtf8Builder& Out, const FBatchIds& Ids, TConstArrayView<FStructView> Objects);
PLAINPROPS_API void PrintDiff(FUtf8Builder& Out, const FIds& Ids, const FDiffPath& Diff);
PLAINPROPS_API void PrintDiff(FUtf8Builder& Out, const FBatchIds& Ids, const FReadDiffPath& Diff);
PLAINPROPS_API void PrintDiffs(FUtf8Builder& Out, const FIds& Ids, TConstArrayView<FDiffPath> Diffs);

} // namespace PlainProps
