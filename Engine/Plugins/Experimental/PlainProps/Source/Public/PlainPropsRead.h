// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "HAL/PlatformMemory.h"
#include "Memory/MemoryView.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/VarInt.h"
#include "PlainPropsTypes.h"

namespace PlainProps
{

class FLeafRangeLoadView;
struct FEnumSchema;
struct FSchemaBatch;
struct FStructSchema;

//////////////////////////////////////////////////////////////////////////

// Represents a batch currently being read from
class FSchemaBatchId
{
	friend class FReadSchemaRegistry;
	uint16 Idx = 0;
};

// @param Schemas must outlive read batch
PLAINPROPS_API const FSchemaBatch*		ValidateSchemas(FMemoryView Schemas);
PLAINPROPS_API FSchemaBatchId			MountReadSchemas(const FSchemaBatch* Schemas);
PLAINPROPS_API const FSchemaBatch*		UnmountReadSchemas(FSchemaBatchId Batch);

PLAINPROPS_API uint32					NumStructSchemas(FSchemaBatchId Batch);
PLAINPROPS_API const FStructSchema&		ResolveStructSchema(FSchemaBatchId Batch, FStructSchemaId Id);
PLAINPROPS_API const FEnumSchema&		ResolveEnumSchema(FSchemaBatchId Batch, FEnumSchemaId Id);
PLAINPROPS_API FNestedScope				ResolveUntranslatedNestedScope(FSchemaBatchId Batch, FNestedScopeId Id);
PLAINPROPS_API FParametricTypeView		ResolveUntranslatedParametricType(FSchemaBatchId Batch, FParametricTypeId Id);

//////////////////////////////////////////////////////////////////////////

class FByteReader
{
	const uint8* It = nullptr;
#if DO_CHECK
	const uint8* End = nullptr;
#endif
public:
	FByteReader() = default;
	explicit FByteReader(FMemoryView View) : FByteReader(static_cast<const uint8*>(View.GetData()), View.GetSize()) {}
	FByteReader(const uint8* Data, uint64 NumBytes) : FByteReader(Data, Data + NumBytes) {}
	FByteReader(const uint8* InBegin, const uint8* InEnd)
	: It(InBegin)
#if DO_CHECK
	, End(InEnd)
#endif
	{}

	[[nodiscard]] inline const uint8* GrabBytes(uint64 NumBytes)
	{
#if DO_CHECK
		check(It + NumBytes <= End);
#endif
		const uint8* Out = It;
		It += NumBytes;
		return Out;
	}

	[[nodiscard]] inline FMemoryView GrabSlice(uint64 NumBytes)
	{
		return FMemoryView(GrabBytes(NumBytes), NumBytes);
	}

	[[nodiscard]] inline FMemoryView GrabSkippableSlice()
	{
		return GrabSlice(GrabVarIntU());
	}

	[[nodiscard]] inline uint8 GrabByte()
	{
		return *GrabBytes(1);
	}

	template<typename T>
	[[nodiscard]] inline T Grab()
	{
		return FPlatformMemory::ReadUnaligned<T>(GrabBytes(sizeof(T)));
	}
	
	[[nodiscard]] FORCEINLINE uint64 GrabVarIntU()
	{
#if DO_CHECK
		checkSlow(It < End);
#endif
		uint32 NumBytesRead;
		uint64 Out = ReadVarUInt(It, NumBytesRead);
		It += NumBytesRead;
#if DO_CHECK
		checkSlow(It <= End);
#endif
		return Out;
	}

	[[nodiscard]] const uint8* Peek() const { return It; }
	
	[[nodiscard]] FMemoryView PeekSkippableSlice() const { return FByteReader(*this).GrabSkippableSlice(); }

	template<typename T>
	void SkipAlignmentPadding()
	{
		for (; !IsAligned(It, alignof(T)); ++It)
		{
			check(*It == 0);
		}
	}
	
	void CheckEmpty() const
	{
#if DO_CHECK
		check(It == End);
#endif
	}
	void CheckNonEmpty() const
	{
#if DO_CHECK
		check(It != End);
#endif
	}
	void CheckSize(int64 ExpectedSize) const
	{
#if DO_CHECK
		check(End - It == ExpectedSize);
#endif
	}

	uint64 CheckableSize() const
	{
#if DO_CHECK
		return static_cast<uint64>(End - It);
#else
		return 0;
#endif
	}
};

// Helper that consumes 8 bits from the byte value stream
class FBitCacheReader
{
	uint8 Bits = 0;
	uint8 BitIt = 0;
public:
	[[nodiscard]] FORCEINLINE bool GrabNext(FByteReader& Bytes)
	{
		BitIt <<= 1; // Shift up til overflow

		if (BitIt == 0)
		{
			Bits = Bytes.GrabByte();
			BitIt = 1;
		}

		return !!(Bits & BitIt);
	}

	FORCENOINLINE void Skip(uint32 Num, FByteReader& Bytes)
	{
		uint32 NumCached = 1 + FMath::CountLeadingZeros8(BitIt);

		if (NumCached > Num)
		{
			BitIt <<= Num;
		}
		else
		{
			uint32 NumUncached = Num - NumCached;

			// Grab new bytes, keep the last byte and bit within it
			uint32 NumBytes = Align(NumUncached + 1, 8) / 8;
			Bits = Bytes.GrabBytes(NumBytes)[NumBytes - 1];
			BitIt = 1 << (NumUncached % 8);
		}
	}
};

//////////////////////////////////////////////////////////////////////////

struct FStructSchemaHandle
{
	FStructSchemaId			Id;
	FSchemaBatchId			Batch;

	const FStructSchema&	Resolve() const { return ResolveStructSchema(Batch, Id); }
};

struct FStructView
{
	FStructSchemaHandle		Schema;
	FByteReader				Values;
};

//////////////////////////////////////////////////////////////////////////

union FMemberValue
{
	const uint8*			Ptr;	// From byte stream
	bool					bValue; // From bit cache
};

struct FLeafView
{
	FUnpackedLeafType		Leaf;
	FSchemaBatchId			Batch;
	FOptionalEnumSchemaId	Enum;
	FMemberValue			Value;

	FORCEINLINE bool		AsBool() const		{ return As<bool>(); }
	FORCEINLINE int8		AsS8() const		{ return As<int8>(); }
	FORCEINLINE uint8		AsU8() const		{ return As<uint8>(); }
	FORCEINLINE int16		AsS16() const		{ return As<int16>(); }
	FORCEINLINE uint16		AsU16() const		{ return As<uint16>(); }
	FORCEINLINE int32		AsS32() const		{ return As<int32>(); }
	FORCEINLINE uint32		AsU32() const		{ return As<uint32>(); }
	FORCEINLINE int64		AsS64() const		{ return As<int64>(); }
	FORCEINLINE uint64		AsU64() const		{ return As<uint64>(); }
	FORCEINLINE double		AsDouble() const	{ return As<double>(); }
	FORCEINLINE float		AsFloat() const		{ return As<float>(); }
	FORCEINLINE uint8		AsHex8() const		{ return As<uint8,  {ELeafType::Hex, ELeafWidth::B8}>();  }
	FORCEINLINE uint16		AsHex16() const		{ return As<uint16, {ELeafType::Hex, ELeafWidth::B16}>(); }
	FORCEINLINE uint32		AsHex32() const		{ return As<uint32, {ELeafType::Hex, ELeafWidth::B32}>(); }
	FORCEINLINE uint64		AsHex64() const		{ return As<uint64, {ELeafType::Hex, ELeafWidth::B64}>(); }
	FORCEINLINE char8_t		AsChar8() const		{ return As<char8_t>(); }
	FORCEINLINE char16_t	AsChar16() const	{ return As<char16_t>(); }
	FORCEINLINE char32_t	AsChar32() const	{ return As<char32_t>(); }

	template<Arithmetic T>
	FORCEINLINE T			AsUnderlyingValue() const
	{
		return As<T, FUnpackedLeafType{ELeafType::Enum, ReflectArithmetic<T>.Width}>();
	}
	
	template<typename T, FUnpackedLeafType ExpectedLeaf = ReflectLeaf<T>>
	FORCEINLINE T			As() const
	{
		check(Leaf == ExpectedLeaf);
		return *reinterpret_cast<const T*>(Value.Ptr);
	}

	template<>
	FORCEINLINE bool		As() const
	{
		check(Leaf == ReflectLeaf<bool>);
		return Value.bValue;
	}

};

//////////////////////////////////////////////////////////////////////////

//struct FEnumSchemaFlatView
//{
//	const FEnumSchema& Schema;
//	
//	TOptional<uint64>		ToValue(FMemberId Id) const;
//	TOptional<FMemberId>	ToId(uint64 Value) const;
//	bool					IsSet(FMemberId Id, uint64 Value) const;
//};
//
//struct FEnumSchemaFlagView
//{
//	const FEnumSchema& Schema;
//
//	FEnumSchemaFlagView(const FEnumSchema& InSchema); // @pre sizeof(T) matches Schema.Width and all schema values use single bit
//	
//	TOptional<uint64>						ToFlags(TConstArrayView<FMemberId> Ids) const;
//	TOptional<TConstArrayView<FMemberId>>	ToIds(uint64 Flags) const;
//	bool									HasAny(FMemberId Id, uint64 Flags) const;
//	bool									HasAll(FMemberId Id, uint64 Flags) const;
//};
//
//struct FEnumFlatView
//{
//	const FEnumSchemaFlatView* Schema;
//	uint64 Value;
//
//	bool Equals(FMemberId Id) const;
//};
//
//struct FEnumFlagView
//{
//	const FEnumSchemaFlagView* Schema;
//	uint64 Value;
//};

//////////////////////////////////////////////////////////////////////////

class FMemberReader;
class FNestedRangeIterator;
class FStructRangeIterator;

template<typename IteratorType>
class TStructuralRangeView;

class FLeafRangeView;
using FStructRangeView = TStructuralRangeView<FStructRangeIterator>;
using FNestedRangeView = TStructuralRangeView<FNestedRangeIterator>;

struct FRangeSchema
{
	FMemberType				ItemType;
	FSchemaBatchId			Batch; // Needed to resolve inner schema
	FOptionalSchemaId		InnermostSchema;
	const FMemberType*		NestedItemTypes; // For nested ranges, can be out-of-bounds otherwise
};

class FRangeView
{
public:
	uint64							Num() const				{ return NumItems;}
	bool							IsEmpty() const			{ return NumItems == 0; }
	FMemberType						GetItemType() const		{ return Schema.ItemType; }
	bool							IsLeafRange() const		{ return Schema.ItemType.GetKind() == EMemberKind::Leaf; }
	bool							IsStructRange() const	{ return Schema.ItemType.GetKind() == EMemberKind::Struct; }
	bool							IsNestedRange() const	{ return Schema.ItemType.GetKind() == EMemberKind::Range; }

	PLAINPROPS_API FLeafRangeView	AsLeaves() const;		// @pre IsLeafRange()
	PLAINPROPS_API FStructRangeView	AsStructs() const;		// @pre IsStructRange()
	PLAINPROPS_API FNestedRangeView	AsRanges() const;		// @pre IsNestedRange()

private:
	friend FRangeSchema GetSchema(FRangeView Range);
	friend FMemoryView GetValues(FRangeView Range);
	friend FMemberReader;
	friend FNestedRangeIterator;

	FRangeSchema					Schema;
	uint64							NumItems;
	FMemoryView						Values;
};

//////////////////////////////////////////////////////////////////////////

class FBoolRangeIterator
{
public:
	FBoolRangeIterator(const uint8* Data, uint64 Idx)
	: Byte(Data + Idx / 8)
	, Mask(1u << (Idx % 8))
	{}

	bool operator*() const
	{
		return !!((*Byte) & Mask);
	}

	FBoolRangeIterator& operator++()
	{
		Mask <<= 1;
		if (Mask == 0x100)
		{
			++Byte;
			Mask = 1;
		}
		return *this;
	}

	FBoolRangeIterator operator++(int)
	{
		FBoolRangeIterator Tmp(*this);
		operator++();
		return Tmp;
	}

	bool operator!=(FBoolRangeIterator Rhs)
	{
		return Byte != Rhs.Byte || Mask != Rhs.Mask;
	}

private:
	const uint8* Byte;
	uint32 Mask;
};

class FBoolRangeView
{
public:
	FBoolRangeView(const void* Bits, uint64 Num)
	: Data(static_cast<const uint8*>(Bits))
	, NumBits(Num)
	{}

	inline uint64				Num() const								{ return NumBits;}
	inline bool					operator[](uint64 Idx) const			{ return *FBoolRangeIterator(Data, Idx); }
	inline void					Copy(void* Dst, uint64 NumBytes) const;
	inline FBoolRangeIterator	begin() const							{ return FBoolRangeIterator(Data, 0); }
	inline FBoolRangeIterator	end() const								{ return FBoolRangeIterator(Data, NumBits); }

private:
	const uint8* Data;
	uint64 NumBits;

	friend inline uint64 GetNum(const FBoolRangeView& Range) { return Range.NumBits; }
};

//////////////////////////////////////////////////////////////////////////

template<typename T>
class TRangeView
{
public:
	TRangeView(const T* InData, uint64 InNum)
	: Data(InData)
	, NumItems(InNum)
	{}

	inline uint64			Num() const								{ return NumItems;}
	inline T				operator[](uint64 Idx) const			{ check(Idx < NumItems); return Data[Idx]; }
	inline void				Copy(void* Dst, uint64 NumBytes) const;
	inline const T*			begin() const							{ return Data; }
	inline const T*			end() const								{ return Data + NumItems; }
private:
	const T* Data;
	uint64 NumItems;
	
	friend inline uint64 GetNum(const TRangeView& Range) { return Range.NumItems; }
	friend inline const T* GetData(const TRangeView& Range) { return Range.Data; }
};

//  Works with FBoolRangeView, which lack GetData(), TRangeView, TArrayView, std::intializer_list and T[]
template<class RangeTypeA, class RangeTypeB>
bool EqualItems(RangeTypeA&& A, RangeTypeB&& B)
{
	if (static_cast<uint64>(GetNum(A)) != static_cast<uint64>(GetNum(B)))
	{
		return false;
	}

	auto BIt = std::begin(B);
	for (auto bA : A)
	{
		if (bA != *BIt++)
		{
			return false;
		}
	}

	return true;
}

inline void	FBoolRangeView::Copy(void* Dst, uint64 NumBytes) const
{
	check(NumBytes == NumBits);
	bool* bOut = static_cast<bool*>(Dst);
	for (bool b : *this)
	{
		*bOut++ = b;
	}	
}

template<typename T>
inline void TRangeView<T>::Copy(void* Dst, uint64 NumBytes) const
{
	check(NumBytes == NumItems * sizeof(T));
	if (NumBytes)
	{
		FMemory::Memcpy(Dst, Data, NumBytes);
	}
}

//////////////////////////////////////////////////////////////////////////

class FLeafRangeView
{
public:
	FLeafRangeView(FUnpackedLeafType Leaf, FSchemaBatchId InBatch, FOptionalEnumSchemaId InEnum, uint64 Num, const uint8* Data) 
	: Type(Leaf.Type)
	, Width(Leaf.Width)
	, Batch(InBatch)
	, Enum(InEnum)
	, NumItems(Num)
	, Values(Data)
	{}

	uint64					Num() const			{ return NumItems; }

	// These range views hide the internal representations to enable future format changes, 
	// e.g. store zeroes or 1.0f in some compact fashion or even variable length int encodings
	//
	// They could also provide various conversion helpers
	
	template<typename T, FUnpackedLeafType ExpectedLeaf = ReflectLeaf<T>>
	auto					As() const
	{
		check(FUnpackedLeafType(Type, Width) == ExpectedLeaf);
		return TRangeView<T>(reinterpret_cast<const T*>(Values), NumItems);	
	}

	template<> auto			As<bool>() const	{ return AsBools(); }

	FBoolRangeView			AsBools() const
	{
		check(Type == ELeafType::Bool);
		return FBoolRangeView(Values, NumItems);
	}

	template<Arithmetic T>
	auto					AsUnderlyingValues() const
	{
		return As<T, FUnpackedLeafType{ELeafType::Enum, ReflectArithmetic<T>.Width}>();
	}

	TRangeView< int8>		AsS8s() const		{ return As< int8>(); }
	TRangeView<uint8>		AsU8s() const		{ return As<uint8>(); }
	TRangeView< int16>		AsS16s() const		{ return As< int16>(); }
	TRangeView<uint16>		AsU16s() const		{ return As<uint16>(); }
	TRangeView< int32>		AsS32s() const		{ return As< int32>(); }
	TRangeView<uint32>		AsU32s() const		{ return As<uint32>(); }
	TRangeView< int64>		AsS64s() const		{ return As< int64>(); }
	TRangeView<uint64>		AsU64s() const		{ return As<uint64>(); }
	TRangeView<float>		AsFloats() const	{ return As<float>(); }
	TRangeView<double>		AsDoubles() const	{ return As<double>(); }
	TRangeView<uint8>		AsHex8s() const		{ return As<uint8,  {ELeafType::Hex, ELeafWidth::B8}>();  }
	TRangeView<uint16>		AsHex16s() const	{ return As<uint16, {ELeafType::Hex, ELeafWidth::B16}>(); }
	TRangeView<uint32>		AsHex32s() const	{ return As<uint32, {ELeafType::Hex, ELeafWidth::B32}>(); }
	TRangeView<uint64>		AsHex64s() const	{ return As<uint64, {ELeafType::Hex, ELeafWidth::B64}>(); }
	TRangeView<char8_t>		AsUtf8() const		{ return As<char8_t>(); }
	TRangeView<char16_t>	AsUtf16() const		{ return As<char16_t>(); }
	TRangeView<char32_t>	AsUtf32() const		{ return As<char32_t>(); }

	FLeafRangeLoadView		AsLoadView() const;

private:
	ELeafType						Type;
	ELeafWidth						Width;
	FSchemaBatchId					Batch;
	FOptionalEnumSchemaId			Enum;
	uint64							NumItems;
	const uint8*					Values;

	friend FUnpackedLeafType GetType(FLeafRangeView Range);
	friend const uint8* GetValues(FLeafRangeView Range);
	friend FOptionalEnumSchemaId GetEnumSchemaId(FLeafRangeView Range);
};

//////////////////////////////////////////////////////////////////////////

class FNestedRangeIterator
{
	friend FNestedRangeView;
	FRangeSchema			Schema;
	FByteReader				ByteIt;
	FBitCacheReader			BitIt;

public:
	FNestedRangeIterator(const FRangeSchema& InSchema, FMemoryView Data)
	: Schema(InSchema) 
	, ByteIt(Data)
	{}

	PLAINPROPS_API FRangeView	operator*() const;
	bool						operator!=(const FNestedRangeIterator& Rhs) const { return ByteIt.Peek() != Rhs.ByteIt.Peek(); };
	PLAINPROPS_API void			operator++();
};

class FStructRangeIterator
{
	friend FStructRangeView;
	FStructSchemaHandle		Schema;
	FByteReader				ByteIt;

public:
	FStructRangeIterator(const FStructSchemaHandle& InSchema, FMemoryView Data)
	: Schema(InSchema) 
	, ByteIt(Data)
	{}

	FStructView operator*() const { return { Schema, FByteReader(ByteIt.PeekSkippableSlice()) }; }
	bool operator!=(const FStructRangeIterator& Rhs) const { return ByteIt.Peek() != Rhs.ByteIt.Peek(); };
	void operator++() { (void)ByteIt.GrabSkippableSlice(); }
};

template<typename IteratorType>
class TStructuralRangeView
{
public:
	using SchemaType = decltype(IteratorType::Schema);
	TStructuralRangeView(uint64 N, FMemoryView D, SchemaType S) : NumItems(N), Data(D), Schema(S) {}

	uint64				Num() const			{ return NumItems; }
	const SchemaType&	GetSchema() const	{ return Schema; }
	IteratorType		begin() const		{ return IteratorType(Schema, Data); }
	IteratorType		end() const			{ return IteratorType(Schema, FMemoryView(Data.GetDataEnd(), 0)); }


private:
	uint64				NumItems;
	FMemoryView			Data;
	SchemaType			Schema;
};

//////////////////////////////////////////////////////////////////////////

//class FAnyMemberView
//{
//public:
//	EMemberKind				GetKind() const			{ return Type.Kind; };
//
//	bool					IsLeaf() const			{ return Type.Kind == EMemberKind::Leaf; }
//	bool					IsStruct() const		{ return Type.Kind == EMemberKind::Struct; }
//	bool					IsRange() const			{ return Type.Kind == EMemberKind::Range; }
//
//	FStructView				AsStruct() const;		// @pre bool() && IsStruct()
//	FLeafView				AsLeaf() const;			// @pre bool() && IsLeaf()
//	FRangeView				AsRange() const;		// @pre bool() && IsRange()
//
//private:
//	friend class FMemberReader;
//
//	FMemberType			Type;
//	uint32				BatchId; // To resolve struct schema
//	FMemberValue		Value;
//};

//////////////////////////////////////////////////////////////////////////

// Iterates over struct members
class FMemberReader
{
public:
	explicit FMemberReader(FStructView Struct) : FMemberReader(Struct.Schema.Resolve(), Struct.Values, Struct.Schema.Batch) {}
	PLAINPROPS_API FMemberReader(const FStructSchema& Schema, FByteReader Values, FSchemaBatchId InBatch);

	bool								HasMore() const			{ return MemberIdx < NumMembers; }
	bool								HasSuper() const		{ return UsesSuper && MemberIdx == 0; }		
	
	PLAINPROPS_API FOptionalMemberId	PeekName() const;			// @pre HasMore()
	PLAINPROPS_API FMemberId			PeekNameUnchecked() const;  // @pre HasMore() && !HasSuper()
	PLAINPROPS_API EMemberKind			PeekKind() const;			// @pre HasMore()
	PLAINPROPS_API FMemberType			PeekType() const;			// @pre HasMore()

	PLAINPROPS_API FLeafView			GrabLeaf();					// @pre PeekKind() == EMemberKind::Leaf
	PLAINPROPS_API FRangeView			GrabRange();				// @pre PeekKind() == EMemberKind::Range
	PLAINPROPS_API FStructView			GrabStruct();				// @pre PeekKind() == EMemberKind::Struct
	//FAnyMemberView					GrabAny();					// @pre HasMore()

	PLAINPROPS_API void					SkipMember();				// @pre HasMore()

	// Experimental 
	// @pre Has N more contiguous members of the expected leaf type
	template<Arithmetic T>
	void								GrabLeaves(T* Out, uint32 N);

	// Experimental
	// @pre Has N more contiguous members of the expected leaf type
	template<Enumeration T>
	void								GrabEnums(T* Out, uint32 N);

protected: // for unit tests
	const FMemberType*		Footer;
	const FSchemaBatchId	Batch;					// Needed to resolve schemas
	const bool				IsSparse : 1;
	const bool				UsesSuper : 1;
	const uint32			NumMembers;
	const uint32			NumRangeTypes;			// Number of ranges and nested ranges

	uint32					MemberIdx = 0;
	uint32					RangeTypeIdx = 0;		// Types of [nested] ranges
	uint32					InnerSchemaIdx = 0;		// Types of static structs and enums
	FBitCacheReader			Bits;
	FByteReader				ValueIt;

#if DO_CHECK
	const uint32			NumInnerSchemas;		// Number of static structs and enums
#endif

	const FMemberType*		GetMemberTypes() const;
	const FMemberType*		GetRangeTypes() const;
	const FSchemaId*		GetInnerSchemas() const;
	const FMemberId*		GetMemberNames() const;
	
	void					AdvanceToNextMember();
	void					AdvanceToLaterMember(uint32 Num);
	void					SkipMissingSparseMembers();
	void					SkipSchema(FMemberType InnermostType);

	using FMemberTypeRange = TConstArrayView<FMemberType>;

	FMemberTypeRange		GrabRangeTypes();
	FSchemaId				GrabInnerSchema();
	FStructSchemaId			GrabStructSchema(FStructType Type);
	FOptionalSchemaId		GrabRangeSchema(FMemberType InnermostType);
	inline FEnumSchemaId	GrabEnumSchema()		{ return static_cast<FEnumSchemaId&&>(GrabInnerSchema()); }
	inline bool				GrabBit()				{ return Bits.GrabNext(/* in-out */ ValueIt); }
	inline uint64			GrabSkipLength()		{ return ValueIt.GrabVarIntU(); }

	void					GrabBools(void* Out, uint32 Num);
	void					GrabEnums(void* Out, uint32 Num, SIZE_T NumBytes);
	void					GrabLeaves(void* Out, uint32 Num, SIZE_T NumBytes);
};

template<Arithmetic T>
void FMemberReader::GrabLeaves(T* Out, uint32 N)
{
	if (N)
	{
		if constexpr (std::is_same_v<bool, T>)
		{
			GrabBools(Out, N);
		}
		else
		{
			GrabLeaves(Out, N, sizeof(T));
		}
	}
}

template<Enumeration T>
void FMemberReader::GrabEnums(T* Out, uint32 N)
{
	if (N)
	{
		GrabLeaves(Out, N, sizeof(T));
		InnerSchemaIdx += N;
	}
}

//////////////////////////////////////////////////////////////////////////

// Hides the inheritance chain and iterates over super members first
class FFlatMemberReader
{
public:
	FFlatMemberReader(FStructView Struct);

	bool					HasMore() const			{ return It->HasMore(); }

	FMemberId				PeekName() const		{ return It->PeekName().Get(); }
	EMemberKind				PeekKind() const		{ return It->PeekKind(); }
	FType					PeekOwner() const		{ return It->Owner; }

	FLeafView				GrabLeaf()				{ return Grabbed(It->GrabLeaf()); }
	FRangeView				GrabRange()				{ return Grabbed(It->GrabRange()); }
	FStructView				GrabStruct()			{ return Grabbed(It->GrabStruct()); }

private:
	struct FReader : public FMemberReader
	{
		FReader(FStructView Struct);
		FType Owner;
	};
	TArray<FReader, TInlineAllocator<8>> Lineage;
	FReader* It = nullptr;

	template<class ViewType>
	ViewType Grabbed(ViewType Out)
	{
		It -= (It != &Lineage[0] && !It->HasMore());
		return Out;
	}
};

//////////////////////////////////////////////////////////////////////////

// Serialized id resolver
class FBatchIds : public FIdsBase
{
public:
	using FIdsBase::Resolve;
	using FIdsBase::AppendString;

	PLAINPROPS_API FBatchIds(FSchemaBatchId Batch);

	PLAINPROPS_API virtual uint32	NumEnums() const;
	PLAINPROPS_API virtual uint32	NumStructs() const;
	PLAINPROPS_API virtual FType	Resolve(FEnumSchemaId Id) const;
	PLAINPROPS_API virtual FType	Resolve(FStructSchemaId Id) const;

	PLAINPROPS_API virtual void		AppendString(FUtf8Builder& Out, FEnumSchemaId Enum) const;
	PLAINPROPS_API virtual void		AppendString(FUtf8Builder& Out, FStructSchemaId Struct) const;

	const FSchemaBatch& GetSchemas() const { return Schemas; } 
	FSchemaBatchId GetBatchId() const { return BatchId; } 
protected:
	const FSchemaBatch& Schemas;
	FSchemaBatchId BatchId;
};

// Serialized id resolver for ESchemaFormat::InMemoryNames
class FMemoryBatchIds final : public FBatchIds
{
public:
	using FBatchIds::Resolve;
	using FBatchIds::AppendString;

	FMemoryBatchIds(FSchemaBatchId Batch, const FIdsBase& InNames) : FBatchIds(Batch), Names(InNames) {}

	virtual uint32				NumNames() const override						{ return Names.NumNames(); }
	virtual uint32				NumNestedScopes() const override				{ return Names.NumNestedScopes(); }
	virtual uint32				NumParametricTypes() const override				{ return Names.NumParametricTypes(); }
	virtual FNestedScope		Resolve(FNestedScopeId Id) const override		{ return Names.Resolve(Id); }
	virtual FParametricTypeView	Resolve(FParametricTypeId Id) const override	{ return Names.Resolve(Id); }

	virtual void				AppendString(FUtf8Builder& Out, FNameId Name) const override { Names.AppendString(Out, Name); }

private:
	const FIdsBase& Names;
};

// Serialized id resolver for ESchemaFormat::StableNames
class FStableBatchIds : public FBatchIds
{
public:
	using FBatchIds::FBatchIds;
	using FBatchIds::Resolve;

	PLAINPROPS_API virtual uint32				NumNestedScopes() const override final;
	PLAINPROPS_API virtual uint32				NumParametricTypes() const override final;
	PLAINPROPS_API virtual FNestedScope			Resolve(FNestedScopeId Id) const override final;
	PLAINPROPS_API virtual FParametricTypeView	Resolve(FParametricTypeId Id) const override final;
};

} // namespace PlainProps
