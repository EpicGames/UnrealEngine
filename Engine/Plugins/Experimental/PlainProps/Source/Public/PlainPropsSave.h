// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsBind.h"

namespace PlainProps 
{

struct FBuiltStruct;

struct FSaveContext : FBindContext
{
	FScratchAllocator&			Scratch;
	IDefaultStructs*			Defaults = nullptr;
};

template<typename Runtime>
FSaveContext MakeSaveContext(FScratchAllocator& Scratch)
{
	return { Runtime::GetTypes(), Runtime::GetSchemas(), Runtime::GetCustoms(), Scratch, Runtime::GetDefaults() };
}

////////////////////////////////////////////////////////////////////////////////////////////////

// FBaseline helper
struct FDefaultLink
{
	FDefaultLink(const void* Default) : Ptr(reinterpret_cast<uint64>(Default)) {}
	uint64 Ptr;
};

// A default pointer or a short chain of super default pointers,
// where last default is shared by all remaining supers
//
// Optimized for normal case where default is the most derived type being saved.
// Handles default being some super class, via nullptr(s) first and default last.
// Can also help multiple inheritance impacting super offsets. 
class FBaseline
{
public:
	FBaseline(const void* Default) : Ptr(reinterpret_cast<uint64>(Default)) {}
	PLAINPROPS_API FBaseline(FDefaultLink* Chain UE_LIFETIMEBOUND, int32 Num);

	const void*				Get() const;	// Get default instance for current type
	FBaseline				Super() const;	// Get baseline for super type

private:
	static constexpr uint64 TagBit = uint64(1) << FPlatformMemory::KernelAddressBit;
	uint64 Ptr = 0;
	FBaseline(uint64 In) : Ptr(In) {}
};

////////////////////////////////////////////////////////////////////////////////////////////////

[[nodiscard]] PLAINPROPS_API FBuiltStruct*	SaveStruct(const void* Struct, FBindId BindId, const FSaveContext& Context);
[[nodiscard]] PLAINPROPS_API FBuiltStruct*	SaveStructDelta(const void* Struct, FBaseline Base, FBindId BindId, const FSaveContext& Context);
[[nodiscard]] PLAINPROPS_API FBuiltStruct*	SaveStructDeltaIfDiff(const void* Struct, FBaseline Base, FBindId BindId, const FSaveContext& Context);
[[nodiscard]] PLAINPROPS_API FBuiltRange*	SaveRange(const void* Range, FRangeMemberBinding Member, const FSaveContext& Ctx);
[[nodiscard]] PLAINPROPS_API FBuiltRange*	SaveLeafRange(const void* Range, const ILeafRangeBinding& Binding, FUnpackedLeafType Leaf, const FSaveContext& Ctx);
[[nodiscard]] PLAINPROPS_API FBuiltRange*	SaveStructRange(TConstArrayView<FBuiltStruct*> Structs, const FSaveContext& Ctx);

PLAINPROPS_API void							SaveSuper(FMemberBuilder& Out, const void* Super, FBindId SuperId, const FSaveContext& Ctx);
// Save Super if any members differ from SuperBase
PLAINPROPS_API void							SaveSuperDelta(FMemberBuilder& Out, const void* Super, FBaseline SuperBase, FBindId SuperId, const FSaveContext& Ctx);

////////////////////////////////////////////////////////////////////////////////////////////////

inline const void* FBaseline::Get() const
{
	// AsLink()->Ptr is untagged, only last link is tagged to mark end and Super() resolves it 
    uint64 Out = (Ptr & TagBit) ? reinterpret_cast<const FDefaultLink*>(Ptr & ~TagBit)->Ptr : Ptr;
    return reinterpret_cast<const void*>(Out);
}

inline FBaseline FBaseline::Super() const
{
    if (Ptr & TagBit)
    {
		// Return next link in chain, but resolve the last tagged link to an "inline" archetype
		FDefaultLink SuperLink = reinterpret_cast<const FDefaultLink*>(Ptr & ~TagBit)[1];
		return (SuperLink.Ptr & TagBit) ? FBaseline(SuperLink.Ptr & ~TagBit)
										: FBaseline(Ptr + sizeof(FDefaultLink));
	}

	// Normal case, reuse the same archetype for all super classes
	return *this;
}

} // namespace PlainProps
