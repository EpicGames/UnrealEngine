// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsTypes.h"
#include "PlainPropsBindCtti.h"
#include "PlainPropsBuild.h"
#include "PlainPropsDiff.h"
#include "PlainPropsLoadMember.h"
#include "PlainPropsSave.h"

namespace PlainProps 
{

inline bool Track(bool bDiff, const FBindContext&, FMemberBindType, FMemberId, FDiffMetadata, const void*, const void*)
{
	return bDiff;
}

inline bool Track(bool bDiff, FDiffContext& Ctx, FMemberBindType Type, FMemberId Name, FDiffMetadata Meta, const void* A, const void* B)
{
	if (bDiff)
	{
		Ctx.AddDiff({Type, ToOptional(Name), Meta, A, B, Ctx.LastSibling});
	}
	
	return bDiff;
}

// TMemberSerializer helper
template<typename T>
struct TStructSerializer : FBothStructId
{
	static constexpr EMemberKind Kind = EMemberKind::Struct;

	template<class Ids>
	TStructSerializer(TCustomInit<Ids>& Init, TConstArrayView<FMemberId> Names, FBothStructId Both = GetStructBothId<Ids, T>())
	: FBothStructId(Both)
	{
		Init.RegisterInnerStruct(Both, Names);
	}

	FMemberSpec SpecMember() const { return FMemberSpec(DeclId); }

	FTypedValue SaveMember(const T& Value, const FSaveContext& Ctx) const
	{
		FMemberSchema Schema = { DefaultStructType, DefaultStructType, 0, FInnerId(BindId), nullptr};
		return { Schema, {.Struct = SaveItem(Value, Ctx)} };
	}

	void LoadMember(T& Dst, /* in-out */ FMemberLoader& Src) const
	{
		LoadStruct(&Dst, Src.GrabStruct());
	}

	void ConstructAndLoadMember(void* Dst, /* in-out */ FMemberLoader& Src) const
	{
		if constexpr (std::is_default_constructible_v<T>)
		{
			LoadMember(*new (Dst) T, Src);
		}
		else
		{
			static_assert(!std::is_void_v<CustomBind<T>>, "Non-default constructible types must load via custom bindings");
			ConstructAndLoadStruct(Dst, Src.GrabStruct());
		}
	}
	
	template<class ContextType>
	bool DiffMember(const T& A, const T& B, FMemberId Name, ContextType& Ctx) const
	{
		return Track(DiffStructs(&A, &B, BindId, Ctx), Ctx, DefaultStructBindType, Name, {.Struct = BindId}, &A, &B);
	}

	////////// TRangeSaver API //////////
	
	using BuiltItemType = FBuiltStruct*;

	FBuiltStruct* SaveItem(const T& Value, const FSaveContext& Ctx) const
	{
		return SaveStruct(&Value, BindId, Ctx);
	}

	FMemberSchema MakeMemberRangeSchema(ESizeType MaxSize) const
	{
		return { FMemberType(MaxSize), DefaultStructType, 1, FInnerId(BindId), nullptr };
	}
};

//////////////////////////////////////////////////////////////////////////

template<typename T>
inline constexpr FMemberType ReflectInnermostType()
{
	return DefaultStructType;
}

template<Arithmetic T>
inline constexpr FMemberType ReflectInnermostType()
{
	return ReflectArithmetic<T>.Pack();
}
template<Enumeration T>
inline constexpr FMemberType ReflectInnermostType()
{
	return ReflectEnum<T>.Pack();
}

// TRangeSerializer constructor helper, FMemberBindType isn't default-constructible 
union FMemberBindTypeInitializer
{
	FMemberBindTypeInitializer() : Unused() {}
	uint8_t			Unused;
	FMemberBindType Value;
};

// TMemberSerializer helper
template<typename T, typename RangeBinding>
struct TRangeSerializer
{
	static constexpr EMemberKind	Kind = EMemberKind::Range;
	static constexpr bool			bLeafRange = std::is_base_of_v<ILeafRangeBinding, RangeBinding>;
	static constexpr uint16			NumRanges = CountRangeBindings<RangeBinding>();
	static constexpr uint16			NumTypes = NumRanges + 1;

	FRangeBinding const* const		Bindings = nullptr;
	FOptionalInnerId				InnermostId;
	FOptionalInnerId				InnermostSpecId;
	FMemberBindTypeInitializer		BindTypes[NumTypes];
	FMemberType						Types[NumTypes];

	template<class Ids>
	TRangeSerializer(TCustomInit<Ids>& Init, TConstArrayView<FMemberId> Names)
	: Bindings(GetRangeBindings<RangeBinding, Ids, NumRanges>().GetData())
	{
		for (uint16 Idx = 0; Idx < NumRanges; ++Idx)
		{
			ESizeType MaxSize = Bindings[Idx].GetSizeType();
			Types[Idx] = FMemberType(MaxSize);
			BindTypes[Idx].Value = FMemberBindType(MaxSize);
		}

		using InnermostType = typename TInnermostType<RangeBinding, NumRanges>::Type;
		Types[NumRanges] = ReflectInnermostType<InnermostType>();
		
		FMemberSpec Spec;
		BindTypes[NumRanges].Value = BindInnermostType<InnermostType, Ids>(/* out */ InnermostId, /* out */ Spec);
		InnermostSpecId = InnermostId;

		if constexpr (!LeafType<InnermostType>)
		{			
			FBothStructId Both = {InnermostId.Get().AsStructBindId(), GetStructDeclId<Ids, InnermostType>() };
			InnermostSpecId = FInnerId(Both.DeclId);
			Init.RegisterInnerStruct(Both, Names);
		}
	}

	FMemberSpec SpecMember() const
	{
		return FMemberSpec(Types, InnermostSpecId);
	}

	void LoadMember(T& Dst, /* in-out */ FMemberLoader& Src) const
	{
		LoadRange(&Dst, Src.GrabRange(), MakeArrayView(Bindings, NumRanges));
	}

	void ConstructAndLoadMember(void* Dst, /* in-out */ FMemberLoader& Src) const
	{
		static_assert(std::is_default_constructible_v<T>, TEXT("Ranges must be default-constructible"));
		LoadMember(*new (Dst) T, Src);
	}

	FTypedValue	SaveMember(const T& Value, const FSaveContext& Ctx) const
	{
		return { MakeMemberSchema(), {.Range = SaveItem(Value, Ctx)} };
	}

	template<typename ContextType>
	bool DiffItems(const void* A, const void* B, const IItemRangeBinding& Binding, ContextType& Ctx) const
	{
		if constexpr (Kind == EMemberKind::Leaf)
		{
			return DiffRanges(A, B, Binding, ReflectLeaf<T>);
		}
		else if constexpr (Kind == EMemberKind::Struct)
		{
			return DiffRanges(A, B, Binding, InnermostId.Get().AsStructBindId(), Ctx);
		}
		else
		{
			return DiffRanges(A, B, Binding, MakeInnerRangeBinding(), Ctx);
		}
	}

	inline FDiffMetadata MakeDiffMetadata() const
	{
		if constexpr (Kind == EMemberKind::Leaf)
		{
			return { .Leaf = ToOptionalEnum(InnermostId) };
		}
		else if constexpr (Kind == EMemberKind::Struct)
		{
			return { .Struct = InnermostId.Get().AsStructBindId() };
		}
		else
		{
			return { .Range = Bindings[1] };
		}
	}

	template<typename ContextType>
	bool DiffMember(const T& A, const T& B, FMemberId Name, ContextType& Ctx) const
	{
		bool bDiff = bLeafRange ? Bindings[0].AsLeafBinding().DiffLeaves(&A, &B)
			: DiffItems(&A, &B, Bindings[0].AsItemBinding(), Ctx);
		return Track(bDiff, Ctx, BindTypes[0].Value, Name, MakeDiffMetadata(), &A, &B);
	}
		
	FMemberSchema MakeMemberSchema() const
	{
		return { Types[0], Types[1], NumRanges, InnermostId, NumRanges > 1 ? Types + 1 : nullptr };
	}

	FRangeMemberBinding MakeInnerRangeBinding() const
	{
		return { &BindTypes[1].Value, Bindings, NumRanges, InnermostId, /* offset */ 0 };
	}

	////////// TRangeSaver API //////////

	using BuiltItemType = FBuiltRange*;

	FMemberSchema MakeMemberRangeSchema(ESizeType MaxSize) const
	{
		return { FMemberType(MaxSize), Types[0], NumTypes, InnermostId, Types };
	}

	FBuiltRange* SaveItem(const T& Item, const FSaveContext& Ctx) const
	{
		if constexpr (bLeafRange)
		{
			return SaveLeafRange(&Item, Bindings[0].AsLeafBinding(), Types[1].AsLeaf(), Ctx);
		}
		else
		{
			FRangeMemberBinding MemberBinding = { &BindTypes[1].Value, Bindings, NumRanges, InnermostId, /* offset */ 0 };
			return SaveRange(&Item, MakeInnerRangeBinding(), Ctx);
		}
	}
};

//////////////////////////////////////////////////////////////////////////

// TMemberSerializer helper
template<typename T>
struct TLeafSerializer
{
	static void LoadMember(T& Dst, /* in-out */ FMemberLoader& Src)
	{
		Dst = Src.GrabLeaf().As<T>();
	}

	static void ConstructAndLoadMember(void* Dst, /* in-out */ FMemberLoader& Src)
	{
		LoadMember(*static_cast<T*>(Dst), Src);
	}

	static FTypedValue SaveLeaf(FMemberType Type, FOptionalInnerId Id, T Value)
	{
		return { {Type, Type, 0, Id, nullptr}, {.Leaf = ValueCast(Value)} };
	}

	////////// TRangeSaver API //////////
	
	using BuiltItemType = T;

	static T SaveItem(T Value, const FSaveContext&) { return Value;}
};

// TMemberSerializer helper
template<Arithmetic T>
struct TArithmeticSerializer : TLeafSerializer<T>
{
	static constexpr FMemberType MemberType = ReflectArithmetic<T>.Pack();
	static constexpr FMemberBindType MemberBindType{ReflectArithmetic<T>};

	template<class Ids>
	TArithmeticSerializer(TCustomInit<Ids>&, TConstArrayView<FMemberId>)
	{}

	static FMemberSpec SpecMember()
	{
		return Specify<T>();
	}

	static FTypedValue SaveMember(T Value, const FSaveContext&)
	{
		return TLeafSerializer<T>::SaveLeaf(MemberType, NoId, Value);
	}

	template<class ContextType>
	inline static bool DiffMember(const T& A, const T& B, FMemberId Name, ContextType& Ctx)
	{
		return Track(A != B, Ctx, MemberBindType, Name, {.Leaf = NoId}, &A, &B);
	}

	////////// TRangeSaver API //////////

	FMemberSchema MakeMemberRangeSchema(ESizeType MaxSize) const
	{
		return { FMemberType(MaxSize), MemberType, 1, NoId, nullptr };
	}
};

// TMemberSerializer helper
template<Enumeration T>
struct TEnumSerializer : TLeafSerializer<T>
{
	static constexpr FMemberType MemberType = ReflectEnum<T>.Pack();
	static constexpr FMemberBindType MemberBindType = ReflectEnum<T>;
	
	const FEnumId Id;

	template<class Ids>
	TEnumSerializer(TCustomInit<Ids>&, TConstArrayView<FMemberId>)
	: Id(GetEnumId<Ids, T>())
	{}

	FMemberSpec SpecMember() const
	{
		return Specify<T>(Id);
	}
	
	FTypedValue SaveMember(T Value, const FSaveContext& Ctx) const
	{
		return TLeafSerializer<T>::SaveLeaf(MemberType, Id, Value);
	}

	template<class ContextType>
	inline bool DiffMember(const T& A, const T& B, FMemberId Name, ContextType& Ctx) const
	{
		return Track(A != B, Ctx, MemberBindType, Name, {.Leaf = Id}, &A, &B);
	}

	////////// TRangeSaver API //////////

	FMemberSchema MakeMemberRangeSchema(ESizeType MaxSize) const
	{
		return { FMemberType(MaxSize), MemberType, 1, FInnerId(Id), nullptr };
	}
};

//////////////////////////////////////////////////////////////////////////

template<typename T>
struct TSelectSerializer
{
	using CustomBinding = CustomBind<T>;
	using RangeBinding = RangeBind<T>;
	static constexpr bool bRange = std::is_void_v<CustomBinding> && !std::is_void_v<RangeBinding>;
	using Type = std::conditional_t<bRange, TRangeSerializer<T, RangeBinding>, TStructSerializer<T>>;
};

template<Arithmetic T>
struct TSelectSerializer<T>
{
	using Type = TArithmeticSerializer<T>;
};

template<Enumeration T>
struct TSelectSerializer<T>
{
	using Type = TEnumSerializer<T>;
};

// Helps templated custom bindings save generic members
template<typename T>
using TMemberSerializer = typename TSelectSerializer<T>::Type;

//////////////////////////////////////////////////////////////////////////

// Helps hide FBuiltRange internals
class FRangeSaverBase
{
public:
	PLAINPROPS_API FRangeSaverBase(FScratchAllocator& Scratch, uint64 Num, SIZE_T ItemSize);

protected:
	template<typename BuiltItemType>
	inline void AddBuiltItem(BuiltItemType Item)
	{
#if DO_CHECK
		check(It + sizeof(Item) <= End);
#endif
		reinterpret_cast<BuiltItemType&>(*It) = Item;
		It += sizeof(Item);
	}

	[[nodiscard]] FTypedRange Finalize(FMemberSchema RangeSchema)
	{
#if DO_CHECK
		check(It == End);
#endif
		return { RangeSchema, Range };
	}

	FBuiltRange*				Range;
	uint8*						It;
#if DO_CHECK
	const uint8*				End;
#endif
};

// Saves a range of T without a range binding
template<typename T>
class TRangeSaver : public FRangeSaverBase
{
public:
	using FMemberSerializer = TMemberSerializer<T>;
	using BuiltItemType = typename FMemberSerializer::BuiltItemType;
	static constexpr SIZE_T ItemSize = sizeof(BuiltItemType);

	TRangeSaver(const FSaveContext& InCtx, uint64 Num, const FMemberSerializer& InSchema)
	: FRangeSaverBase(InCtx.Scratch, Num, sizeof(BuiltItemType))
	, Schema(InSchema)
	, Ctx(InCtx)
	{}

	void AddItem(const T& Item)
	{
		FRangeSaverBase::AddBuiltItem(Schema.SaveItem(Item, Ctx));
	}

	[[nodiscard]] FTypedRange Finalize(ESizeType MaxSize)
	{
		return FRangeSaverBase::Finalize(Schema.MakeMemberRangeSchema(MaxSize));
	}

private:	
	const FMemberSerializer&	Schema;
	const FSaveContext&			Ctx;
};

template<Arithmetic LeafType>
class TLeafRangeSaver : public FRangeSaverBase
{
public:
	TLeafRangeSaver(FScratchAllocator& Scratch, uint64 Num) : FRangeSaverBase(Scratch, Num, sizeof(LeafType)) {}
	inline void AddItem(LeafType Item) {FRangeSaverBase::AddBuiltItem(Item); }
	using FRangeSaverBase::Finalize;
};

class FNestedRangeSaver : public FRangeSaverBase
{
public:
	FNestedRangeSaver(FScratchAllocator& Scratch, uint64 Num) : FRangeSaverBase(Scratch, Num, sizeof(FBuiltRange*)) {}
	inline void AddItem(const FBuiltRange* Item) {	FRangeSaverBase::AddBuiltItem(Item); }
	using FRangeSaverBase::Finalize;
};

class FStructRangeSaver : public FRangeSaverBase
{
public:
	FStructRangeSaver(FScratchAllocator& Scratch, uint64 Num) : FRangeSaverBase(Scratch, Num, sizeof(FBuiltStruct*)) {}
	inline void AddItem(const FBuiltStruct* Item) {	FRangeSaverBase::AddBuiltItem(Item); }
	using FRangeSaverBase::Finalize;
};

//////////////////////////////////////////////////////////////////////////

PLAINPROPS_API void							SaveMember(FMemberBuilder& Out, const void* Struct, FMemberId Name, const FSaveContext& Ctx, FLeafMemberBinding Member);
PLAINPROPS_API void							SaveMember(FMemberBuilder& Out, const void* Struct, FMemberId Name, const FSaveContext& Ctx, FRangeMemberBinding Member);
PLAINPROPS_API void							SaveMember(FMemberBuilder& Out, const void* Struct, FMemberId Name, const FSaveContext& Ctx, FStructMemberBinding Member);

//////////////////////////////////////////////////////////////////////////

[[nodiscard]] PLAINPROPS_API FMemberSchema	CreateRangeSchema(FScratchAllocator& Scratch, FRangeMemberBinding Member);

} // namespace PlainProps
