// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "PlainPropsDeclare.h"
#include "PlainPropsTypes.h"

namespace PlainProps 
{

enum ESpecBool { SpecBool };
enum ESpecIntS { SpecS8, SpecS16, SpecS32, SpecS64 };			// Values match ELeafWidth
enum ESpecIntU { SpecU8, SpecU16, SpecU32, SpecU64 };			// Values match ELeafWidth
enum ESpecHex { SpecHex8, SpecHex16, SpecHex32, SpecHex64 };	// Values match ELeafWidth
enum ESpecChar { SpecUtf8, SpecUtf16, SpecUtf32 };				// Values match ELeafWidth
enum ESpecFloat { SpecF32 = 2, SpecF64 = 3 };					// Values match ELeafWidth
enum ESpecDynamicStruct { SpecDynamicStruct };

struct FMemberSpecView
{
	FMemberType						InnermostType;
	FOptionalInnerId				InnermostId;
	TConstArrayView<ESizeType>		Ranges;
};

// Specifies the type of a member before declaring it
class FMemberSpec
{
public:
	FMemberSpec() : bSpecified(false) {}

	// @pre !Type.IsRange()
	FMemberSpec(FMemberType Type, FOptionalInnerId Id) : InnermostType(Type), InnermostId(Id) { check(!Type.IsRange());	}

	FMemberSpec(FUnpackedLeafType Leaf, FOptionalEnumId Id) : FMemberSpec(FMemberType(Leaf.Pack()), FOptionalInnerId(Id)) {}
	FMemberSpec(ELeafType Type, ELeafWidth Width, FOptionalEnumId Id) : FMemberSpec({Type, Width}, Id) {}
	FMemberSpec(ESpecBool)		: FMemberSpec(ELeafType::Bool,		ELeafWidth::B8,	NoId) {}
	FMemberSpec(ESpecIntS In)	: FMemberSpec(ELeafType::IntS,		static_cast<ELeafWidth>(In), NoId) {}
	FMemberSpec(ESpecIntU In)	: FMemberSpec(ELeafType::IntU,		static_cast<ELeafWidth>(In), NoId) {}
	FMemberSpec(ESpecHex In)	: FMemberSpec(ELeafType::Hex,		static_cast<ELeafWidth>(In), NoId) {}
	FMemberSpec(ESpecChar In)	: FMemberSpec(ELeafType::Unicode,	static_cast<ELeafWidth>(In), NoId) {}
	FMemberSpec(ESpecFloat In)	: FMemberSpec(ELeafType::Float,		static_cast<ELeafWidth>(In), NoId) {}
	
	template<typename T> requires (std::is_convertible_v<T, FDeclId>)
	FMemberSpec(T Id)				: FMemberSpec(FStructType{EMemberKind::Struct, /* IsDynamic */ 0, /* IsSuper */ 0}, FInnerId(FDeclId(Id))) {}
	FMemberSpec(ESpecDynamicStruct) : FMemberSpec(FStructType{EMemberKind::Struct, /* IsDynamic */ 1, /* IsSuper */ 0}, NoId) {}

	PLAINPROPS_API FMemberSpec(ESizeType MaxSize, FMemberSpec Inner);
	PLAINPROPS_API FMemberSpec(TConstArrayView<FMemberType> Members, FOptionalInnerId InInnermostId);
	PLAINPROPS_API FMemberSpec(FMemberType Type, TConstArrayView<FMemberType> InnerRangeTypes, FOptionalInnerId InInnermostId);
	
	void RangeWrap(ESizeType MaxSize)		{ Ranges.Insert(MaxSize, 0); }
	void RangeWrap(FMemberType Range)		{ Ranges.Insert(Range.AsRange().MaxSize, 0); }

	operator FMemberSpecView() const UE_LIFETIMEBOUND;
	explicit operator bool() const { return bSpecified; }
	
private:
	using RangeSizeArray = TArray<ESizeType, TInlineAllocator<8>>;
	
	bool							bSpecified = true;						
	FMemberType						InnermostType;
	FOptionalInnerId				InnermostId;
	RangeSizeArray					Ranges;

	FMemberSpec(FStructType Type, FOptionalInnerId Id) : InnermostType(Type), InnermostId(Id) {}
};

inline FMemberSpec::operator FMemberSpecView() const
{
	checkf(bSpecified, TEXT("Bug, unspecified members should fail earlier checks."));
	return { InnermostType, InnermostId, Ranges };
}

////////////////////////////////////////////////////////////////////////////////////////////////

template<Arithmetic T>
inline FMemberSpec Specify()
{
	return FMemberSpec(ReflectArithmetic<T>, NoId);
}

template<Enumeration T>
inline FMemberSpec Specify(FEnumId Id)
{
	return FMemberSpec(ReflectEnum<T>, Id);
}

////////////////////////////////////////////////////////////////////////////////////////////////

// Struct declaration input
struct FStructSpec
{
	FDeclId									Id;
	FOptionalDeclId							Super;
	uint16									Version = 0;
	EMemberPresence							Occupancy;
	TConstArrayView<FMemberId>				MemberNames;
	TConstArrayView<FMemberSpec>			MemberTypes;
};

////////////////////////////////////////////////////////////////////////////////////////////////

// Helps specify single member structs
struct FSoleMemberSpec
{
	FSoleMemberSpec(FDeclId InId, FMemberId Name, FMemberSpec Spec, FOptionalDeclId InSuper = NoId, uint16 Ver = 0)
	: Id(InId), Super(InSuper), Version(Ver), MemberName(Name), MemberType(Spec)
	{}

	FDeclId									Id;
	FOptionalDeclId							Super;
	uint16									Version;
	FMemberId								MemberName;
	FMemberSpec								MemberType;

	operator FStructSpec() const UE_LIFETIMEBOUND
	{
		return {Id, Super, Version, EMemberPresence::RequireAll, MakeArrayView(&MemberName, 1), MakeArrayView(&MemberType, 1) };
	}
};


} // namespace PlainProps
