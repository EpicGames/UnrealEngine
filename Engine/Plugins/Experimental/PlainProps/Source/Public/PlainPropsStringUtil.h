// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <string_view>
#include <array>
#include "HAL/Platform.h" // uint32 etc
#include "Misc/AsciiSet.h"

namespace PlainProps::Private
{

inline constexpr void Append(char*& ToIt, std::string_view From)
{
    for (char Char : From)
    {
        *ToIt++ = Char;
    }
}

template <const std::string_view&... Strs>
struct TConcat
{
    inline static constexpr std::size_t Len = (Strs.size() + ... + 0);

    static constexpr std::array<char, Len + 1> Chars = []()
    {    
        std::array<char, Len + 1> Out;
        char* OutIt = Out.data();
        (Append(OutIt, Strs), ...);
        *OutIt = '\0';
        return Out;
    }();

    inline static constexpr std::string_view Value{Chars.data(), Len};
};

template<uint64 N>
struct THexString
{
	inline static constexpr char At(uint32 Idx)
	{
		uint64 Nibble =	(N >> 4 * Idx) & uint64(15);
		return static_cast<char>(Nibble > 9 ? Nibble - 10 + 'A' : Nibble + '0'); 
	}

	static constexpr char Chars[16] = { At(15), At(14), At(13), At(12), At(11), At(10), At(9), At(8), At(7), At(6), At(5), At(4), At(3), At(2), At(1), At(0) };

	inline static constexpr uint32 CalcLen()
	{
		uint32 Idx = 15;
		for (; Idx && (N >> (Idx * 4)) == 0; --Idx) {}
		return Idx + 1;
	}

	inline static constexpr uint32 Len = CalcLen();
	inline static constexpr std::string_view Value{Chars + 16 - Len, Len};
};

inline constexpr std::string_view TrimWhitespace(std::string_view Str)
{
	constexpr FAsciiSet Whitespace(" \v\f\t\r\n");
	while (!Str.empty() && Whitespace.Contains(Str.front())) Str.remove_prefix(1);
	while (!Str.empty() && Whitespace.Contains(Str.back())) Str.remove_suffix(1);
	return Str;
}

// Returns number of delimiter separated tokens, includes empty tokens.
// Returns zero for empty or whitespace only Str.
inline constexpr std::size_t CountTokens(std::string_view Str, char Delimiter)
{
	std::size_t Num = 1;
	for (char Char : Str) Num += (Char == Delimiter);
	return Num > 1 ? Num : !TrimWhitespace(Str).empty();
}

// Returns array of delimiter separated and whitespace stripped tokens, includes empty tokens.
// Returns empty array for empty or whitespace only Str.
template <std::size_t N>
inline constexpr std::array<std::string_view, N> ParseTokensImpl(std::string_view Str, char Delimiter)
{
	std::array<std::string_view, N> Out{};
	std::size_t Idx = 0;
	while (Idx < N)
	{
		std::size_t Pos = Str.find(Delimiter);
		Out[Idx++] = TrimWhitespace(Str.substr(0, Pos));
		if (Pos == std::string_view::npos) break;
		Str.remove_prefix(Pos + 1);
	}
	return Out;
}

// Returns array of comma separated and whitespace stripped tokens, includes empty tokens.
// Returns empty array for empty or whitespace only Str.
// Can be used for a string view based on #__VA__ARGS.
template <const std::string_view& Str>
inline constexpr auto ParseArgs = ParseTokensImpl<Private::CountTokens(Str, ',')>(Str, ',');

} // namespace PlainProps::Private

namespace PlainProps
{

template <const std::string_view&... Strs>
inline constexpr std::string_view Concat = Private::TConcat<Strs...>::Value;

template <uint64 N>
inline constexpr std::string_view HexString = Private::THexString<N>::Value;

} // namespace PlainProps
