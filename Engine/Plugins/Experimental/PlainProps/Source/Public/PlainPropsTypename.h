// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsCtti.h"
#include "PlainPropsTypes.h"
#include <string_view>

namespace PlainProps 
{

// A declared struct can be bound to multiple runtime implementations. ETypename indicates which typename is intended.
//
// This enum is also used to type-erase ranges, e.g. in UE FString, TArray<char8_t>, TArray<char, TInlineAllocator<64>> 
// all map to a UTF8 range and the runtime can change data structures without impacting saved format.
//
// This can be used for both schema bindings and custom bindings. E.g. weak and strong reference class can be saved identically
// if the strong / weak semantics is considered a runtime detail. Typed references can be type-erased into an untyped reference.
//
// When there is only a single runtime memory representation the Bind name and Decl name is usually the same
enum class ETypename
{
	Decl,	// Declared type name, saved to disk / schemas
	Bind	// Bound type name, in-memory id that uniquely identifies a binding. Often same as Decl.
};

// Trait that defines DeclName, BindName and template Parameters that's part of the typename
//
// Also defines RangeBindName, used when otherwise nameless ranges are captured in Parameters, 
// e.g. UE TPair<int, FString> need a different bind name than TPair<int, TArray<char8_t>>
//
// At least one of DeclName, BindName or RangeBindName should exist and be a constexpr std::string_view
// Template parameters are captured as using Parameters = std::tuple<T>;
template<typename T>
struct TTypename
{
	inline static constexpr std::string_view DeclName = CttiOf<T>::Name;
	inline static constexpr std::string_view Namespace = CttiOf<T>::Namespace;
};

template<class Typename>
concept ParametricName = requires { typename Typename::Parameters; };

template<class Typename>
concept ExplicitBindName = requires { Typename::BindName; };

////////////////////////////////////////////////////////////////////////////////////////////////

// WIP type trait used to type-erase range bind names, might get folded into TTypename
template<typename T>
struct TShortTypename;

template<typename T>
constexpr std::string_view ShortTypename = TShortTypename<T>::Value;

struct FOmitTypename { inline static constexpr std::string_view Value{}; };

////////////////////////////////////////////////////////////////////////////////////////////////

template<ELeafType Type, ELeafWidth Width>
constexpr std::string_view IllegalArithmetic()
{
	static_assert((int)Type + (int)Width == -1, "Illegal ELeafType/ELeafWidth combination");
	return "ERR";
}

template<ELeafType Type, ELeafWidth Width>
constexpr std::string_view ArithmeticName = IllegalArithmetic<Type, Width>();

template<> inline constexpr std::string_view ArithmeticName<ELeafType::Bool, ELeafWidth::B8>		= "bool";
template<> inline constexpr std::string_view ArithmeticName<ELeafType::IntS, ELeafWidth::B8>		= "i8";
template<> inline constexpr std::string_view ArithmeticName<ELeafType::IntS, ELeafWidth::B16>		= "i16";
template<> inline constexpr std::string_view ArithmeticName<ELeafType::IntS, ELeafWidth::B32>		= "i32";
template<> inline constexpr std::string_view ArithmeticName<ELeafType::IntS, ELeafWidth::B64>		= "i64";
template<> inline constexpr std::string_view ArithmeticName<ELeafType::IntU, ELeafWidth::B8>		= "u8";
template<> inline constexpr std::string_view ArithmeticName<ELeafType::IntU, ELeafWidth::B16>		= "u16";
template<> inline constexpr std::string_view ArithmeticName<ELeafType::IntU, ELeafWidth::B32>		= "u32";
template<> inline constexpr std::string_view ArithmeticName<ELeafType::IntU, ELeafWidth::B64>		= "u64";
template<> inline constexpr std::string_view ArithmeticName<ELeafType::Float, ELeafWidth::B32>		= "f32";
template<> inline constexpr std::string_view ArithmeticName<ELeafType::Float, ELeafWidth::B64>		= "f64";
template<> inline constexpr std::string_view ArithmeticName<ELeafType::Hex, ELeafWidth::B8>			= "hex8";
template<> inline constexpr std::string_view ArithmeticName<ELeafType::Hex, ELeafWidth::B16>		= "hex16";
template<> inline constexpr std::string_view ArithmeticName<ELeafType::Hex, ELeafWidth::B32>		= "hex32";
template<> inline constexpr std::string_view ArithmeticName<ELeafType::Hex, ELeafWidth::B64>		= "hex64";
template<> inline constexpr std::string_view ArithmeticName<ELeafType::Unicode, ELeafWidth::B8>		= "utf8";
template<> inline constexpr std::string_view ArithmeticName<ELeafType::Unicode, ELeafWidth::B16>	= "utf16";
template<> inline constexpr std::string_view ArithmeticName<ELeafType::Unicode, ELeafWidth::B32>	= "utf32";

template<Arithmetic T>
struct TTypename<T>
{
	inline static constexpr FUnpackedLeafType Leaf = ReflectArithmetic<T>;
	inline static constexpr std::string_view DeclName = ArithmeticName<Leaf.Type, Leaf.Width>;
	inline static constexpr std::string_view Namespace;
};

//////////////////////////////////////////////////////////////////////////

template<class CustomBinding>
concept WithCustomTypename = requires { typename CustomBinding::FCustomTypename; };

template<class CustomBinding>
struct TCustomTypename
{
	using Type = TTypename<typename CustomBinding::Type>;
};

template<WithCustomTypename CustomBinding>
struct TCustomTypename<CustomBinding>
{
	using Type = CustomBinding::FCustomTypename;
};

} // namespace PlainProps