// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/StringFwd.h"
#include "Misc/AssertionMacros.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Templates/TypeHash.h"
#include <type_traits>

namespace PlainProps
{

enum class EMemberKind : uint8 { Leaf, Struct, Range };
enum class ELeafType : uint8 { Bool, IntS, IntU, Float, Hex, Enum, Unicode };
enum class ELeafWidth : uint8 { B8, B16, B32, B64 };
enum class ESizeType : uint8 { Uni, S8, U8, S16, U16, S32, U32, S64, U64 };

inline constexpr SIZE_T SizeOf(ELeafWidth Width)
{
	return SIZE_T(1) << static_cast<uint32>(Width);
}

inline ELeafWidth WidthOf(SIZE_T Size)
{
	check(Size == 1 || Size == 2 || Size == 4 || Size == 8);
	return static_cast<ELeafWidth>(FMath::FloorLog2NonZero_64(Size));
}

inline SIZE_T SizeOf(ESizeType Width)
{
	check(Width != ESizeType::Uni);
    return SIZE_T(1) << ((uint8)Width - 1) / 2; 
}

inline constexpr uint64 Max(ESizeType Width)
{
    const uint8_t LeadingZeroes[] = {63, 57, 56, 49, 48, 33, 32, 1, 0};
    return ~uint64_t(0) >> LeadingZeroes[(uint8)Width]; 
}

template<SIZE_T Size>
constexpr ELeafWidth IllegalLeafWidth()
{
	static_assert(Size == 1);
	return ELeafWidth::B8;
}

template<SIZE_T Size> constexpr ELeafWidth LeafWidth = IllegalLeafWidth<Size>();
template<> inline constexpr ELeafWidth LeafWidth<1> = ELeafWidth::B8;
template<> inline constexpr ELeafWidth LeafWidth<2> = ELeafWidth::B16;
template<> inline constexpr ELeafWidth LeafWidth<4> = ELeafWidth::B32;
template<> inline constexpr ELeafWidth LeafWidth<8> = ELeafWidth::B64;

//////////////////////////////////////////////////////////////////////////

struct FLeafType
{
	EMemberKind		_ : 2;
	ELeafWidth		Width : 2;
	ELeafType		Type : 3;
	uint8			_Pad : 1;
};

struct FRangeType
{
	EMemberKind		_ : 2;
	ESizeType		MaxSize : 4;
	uint8			_Pad : 2;
};

struct FStructType
{
    EMemberKind		_ : 2;
	uint8			IsDynamic : 1;	// Which schema stored in value stream, different instances in same batch have different types
	uint8			IsSuper : 1;	// Inherited members from some super struct, can only appear as first member
//	uint8			IsDense : 1;	// Potential optimization to avoid accessing schema
	uint8			_Pad : 4;
};

union FMemberType
{
	FMemberType() = default; // Uninitialized
	constexpr explicit FMemberType(FLeafType InLeaf) : Leaf(InLeaf) {}
	constexpr explicit FMemberType(ELeafType Type, ELeafWidth Width) : Leaf({EMemberKind::Leaf, Width, Type}) {}
	constexpr explicit FMemberType(FRangeType InRange) : Range(InRange) {}
	constexpr explicit FMemberType(ESizeType MaxSize) : Range({EMemberKind::Range, MaxSize}) {}
	constexpr explicit FMemberType(FStructType InStruct) : Struct(InStruct) {}

	bool			IsLeaf() const		{ return Kind == EMemberKind::Leaf; }
	bool			IsRange() const		{ return Kind == EMemberKind::Range; }
	bool			IsStruct() const	{ return Kind == EMemberKind::Struct; }
	EMemberKind     GetKind() const		{ return Kind; }

	FLeafType		AsLeaf() const		{ check(IsLeaf());		return Leaf; }
	FRangeType		AsRange() const		{ check(IsRange());		return Range; }
	FStructType		AsStruct() const	{ check(IsStruct());	return Struct; }
	uint8			AsByte() const		{ return reinterpret_cast<const uint8&>(*this); }

	friend inline bool operator==(FMemberType A, FMemberType B) { return A.AsByte() == B.AsByte(); }
private:
    EMemberKind     Kind : 2;
    FLeafType	    Leaf;
    FRangeType		Range;
    FStructType		Struct;
};

//////////////////////////////////////////////////////////////////////////

struct FNameId
{
	uint32 Idx = ~0u;

	bool operator==(FNameId O) const { return Idx == O.Idx; }
	friend uint32 GetTypeHash(FNameId Id) { return Id.Idx; };
};

struct FMemberId
{
	FNameId Id;

	bool operator==(FMemberId O) const { return Id == O.Id; }
	friend uint32 GetTypeHash(FMemberId Member) { return Member.Id.Idx; };
};

//////////////////////////////////////////////////////////////////////////

// Runtime id for an enum FType
struct FEnumId
{
	uint32 Idx = ~0u;
	friend uint32 GetTypeHash(FEnumId In) { return GetTypeHash(In.Idx); };
	friend bool operator==(FEnumId, FEnumId) = default;
};

// Runtime id for a struct FType, i.e. a class or struct
struct FStructId
{
	uint32 Idx = ~0u;
	friend uint32 GetTypeHash(FStructId In) { return GetTypeHash(In.Idx); };
	friend bool operator==(FStructId, FStructId) = default;
};

// Abstract FStructId used in declarations and stable schemas
//
// Might type-erase runtime details such as container allocator
struct FDeclId : FStructId {};

// Concrete FStructId used in bindings and in-memory schemas
//
// Uniquely ids a runtime class/struct. Usually same as FDeclId, 
// but might type-erase runtime details.
struct FBindId : FStructId {};

// Static FBindId -> FDeclId cast once you've checked the bind id isnt type-erased
inline FDeclId LowerCast(FBindId Id) { return static_cast<FDeclId>(FStructId(Id)); }
inline FBindId UpCast(FDeclId Id) { return static_cast<FBindId>(FStructId(Id)); }

// Either a runtime FStructId or FEnumId of a member
struct FInnerId
{
	uint32 Idx = ~0u;

	explicit FInnerId(FEnumId In) : Idx(In.Idx) {}
	explicit FInnerId(FStructId In) : Idx(In.Idx) {}
	
	FEnumId		AsEnum() const				{ return {Idx}; }
	FStructId	AsStruct() const			{ return {Idx}; }
	FDeclId		AsStructDeclId() const		{ return {Idx}; }
	FBindId		AsStructBindId() const		{ return {Idx}; }

	friend uint32 GetTypeHash(FInnerId In)	{ return GetTypeHash(In.Idx); };
	friend bool operator==(FInnerId, FInnerId) = default;
	explicit FInnerId(uint32 InIdx) : Idx(InIdx) {}
};

//////////////////////////////////////////////////////////////////////////

// Serialized schema id of a runtime FInnerId
struct FSchemaId
{
	uint32 Idx = ~0u;
	
	bool operator==(FSchemaId O) const { return Idx == O.Idx; }
	friend uint32 GetTypeHash(FSchemaId Schema) { return GetTypeHash(Schema.Idx); };
};

// Serialized struct id of a runtime FStructId
struct FStructSchemaId : FSchemaId {};
static void KeepDebugInfo(FStructSchemaId*) {};

// Serialized enum id of a runtime FEnumId
struct FEnumSchemaId : FSchemaId {};
static void KeepDebugInfo(FEnumSchemaId*) {};

//////////////////////////////////////////////////////////////////////////

struct FNoId {};
inline constexpr FNoId NoId;

//////////////////////////////////////////////////////////////////////////

struct FNestedScopeId { uint32 Idx; };
struct FFlatScopeId { FNameId Name; };

class FScopeId
{
	static constexpr uint32 NestedBit = 0x80000000u;
	static constexpr uint32 Unscoped = ~0u;

	uint32 Handle;
public:
	FScopeId(FNoId) : Handle(Unscoped) {}
	explicit FScopeId(FFlatScopeId Flat) : Handle(Flat.Name.Idx) 				{ check(AsFlat().Name == Flat.Name); }
	explicit FScopeId(FNestedScopeId Nested) : Handle(Nested.Idx | NestedBit) 	{ check(AsNested().Idx == Nested.Idx); }

	explicit						operator bool() const			{ return Handle != ~0u; }
	bool							IsFlat() const 					{ return !(Handle & NestedBit); }
	bool							IsNested() const 				{ return !!(*this) & !!(Handle & NestedBit); }
	FFlatScopeId					AsFlat() const 					{ check(IsFlat()); 		return {FNameId{Handle}}; }
	FNestedScopeId					AsNested() const				{ check(IsNested()); 	return {Handle & ~NestedBit}; }
	uint32							AsInt() const					{ return Handle; }
	bool							operator==(FScopeId O) const	{ return Handle == O.Handle; }
};

//////////////////////////////////////////////////////////////////////////

struct FConcreteTypenameId
{
	FNameId Id;
	bool operator==(FConcreteTypenameId O) const { return Id == O.Id; }
};

struct FBaseTypenameId
{
	FBaseTypenameId(uint8 InNumParameters, uint32 InIdx)
	: NumParameters(InNumParameters)
	, Idx(InIdx)
	{
		check(Idx == InIdx);
	}

	uint32 NumParameters : 8;
	uint32 Idx : 24;

	uint32 AsInt() const { return (Idx << 8) + NumParameters; }
	static FBaseTypenameId FromInt(uint32 Int) { return FBaseTypenameId(static_cast<uint8>(Int), Int >> 8); }
	bool operator==(FBaseTypenameId O) const { return AsInt() == O.AsInt(); }
};

struct FParametricTypeId : FBaseTypenameId
{
	using FBaseTypenameId::FBaseTypenameId;
	static FParametricTypeId FromInt(uint32 Int) { return static_cast<FParametricTypeId&&>(FBaseTypenameId::FromInt(Int)); }
};

struct FTypenameId : FBaseTypenameId
{
	explicit FTypenameId(FParametricTypeId Parametric) : FBaseTypenameId(Parametric) { check(AsParametric().AsInt() == Parametric.AsInt()); }
	explicit FTypenameId(FConcreteTypenameId Concrete) : FBaseTypenameId(0, Concrete.Id.Idx) { check(AsConcrete().Id == Concrete.Id); }

	bool							IsConcrete() const 		{ return NumParameters == 0; }
	bool							IsParametric() const 	{ return !IsConcrete(); }
	FConcreteTypenameId				AsConcrete() const 		{ check(IsConcrete());		return { FNameId{Idx} };  }
	FParametricTypeId				AsParametric() const 	{ check(IsParametric());	return FParametricTypeId(NumParameters, Idx); }
};

//////////////////////////////////////////////////////////////////////////

struct FType
{
	FScopeId 		Scope;
	FTypenameId		Name;

	bool operator==(FType O) const { return Scope.AsInt() == O.Scope.AsInt() && Name.AsInt() == O.Name.AsInt(); }
	friend uint32 GetTypeHash(FType Type) { return HashCombineFast(Type.Scope.AsInt(), Type.Name.Idx); };
};

//////////////////////////////////////////////////////////////////////////

template<class IdType>
uint32 ToIdx(IdType Id)
{
	return Id.Idx;
}
inline uint32 ToIdx(FMemberId Name) { return Name.Id.Idx; }
inline uint32 ToIdx(FParametricTypeId Id) { return Id.AsInt(); }
inline uint32 ToIdx(FConcreteTypenameId Name) { return Name.Id.Idx; }

template<class IdType>
IdType FromIdx(uint32 Idx)
{
	return {Idx};
}
template<> inline FMemberId FromIdx(uint32 Idx) { return {{Idx}}; }
template<> inline FInnerId FromIdx(uint32 Idx) { return FInnerId(Idx); }
template<> inline FParametricTypeId FromIdx(uint32 Idx) { return FParametricTypeId::FromInt(Idx); }

class FOptionalId
{
protected:
    uint32 Idx = ~0u; 
public:
    explicit operator bool() const { return Idx != ~0u; }
	friend uint32 GetTypeHash(FOptionalId Id) { return Id.Idx; };
};

template<class T>
class TOptionalId : public FOptionalId
{
public:
	constexpr TOptionalId() = default;
	constexpr TOptionalId(FNoId) {}

	template<typename U>
	constexpr TOptionalId(U Id) requires (std::is_convertible_v<U, T>)
	{
		Idx = ToIdx(static_cast<T>(Id));
	}

	template<typename U>
	constexpr explicit TOptionalId(TOptionalId<U> In) requires (std::is_constructible_v<T,U>)
	: FOptionalId(static_cast<const FOptionalId&>(In))
	{}
    
	template<class U>
    explicit operator TOptionalId<U>() const requires(std::is_convertible_v<U, T> || std::is_convertible_v<T, U>)
    {
		return static_cast<const TOptionalId<U>&>(static_cast<const FOptionalId&>(*this));
    }

    T Get() const
	{
		check(*this);
		return FromIdx<T>(Idx);
	}

	T GetOr(T Fallback) const
	{
		return *this ? FromIdx<T>(Idx) : Fallback;
	}

	friend bool operator==(TOptionalId A, TOptionalId B) { return A.Idx == B.Idx;}
	friend bool operator!=(TOptionalId A, TOptionalId B) { return A.Idx != B.Idx;}
};

using FOptionalNameId = TOptionalId<FNameId>;
using FOptionalMemberId = TOptionalId<FMemberId>;
using FOptionalEnumId = TOptionalId<FEnumId>;
using FOptionalStructId = TOptionalId<FStructId>;
using FOptionalDeclId = TOptionalId<FDeclId>;
using FOptionalBindId = TOptionalId<FBindId>;
using FOptionalInnerId = TOptionalId<FInnerId>;
using FOptionalSchemaId = TOptionalId<FSchemaId>;
using FOptionalStructSchemaId = TOptionalId<FStructSchemaId>;
using FOptionalEnumSchemaId = TOptionalId<FEnumSchemaId>;
using FOptionalNestedScopeId = TOptionalId<FNestedScopeId>;
using FOptionalParametricTypeId = TOptionalId<FParametricTypeId>;
using FOptionalConcreteTypenameId = TOptionalId<FConcreteTypenameId>;

template<class IdType>
inline constexpr TOptionalId<IdType> ToOptional(IdType Id) { return Id; }

inline FOptionalInnerId			ToOptionalInner(FOptionalEnumId In)				{ return static_cast<const FOptionalInnerId&>(static_cast<const FOptionalId&>(In)); }
inline FOptionalInnerId			ToOptionalInner(FOptionalStructId In)			{ return static_cast<const FOptionalInnerId&>(static_cast<const FOptionalId&>(In)); }
inline FOptionalInnerId			ToOptionalInner(FOptionalDeclId In)				{ return static_cast<const FOptionalInnerId&>(static_cast<const FOptionalId&>(In)); }
inline FOptionalEnumId			ToOptionalEnum(FOptionalInnerId In)				{ return static_cast<const FOptionalEnumId&>(static_cast<const FOptionalId&>(In)); }
inline FOptionalStructId		ToOptionalStruct(FOptionalInnerId In)			{ return static_cast<const FOptionalStructId&>(static_cast<const FOptionalId&>(In)); }
inline FOptionalStructId		ToOptionalStruct(FOptionalDeclId In)			{ return static_cast<const FOptionalStructId&>(static_cast<const FOptionalId&>(In)); }
inline FOptionalStructId		ToOptionalStruct(FOptionalBindId In)			{ return static_cast<const FOptionalStructId&>(static_cast<const FOptionalId&>(In)); }
inline FOptionalDeclId			ToOptionalDeclId(FOptionalInnerId In)			{ return static_cast<const FOptionalDeclId&>(static_cast<const FOptionalId&>(In)); }
inline FOptionalSchemaId		ToOptionalSchema(FOptionalStructSchemaId In)	{ return static_cast<const FOptionalSchemaId&>(static_cast<const FOptionalId&>(In)); }
inline FOptionalSchemaId		ToOptionalSchema(FOptionalEnumSchemaId In)		{ return static_cast<const FOptionalSchemaId&>(static_cast<const FOptionalId&>(In)); }
inline FOptionalEnumSchemaId	ToOptionalEnum(FOptionalSchemaId In)			{ return static_cast<const FOptionalEnumSchemaId&>(static_cast<const FOptionalId&>(In)); }
inline FOptionalStructSchemaId	ToOptionalStruct(FOptionalSchemaId In)			{ return static_cast<const FOptionalStructSchemaId&>(static_cast<const FOptionalId&>(In)); }

//////////////////////////////////////////////////////////////////////////

// Resolved FNestedScopeId
struct FNestedScope
{
	FScopeId		Outer; // @invariant !!Outer
	FFlatScopeId 	Inner;

	bool operator==(FNestedScope O) const { return Outer.AsInt() == O.Outer.AsInt() && Inner.Name == O.Inner.Name; }
	friend uint32 GetTypeHash(FNestedScope Scope) { return HashCombineFast(Scope.Outer.AsInt(), Scope.Inner.Name.Idx); };
};

struct FParameterIndexRange : FBaseTypenameId { using FBaseTypenameId::FBaseTypenameId; };

// Name-resolved FParametricTypeId
struct FParametricType
{
	FOptionalConcreteTypenameId	Name;
	FParameterIndexRange		Parameters;

	bool operator==(FParametricType O) const { return Name == O.Name && Parameters.AsInt() == O.Parameters.AsInt(); }
};

// Fully resolved FParametricTypeId
struct FParametricTypeView
{
	FParametricTypeView(FConcreteTypenameId InName, uint8 NumParams, const FType* Params) : Name(InName), NumParameters(NumParams), Parameters(Params) {}
	FParametricTypeView(FOptionalConcreteTypenameId InName, uint8 NumParams, const FType* Params) : Name(InName), NumParameters(NumParams), Parameters(Params) {}
	FParametricTypeView(FConcreteTypenameId InName, TConstArrayView<FType> Params);
	FParametricTypeView(FOptionalConcreteTypenameId InName, TConstArrayView<FType> Params);

	FOptionalConcreteTypenameId	Name;
	uint8						NumParameters;
	const FType*				Parameters;

	TConstArrayView<FType>		GetParameters() const { return MakeArrayView(Parameters, NumParameters); }
};

//////////////////////////////////////////////////////////////////////////

template<class T>
concept Arithmetic = std::is_arithmetic_v<T>;

template<class T>
concept Enumeration = std::is_enum_v<T>;

// std::unsigned_integral and other std concepts are not supported by all tool chains yet
template<class T>
concept UnsignedIntegral = std::is_integral_v<T> && !std::is_signed_v<T>;

template<class T>
concept LeafType = std::is_arithmetic_v<T> || std::is_enum_v<T>;

//////////////////////////////////////////////////////////////////////////

struct FUnpackedLeafType
{
	ELeafType Type;
	ELeafWidth Width;

	inline constexpr FUnpackedLeafType(ELeafType InType, ELeafWidth InWidth) : Type(InType), Width(InWidth) {}
	inline constexpr FUnpackedLeafType(FLeafType In) : Type(In.Type), Width(In.Width) {}

	inline uint16 AsInt() const							{ return BitCast<uint16>(*this); }
	inline bool operator==(FUnpackedLeafType O) const	{ return AsInt() == O.AsInt(); }
	inline constexpr FMemberType Pack() const			{ return FMemberType(Type, Width); }
};

template<Enumeration T>
inline constexpr FUnpackedLeafType ReflectEnum = { ELeafType::Enum, LeafWidth<sizeof(T)> };

template<typename T>
constexpr FUnpackedLeafType IllegalLeaf()
{
	static_assert(!sizeof(T), "Unsupported leaf type");
	return { ELeafType::Bool,	ELeafWidth::B8 };
}

template<Arithmetic T>
inline constexpr FUnpackedLeafType ReflectArithmetic = IllegalLeaf<T>;

template<> inline constexpr FUnpackedLeafType ReflectArithmetic<bool>		= { ELeafType::Bool,	ELeafWidth::B8 };
template<> inline constexpr FUnpackedLeafType ReflectArithmetic<int8>		= { ELeafType::IntS,	ELeafWidth::B8 };
template<> inline constexpr FUnpackedLeafType ReflectArithmetic<int16>		= { ELeafType::IntS,	ELeafWidth::B16 };
template<> inline constexpr FUnpackedLeafType ReflectArithmetic<int32>		= { ELeafType::IntS,	ELeafWidth::B32 };
template<> inline constexpr FUnpackedLeafType ReflectArithmetic<int64>		= { ELeafType::IntS,	ELeafWidth::B64 };
template<> inline constexpr FUnpackedLeafType ReflectArithmetic<uint8>		= { ELeafType::IntU,	ELeafWidth::B8 };
template<> inline constexpr FUnpackedLeafType ReflectArithmetic<uint16>		= { ELeafType::IntU,	ELeafWidth::B16 };
template<> inline constexpr FUnpackedLeafType ReflectArithmetic<uint32>		= { ELeafType::IntU,	ELeafWidth::B32 };
template<> inline constexpr FUnpackedLeafType ReflectArithmetic<uint64>		= { ELeafType::IntU,	ELeafWidth::B64 };
template<> inline constexpr FUnpackedLeafType ReflectArithmetic<float>		= { ELeafType::Float,	ELeafWidth::B32 };
template<> inline constexpr FUnpackedLeafType ReflectArithmetic<double>		= { ELeafType::Float,	ELeafWidth::B64 };
template<> inline constexpr FUnpackedLeafType ReflectArithmetic<char>		= { ELeafType::Unicode,	ELeafWidth::B8 };
template<> inline constexpr FUnpackedLeafType ReflectArithmetic<char8_t>	= { ELeafType::Unicode,	ELeafWidth::B8 };
template<> inline constexpr FUnpackedLeafType ReflectArithmetic<char16_t>	= { ELeafType::Unicode,	ELeafWidth::B16 };
template<> inline constexpr FUnpackedLeafType ReflectArithmetic<char32_t>	= { ELeafType::Unicode,	ELeafWidth::B32 };

template<Arithmetic T>
inline constexpr FUnpackedLeafType MakeLeafType() { return ReflectArithmetic<T>; }

template<Enumeration T>
inline constexpr FUnpackedLeafType MakeLeafType() { return ReflectEnum<T>; }

template<typename T>
inline constexpr FUnpackedLeafType ReflectLeaf = MakeLeafType<T>();

//////////////////////////////////////////////////////////////////////////

inline constexpr ESizeType RangeSizeOf(bool)	{ return ESizeType::Uni; }
inline constexpr ESizeType RangeSizeOf(int8)	{ return ESizeType::S8; }
inline constexpr ESizeType RangeSizeOf(int16)	{ return ESizeType::S16; }
inline constexpr ESizeType RangeSizeOf(int32)	{ return ESizeType::S32; }
inline constexpr ESizeType RangeSizeOf(int64)	{ return ESizeType::S64; }
inline constexpr ESizeType RangeSizeOf(uint8)	{ return ESizeType::U8; }
inline constexpr ESizeType RangeSizeOf(uint16)	{ return ESizeType::U16; }
inline constexpr ESizeType RangeSizeOf(uint32)	{ return ESizeType::U32; }
inline constexpr ESizeType RangeSizeOf(uint64)	{ return ESizeType::U64; }

//////////////////////////////////////////////////////////////////////////

template<typename T>
const T* AlignPtr(const void* Ptr)
{
	return static_cast<const T*>(Align(Ptr, alignof(T)));
}

//////////////////////////////////////////////////////////////////////////

enum class ESchemaFormat { StableNames, InMemoryNames };

//////////////////////////////////////////////////////////////////////////

using FUtf8Builder = FUtf8StringBuilderBase;

// Resolves structured ids and converts ids to strings
class FIdsBase
{
public:
	virtual ~FIdsBase() {}

	virtual uint32					NumNames() const = 0;
	virtual uint32					NumNestedScopes() const = 0;
	virtual uint32					NumParametricTypes() const = 0;

	virtual FNestedScope			Resolve(FNestedScopeId Id) const = 0;
	virtual FParametricTypeView		Resolve(FParametricTypeId Id) const = 0;

	virtual void					AppendString(FUtf8Builder& Out, FNameId Name) const = 0;
	PLAINPROPS_API virtual void		AppendString(FUtf8Builder& Out, FMemberId Name) const;
	PLAINPROPS_API virtual void		AppendString(FUtf8Builder& Out, FOptionalMemberId Name) const;
	PLAINPROPS_API virtual void		AppendString(FUtf8Builder& Out, FScopeId Scope) const;
	PLAINPROPS_API virtual void		AppendString(FUtf8Builder& Out, FTypenameId Typename) const;
	PLAINPROPS_API virtual void		AppendString(FUtf8Builder& Out, FType Type) const;
};

// Runtime id resolver
class FIds : public FIdsBase
{
public:
	using FIdsBase::Resolve;
	using FIdsBase::AppendString;

	virtual uint32					NumEnums() const = 0;
	virtual uint32					NumStructs() const = 0;
	virtual FType					Resolve(FEnumId Id) const = 0;
	virtual FType					Resolve(FStructId Id) const = 0;

	PLAINPROPS_API virtual void		AppendString(FUtf8Builder& Out, FEnumId Enum) const;
	PLAINPROPS_API virtual void		AppendString(FUtf8Builder& Out, FStructId Struct) const;
};

// Helps format log messages
class FDebugIds
{
public:
	explicit FDebugIds(const FIds& InIds) : Ids(InIds) {}

	PLAINPROPS_API FString			Print(FNameId Name) const;
	PLAINPROPS_API FString			Print(FMemberId Name) const;
	PLAINPROPS_API FString			Print(FOptionalMemberId Name) const;
	PLAINPROPS_API FString			Print(FScopeId Scope) const;
	PLAINPROPS_API FString			Print(FTypenameId Typename) const;
	PLAINPROPS_API FString			Print(FConcreteTypenameId Typename) const;
	PLAINPROPS_API FString			Print(FParametricTypeId Typename) const;
	PLAINPROPS_API FString			Print(FType Type) const;
	PLAINPROPS_API FString			Print(FEnumId Enum) const;
	PLAINPROPS_API FString			Print(FStructId Struct) const;
	PLAINPROPS_API FString			Print(const struct FMemberSchema& Schema) const;

private:
	const FIds& Ids;
};

} // namespace PlainProps
