// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Find.h"
#include "Containers/Set.h"
#include "Math/IntPoint.h"
#include "Math/Quat.h"
#include "Math/Vector.h"
#include "Math/Transform.h"
#include "Misc/Optional.h"
#include "Misc/TVariant.h"
#include "PlainPropsBindCtti.h"
#include "PlainPropsIndex.h"
#include "PlainPropsLoadMember.h"
#include "PlainPropsSaveMember.h"
#include "PlainPropsStringUtil.h"
#include "UObject/NameTypes.h"

namespace Verse { class FNativeString; }
using FVerseString = Verse::FNativeString;

PP_REFLECT_STRUCT_TEMPLATE(, TTuple, void, Key, Value); // Todo handle TTuple and higher arities
PP_REFLECT_STRUCT_ONLY(0,, FEmptyVariantState, void);

namespace UE::Math
{
PP_REFLECT_STRUCT(, FVector, void, X, Y, Z);
PP_REFLECT_STRUCT(, FVector4, void, X, Y, Z, W);
PP_REFLECT_STRUCT(, FQuat, void, X, Y, Z, W);
PP_REFLECT_STRUCT(, FIntPoint, void, X, Y);
}

template <typename T>
struct TIsContiguousContainer<PlainProps::TRangeView<T>>
{
	static inline constexpr bool Value = true;
};

namespace PlainProps::UE
{

class FSensitiveName
{
public:
#if UE_FNAME_OUTLINE_NUMBER
	using IntType = uint32;
#else
	using IntType = uint64;
#endif
	inline explicit FSensitiveName(FName Name);
	inline explicit FSensitiveName(FAnsiStringView Name) : FSensitiveName(FName(Name)) {}
	inline FName ToName() const { return FName(GetComparisonId(), GetDisplayId(), GetNumber()); }
	inline void AppendString(FUtf8StringBuilderBase& Out) const { ToName().AppendString(Out); }
	inline IntType ToUnstableInt() const;
	inline static FSensitiveName FromUnstableInt(IntType UnstableInt) { return FSensitiveName(UnstableInt); }
	[[nodiscard]] inline bool operator==(FSensitiveName Other) const { return ToUnstableInt() == Other.ToUnstableInt(); }
	[[nodiscard]] inline bool operator!=(FSensitiveName Other) const { return !(*this == Other); }
	[[nodiscard]] friend inline uint32 GetTypeHash(FSensitiveName Name) { return ::GetTypeHash(Name.ToUnstableInt()); }

private:
	inline explicit FSensitiveName(IntType);

#if WITH_CASE_PRESERVING_NAME
	static constexpr uint32 DifferentIdsFlag = 1u << 31;
	static constexpr uint32 DisplayIdMask = ~DifferentIdsFlag;
 	uint32 Value; // DisplayId + DifferentIdsFlag
	inline bool SameIds() const { return (Value & DifferentIdsFlag) == 0; }
	inline FNameEntryId GetDisplayId() const { return FNameEntryId::FromUnstableInt(Value & DisplayIdMask); }
	inline FNameEntryId GetComparisonId() const { return SameIds() ? GetDisplayId() : FName::GetComparisonIdFromDisplayId(GetDisplayId()); }
	inline uint32 GetUnstableValue() const { return Value; }						// ToUnstableInt() helper
#else
	FNameEntryId DisplayId;
	inline FNameEntryId GetDisplayId() const { return DisplayId; }					// ToName() helper
	inline FNameEntryId GetComparisonId() const { return DisplayId; }				// ToName() helper
	inline uint32 GetUnstableValue() const { return DisplayId.ToUnstableInt(); }	// ToUnstableInt() helper
#endif

#if UE_FNAME_OUTLINE_NUMBER
	inline uint32 GetNumber() const { return NAME_NO_NUMBER_INTERNAL; }				// ToName() helper
	PLAINPROPS_API static uint32 InitValue(FName Name);								// Constructor helper for WITH_CASE_PRESERVING_NAME
	PLAINPROPS_API static FNameEntryId InitDisplayId(FName Name);					// Constructor helper for !WITH_CASE_PRESERVING_NAME
#else
	int32 Number;
	inline uint32 GetNumber() const { return Number; }								// ToName() helper
	inline static uint32 InitValue(FName Name);										// Constructor helper for WITH_CASE_PRESERVING_NAME
	inline static FNameEntryId InitDisplayId(FName Name);							// Constructor helper for !WITH_CASE_PRESERVING_NAME
#endif
};

FSensitiveName::FSensitiveName(FName Name)
#if WITH_CASE_PRESERVING_NAME
	: Value(InitValue(Name))
#else
	: DisplayId(InitDisplayId(Name))
#endif
#if !UE_FNAME_OUTLINE_NUMBER
	, Number(Name.GetNumber())
#endif
{}

FSensitiveName::FSensitiveName(IntType UnstableInt)
#if WITH_CASE_PRESERVING_NAME
	: Value(static_cast<uint32>(UnstableInt))
#else
	: DisplayId(FNameEntryId::FromUnstableInt(static_cast<uint32>(UnstableInt)))
#endif
#if !UE_FNAME_OUTLINE_NUMBER
	, Number(static_cast<uint32>(UnstableInt >> 32))
#endif
{}

FSensitiveName::IntType FSensitiveName::ToUnstableInt() const
{
#if UE_FNAME_OUTLINE_NUMBER
	return GetUnstableValue();
#else
	return (uint64(Number) << 32) | GetUnstableValue();
#endif
}

#if !UE_FNAME_OUTLINE_NUMBER // inline FSensitiveName::InitXXX functions that rely only on the public FName API
#if WITH_CASE_PRESERVING_NAME
uint32 FSensitiveName::InitValue(FName Name)
{
	return Name.GetDisplayIndex().ToUnstableInt() | (Name.GetDisplayIndex() != Name.GetComparisonIndex()) * DifferentIdsFlag;
}
#else
FNameEntryId FSensitiveName::InitDisplayId(FName Name)
{
	return Name.GetComparisonIndex();
}
#endif
#endif

// Temporary implementation for numeral structs
class FNumeralGenerator
{
	TIdIndexer<FSensitiveName>&	Indexer;
	TArray<FMemberId>			Cache; // 0, 1, 2, ..

	PLAINPROPS_API FMemberId	Grow(int32 Max);

public:
	FNumeralGenerator(TIdIndexer<FSensitiveName>& InIndexer) : Indexer(InIndexer) { }

	inline FMemberId			Make(uint16 Numeral)
	{
		return int32(Numeral) < Cache.Num() ? Cache[Numeral] : Grow(Numeral);
	}
	
	TConstArrayView<FMemberId>	MakeRange(uint16 Numerals)
	{
		Make(Numerals - 1);
		return MakeArrayView(Cache.GetData(), Numerals);
	}
};

//class FReflection
//{
//	TIdIndexer<FName>	Names;
//	FRuntime			Types;
//
//public:
//	TIdIndexer<FName>&		GetIds() { return Names; }

	//template<typename Ctti>
	//FStructSchemaId			BindStruct();

	//template<typename Ctti>
	//FStructSchemaId			BindStructInterlaced(TConstArrayView<FMemberBinding> NonCttiMembers);
	//FStructSchemaId			BindStruct(FStructSchemaId Id, const ICustomBinding& Custom);
	//FStructSchemaId			BindStruct(FType Type, FOptionalSchemaId Super, TConstArrayView<FNamedMemberBinding> Members, EMemberPresence Occupancy);
	//void					DropStruct(FStructSchemaId Id) { Types.DropStruct(Id); }

	//template<typename Ctti>
	//FEnumSchemaId			BindEnum();
	//FEnumSchemaId			BindEnum(FType Type, EEnumMode Mode, ELeafWidth Width, TConstArrayView<FEnumerator> Enumerators);
	//void					DropEnum(FEnumSchemaId Id) { Types.DropEnum(Id); }
//};
//
//PLAINPROPS_API FReflection GReflection;
//
//struct FIds
//{
//	static FMemberId		IndexMember(FAnsiStringView Name)			{ return GReflection.GetIds().NameMember(FName(Name)); }
//	static FTypenameId		IndexTypename(FAnsiStringView Name)			{ return GReflection.GetIds().MakeTypename(FName(Name)); }
//	static FScopeId			IndexCurrentModule()						{ return GReflection.GetIds().MakeScope(FName(UE_MODULE_NAME)); }
//	static FType			IndexNativeType(FAnsiStringView Typename)	{ return {IndexCurrentModule(), IndexTypename(Typename)}; }
//	static FEnumSchemaId	IndexEnum(FType Name)						;
//	static FEnumSchemaId	IndexEnum(FAnsiStringView Name)				;
//	static FStructSchemaId	IndexStruct(FType Name)					;
//	static FStructSchemaId	IndexStruct(FAnsiStringView Name)			;
//	
//};

// todo: use generic cached instance template?
//template<class Ids>
//FScopeId GetModuleScope()
//{
//	static FScopeId Id = Ids::IndexScope(UE_MODULE_NAME);
//	return Id;
//}

//template<class Ctti>
//class TBindRtti
//{
//	FSchemaId Id;
//public:
//	TBindRtti() : Id(BindRtti<Ctti, FIds>(GReflection.GetTypes()))
//	{}
//
//	~TBindRtti()
//	{
//		if constexpr (std::is_enum_v<Ctti::Type>)
//		{
//			GReflection.DropEnum(static_cast<FEnumSchemaId>(Id));
//		}
//		else
//		{
//			GReflection.DropStruct(static_cast<FStructSchemaId>(Id));
//		}
//	}
//};

} // namespace PlainProps::UE

//#define UEPP_BIND_STRUCT(T) 
	
//////////////////////////////////////////////////////////////////////////
// Below container bindings should be moved to some suitable header
//////////////////////////////////////////////////////////////////////////

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Templates/UniquePtr.h"

namespace PlainProps::UE
{

template <typename T, class Allocator>
struct TArrayBinding : IItemRangeBinding
{
	using SizeType = int32;
	using ItemType = T;
	using ArrayType = TArray<T, Allocator>;
	using IItemRangeBinding::IItemRangeBinding;
	inline static constexpr std::string_view BindName = TTypename<ArrayType>::RangeBindName;

	virtual void MakeItems(FLoadRangeContext& Ctx) const override
	{
		ArrayType& Array = Ctx.Request.GetRange<ArrayType>();
		if constexpr (std::is_default_constructible_v<T>)
		{
			Array.SetNum(Ctx.Request.NumTotal());
		}
		else
		{
			Array.SetNumUninitialized(Ctx.Request.NumTotal());
			Ctx.Items.SetUnconstructed();
		}
		
		Ctx.Items.Set(Array.GetData(), Ctx.Request.NumTotal());
	}

	virtual void ReadItems(FSaveRangeContext& Ctx) const override
	{
		const ArrayType& Array = Ctx.Request.GetRange<ArrayType>();
		Ctx.Items.SetAll(Array.GetData(), static_cast<uint64>(Array.Num()));
	}
};

//////////////////////////////////////////////////////////////////////////

template<class StringType>
struct TStringBinding : ILeafRangeBinding
{
	using SizeType = int32;
	using ItemType = char8_t;
	using CharType = typename StringType::ElementType;
	using ILeafRangeBinding::ILeafRangeBinding;
	inline static constexpr std::string_view BindName = TTypename<StringType>::RangeBindName;
	
	virtual void SaveLeaves(const void* Range, FLeafRangeAllocator& Out) const override
	{
		const TArray<CharType>& SrcArray = static_cast<const StringType*>(Range)->GetCharArray();
		const CharType* Src = SrcArray.GetData();
		int32 SrcLen = SrcArray.Num() - 1;
		if (SrcLen <= 0)
		{
		}
		else if constexpr (sizeof(CharType) == sizeof(char8_t))
		{
			char8_t* Utf8 = Out.AllocateRange<char8_t>(SrcLen);
			FMemory::Memcpy(Utf8, Src, SrcLen);
		}
		else
		{
			int32 Utf8Len = FPlatformString::ConvertedLength<UTF8CHAR>(Src, SrcLen);
			char8_t* Utf8 = Out.AllocateRange<char8_t>(Utf8Len);
			if (Utf8Len == SrcLen)
			{
				for (int32 Idx = 0; Idx < SrcLen; ++Idx)
				{
					Utf8[Idx] = static_cast<char8_t>(Src[Idx]);
				}
			}
			else
			{
				UTF8CHAR* Utf8End = FPlatformString::Convert(reinterpret_cast<UTF8CHAR*>(Utf8), Utf8Len, Src, SrcLen);	
				check((char8_t*)Utf8End - Utf8 == Utf8Len);
			}
		}
	}

	virtual void LoadLeaves(void* Range, FLeafRangeLoadView Items) const override
	{
		TArray<CharType>& Dst = static_cast<StringType*>(Range)->GetCharArray();
		TRangeView<char8_t> Utf8 = Items.As<char8_t>();
		// Todo: Better abstraction that hides internal representation
		const UTF8CHAR* Src = reinterpret_cast<const UTF8CHAR*>(Utf8.begin());
		int32 SrcLen = static_cast<int32>(Utf8.Num());
		if (SrcLen == 0)
		{
			Dst.Reset();
		}
		else if constexpr (sizeof(CharType) == sizeof(char8_t))
		{
			Dst.SetNumUninitialized(SrcLen + 1);
			FMemory::Memcpy(Dst.GetData(), Src, SrcLen);
			Dst[SrcLen] = CharType('\0');	
		}
		else
		{
			int32 DstLen = FPlatformString::ConvertedLength<CharType>(Src, SrcLen);
			Dst.SetNumUninitialized(DstLen + 1);
			if (DstLen == SrcLen)
			{
				CharType* Out = Dst.GetData();
				for (int32 Idx = 0; Idx < SrcLen; ++Idx)
				{
					Out[Idx] = static_cast<CharType>(Src[Idx]);
				}
				Out[SrcLen] = '\0';
			}
			else
			{
				CharType* DstEnd = FPlatformString::Convert(Dst.GetData(), DstLen, Src, SrcLen);
				check(DstEnd - Dst.GetData() == DstLen);
				*DstEnd = '\0';
			}
		}
	}

	virtual bool DiffLeaves(const void* RangeA, const void* RangeB) const override
	{
		const StringType& A = *static_cast<const StringType*>(RangeA);
		const StringType& B = *static_cast<const StringType*>(RangeB);
		// Case-sensitive unnormalized comparison
		return Diff(A.Len(), B.Len(), GetData(A), GetData(B), sizeof(CharType));
	}
};

//////////////////////////////////////////////////////////////////////////

template <typename T>
struct TUniquePtrBinding : IItemRangeBinding
{
	using SizeType = bool;
	using ItemType = T;
	using IItemRangeBinding::IItemRangeBinding;
	inline static constexpr std::string_view BindName = "UniquePtr";

	virtual void MakeItems(FLoadRangeContext& Ctx) const override
	{
		TUniquePtr<T>& Ptr = Ctx.Request.GetRange<TUniquePtr<T>>();
		
		if (Ctx.Request.NumTotal() == 0)
		{
			Ptr.Reset();
			return;
		}
		
		if (!Ptr)
		{
			if constexpr (std::is_default_constructible_v<T>)
			{
				Ptr.Reset(new T);
			}
			else
			{
				Ptr.Reset(reinterpret_cast<T*>(FMemory::Malloc(sizeof(T), alignof(T))));
				Ctx.Items.SetUnconstructed();
			}
		}
		
		Ctx.Items.Set(Ptr.Get(), 1);
	}

	virtual void ReadItems(FSaveRangeContext& Ctx) const override
	{
		const TUniquePtr<T>& Ptr = Ctx.Request.GetRange<TUniquePtr<T>>();
		Ctx.Items.SetAll(Ptr.Get(), Ptr ? 1 : 0);
	}
};

//////////////////////////////////////////////////////////////////////////

template <typename T>
struct TOptionalBinding : IItemRangeBinding
{
	using SizeType = bool;
	using ItemType = T;
	using IItemRangeBinding::IItemRangeBinding;
	inline static constexpr std::string_view BindName = "Optional";

	virtual void MakeItems(FLoadRangeContext& Ctx) const override
	{
		TOptional<T>& Opt = Ctx.Request.GetRange<TOptional<T>>();
		Opt.Reset();

		if (Ctx.Request.NumTotal() == 0)
		{
			return;
		}
		
		if constexpr (std::is_default_constructible_v<T>)
		{
			Opt.Emplace();
			Ctx.Items.Set(reinterpret_cast<T*>(&Opt), 1);
		}
		else if (Ctx.Request.IsFirstCall())
		{
			Ctx.Items.SetUnconstructed();
			Ctx.Items.RequestFinalCall();
			Ctx.Items.Set(reinterpret_cast<T*>(&Opt), 1);	
		}
		else // Move-construct from self reference
		{
			Opt.Emplace(reinterpret_cast<T&&>(Opt));
		}
	}

	virtual void ReadItems(FSaveRangeContext& Ctx) const override
	{
		const TOptional<T>& Opt = Ctx.Request.GetRange<TOptional<T>>();
		check(!Opt || reinterpret_cast<const T*>(&Opt) == &Opt.GetValue());
		Ctx.Items.SetAll(reinterpret_cast<const T*>(Opt ? &Opt : nullptr), Opt ? 1 : 0);
	}
};

//////////////////////////////////////////////////////////////////////////

template <typename T, typename KeyFuncs, typename SetAllocator>
struct TSetBinding : IItemRangeBinding
{
	using SizeType = int32;
	using ItemType = T;
	using SetType = TSet<T, KeyFuncs, SetAllocator>;
	using IItemRangeBinding::IItemRangeBinding;
	inline static constexpr std::string_view BindName = TTypename<SetType>::RangeBindName;

	virtual void MakeItems(FLoadRangeContext& Ctx) const override
	{
		SetType& Set = Ctx.Request.GetRange<SetType>();
		SizeType Num = static_cast<SizeType>(Ctx.Request.NumTotal());

		static constexpr bool bAllocate = sizeof(T) > sizeof(FLoadRangeContext::Scratch);
		static constexpr uint64 MaxItems = bAllocate ? 1 : sizeof(FLoadRangeContext::Scratch) / SIZE_T(sizeof(T));
		
		if (Ctx.Request.IsFirstCall())
		{
			Set.Reset();

			if (uint64 NumRequested = Ctx.Request.NumTotal())
			{
				Set.Reserve(NumRequested);

				// Create temporary buffer
				uint64 NumTmp = FMath::Min(MaxItems, NumRequested);
				void* Tmp = bAllocate ? FMemory::Malloc(sizeof(T)) : Ctx.Scratch;
				Ctx.Items.Set(Tmp, NumTmp, sizeof(T));
				if constexpr (std::is_default_constructible_v<T>)
				{
					for (T* It = static_cast<T*>(Tmp), *End = It + NumTmp; It != End; ++It)
					{
						::new (It) T;
					}
				}
				else
				{
					Ctx.Items.SetUnconstructed();
				}

				Ctx.Items.RequestFinalCall();
			}
		}
		else
		{
			// Add items that have been loaded
			TArrayView<T> Tmp = Ctx.Items.Get<T>();
			for (T& Item : Tmp)
			{
				Set.Emplace(MoveTemp(Item));
			}

			if (Ctx.Request.IsFinalCall())
			{
				// Destroy and free temporaries
				uint64 NumTmp = FMath::Min(MaxItems, Ctx.Request.NumTotal());
				for (T& Item : MakeArrayView(Tmp.GetData(), NumTmp))
				{
					Item.~T();
				}
				if constexpr (bAllocate)
				{
					FMemory::Free(Tmp.GetData());
				}	
			}
			else
			{
				Ctx.Items.Set(Tmp.GetData(), FMath::Min(static_cast<uint64>(Tmp.Num()), Ctx.Request.NumMore()));
				check(Ctx.Items.Get<T>().Num());
			}
		}
	}

#if UE_USE_COMPACT_SET_AS_DEFAULT
	virtual void ReadItems(FSaveRangeContext& Ctx) const override
	{
#if UE_TCOMPACT_SET_ARRAY_VIEW
		const TArrayView<const T> SetData = Ctx.Request.GetRange<SetType>().ArrayView();
		Ctx.Items.SetAll(SetData.GetData(), SetData.Num());
#else
		// Need to break into jail here, compact sets are linear arrays of data so I can use the address of the first element
		// to set the rest. ArrayView is currently disabled on it to maintain API compatibility with sparse set.
		const SetType& Set = Ctx.Request.GetRange<SetType>();
		const T* SetData = Set.FindArbitraryElement();
		Ctx.Items.SetAll(SetData, Set.Num());
#endif
	}
#else
	virtual void ReadItems(FSaveRangeContext& Ctx) const override
	{
		static_assert(offsetof(TSparseSetElement<T>, Value) == 0);
		const TSparseArray<TSparseSetElement<T>>& Elems = Ctx.Request.GetRange<TSparseArray<TSparseSetElement<T>>>();

		if (Elems.IsEmpty())
		{
			Ctx.Items.SetAll(nullptr, 0, sizeof(TSparseSetElement<T>));
		}
		else if (FExistingItemSlice LastRead = Ctx.Items.Slice)
		{
			// Continue partial response
			const TSparseSetElement<T>* NextElem = static_cast<const TSparseSetElement<T>*>(LastRead.Data) + LastRead.Num + /* skip known invalid */ 1;
			Ctx.Items.Slice = GetContiguousSlice(Elems.PointerToIndex(NextElem), Elems);
		}
		else if (Elems.IsCompact())
		{
			Ctx.Items.SetAll(&Elems[0], Elems.Num());
		}
		else
		{
			// Start partial response
			Ctx.Items.NumTotal = Elems.Num();
			Ctx.Items.Stride = sizeof(TSparseSetElement<T>);
			Ctx.Items.Slice = GetContiguousSlice(0, Elems);
		}
	}

	static FExistingItemSlice GetContiguousSlice(int32 Idx, const TSparseArray<TSparseSetElement<T>>& Elems)
	{
		int32 Num = 1;
		for (;!Elems.IsValidIndex(Idx); ++Idx) {}
		for (; Elems.IsValidIndex(Idx + Num); ++Num) {}
		return { &Elems[Idx], static_cast<uint64>(Num) };
	}
#endif
};

//////////////////////////////////////////////////////////////////////////

// Only used for non-default constructible pairs
template <typename K, typename V>
struct TPairBinding : ICustomBinding
{
	using Type = TPair<K,V>;

	template<class Ids>
	TPairBinding(TCustomSpecifier<Ids, 2>& Spec)
	: MemberIds{Ids::IndexMember("Key"), Ids::IndexMember("Value")}
	, Key(Spec, { MemberIds[0] })
	, Value(Spec, { MemberIds[1] })
	{
		Spec.SetMembers(Key.SpecMember(), Value.SpecMember());
	}

	inline void Save(FMemberBuilder& Dst, const TPair<K,V>& Src, const Type* Default, const FSaveContext& Ctx) const
	{
		Dst.Add(MemberIds[0], Key.SaveMember(Src.Key, Ctx));
		Dst.Add(MemberIds[1], Value.SaveMember(Src.Value, Ctx));
	}
		
	inline void Load(TPair<K,V>& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
	{
		FMemberLoader Members(Src);
		check(Members.PeekName() == MemberIds[0]);
		if (Method == ECustomLoadMethod::Construct)
		{
			Key.ConstructAndLoadMember(/* out */ &Dst.Key, /* in-out */ Members);
			Value.ConstructAndLoadMember(/* out */ &Dst.Value, /* in-out */ Members);		
		}
		else
		{
			Key.LoadMember(/* out */ Dst.Key, /* in-out */ Members);
			Value.LoadMember(/* out */ Dst.Value, /* in-out */ Members);				
		}
	}

	template<typename ContextType>
	bool Diff(const TPair<K,V>& A, const TPair<K,V>& B, ContextType& Ctx) const
	{
		return Key.DiffMember(A.Key, B.Key, MemberIds[0], Ctx) || 
			Value.DiffMember(A.Value, B.Value, MemberIds[1], Ctx);
	}

	const FMemberId				MemberIds[2];
	const TMemberSerializer<K>	Key;
	const TMemberSerializer<V>	Value;
};

//////////////////////////////////////////////////////////////////////////

template <typename K, typename V, typename SetAllocator, typename KeyFuncs>
struct TMapBinding : public TSetBinding<TPair<K, V>, KeyFuncs, SetAllocator>
{
	using MapType = TMap<K, V, SetAllocator, KeyFuncs>;
	using Super = TSetBinding<TPair<K, V>, KeyFuncs, SetAllocator>;
	using Super::Super;
	inline static constexpr std::string_view BindName = TTypename<MapType>::RangeBindName;
};

//////////////////////////////////////////////////////////////////////////

template <Enumeration E>
struct TAuxEnumAsByteBinding : ICustomBinding
{
	using Type = TEnumAsByte<E>;

	const FMemberId				MemberIds[1];
	const FEnumId				EnumId;

	template<class Ids>
	TAuxEnumAsByteBinding(TCustomSpecifier<Ids, 1>& Spec)
	: MemberIds{Ids::IndexMember("Value")}
	, EnumId(GetEnumId<Ids, E>())
	{
		Spec.Members[0] = Specify<E>(EnumId);
	}

	void Save(FMemberBuilder& Dst, const TEnumAsByte<E>& Src, const TEnumAsByte<E>* Default, const FSaveContext& Ctx) const
	{
		Dst.AddEnum(MemberIds[0], EnumId, Src.GetValue());
	}
		
	void Load(TEnumAsByte<E>& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
	{
		FMemberLoader Members(Src);
		Dst = Members.GrabLeaf().As<E>();
	}

	static bool Diff(TEnumAsByte<E> A, TEnumAsByte<E> B, const FBindContext& Ctx) { return A != B; }
};

//////////////////////////////////////////////////////////////////////////

//TODO: Consider macroifying parts of this, e.g PP_CUSTOM_BIND(PLAINPROPS_API, FTransform, Transform, Translate, Rotate, Scale)
struct FTransformBinding : ICustomBinding
{
	using Type = FTransform;
	enum class EMember : uint8 { Translate, Rotate, Scale };

	const FMemberId					MemberIds[3];
	const FDualStructId				VectorId;
	const FDualStructId				QuatId;
	mutable FStructDeclarationPtr	VectorDeclaration;
	mutable FStructDeclarationPtr	QuatDeclaration;

	template<class Ids>
	FTransformBinding(TCustomSpecifier<Ids, 3>& Spec)
	: MemberIds{Ids::IndexMember("Translate"), Ids::IndexMember("Rotate"), Ids::IndexMember("Scale")}
	, VectorId(GetStructBindId<Ids, FVector>())
	, QuatId(GetStructBindId<Ids, FQuat>())
	{
		Spec.SetMembers(VectorId, QuatId, VectorId);
	}
	
	PLAINPROPS_API void	Save(FMemberBuilder& Dst, const FTransform& Src, const FTransform* Default, const FSaveContext& Context) const;
	PLAINPROPS_API void	Load(FTransform& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	PLAINPROPS_API bool Diff(const FTransform& A, const FTransform& B, FDiffContext& Ctx) const;
	inline static  bool Diff(const FTransform& A, const FTransform& B, const FBindContext&) { return !A.Equals(B, 0.0); }
};

//////////////////////////////////////////////////////////////////////////

struct FGuidBinding : ICustomBinding
{
	using Type = FGuid;
	const FMemberId MemberIds[4];
	
	template<class Ids>
	FGuidBinding(TCustomSpecifier<Ids, 4>& Spec)
	: MemberIds{Ids::IndexMember("A"), Ids::IndexMember("B"), Ids::IndexMember("C"), Ids::IndexMember("D")}
	{
		Spec.FillMembers(SpecHex32);
	}

	PLAINPROPS_API void	Save(FMemberBuilder& Dst, const FGuid& Src, const FGuid* Default, const FSaveContext&) const;
	inline static void	Plan(FMemcpyLoadPlan& Out) { Out.Size = sizeof(Type); }
	inline static bool	Diff(FGuid A, FGuid B, const FBindContext&) { return A != B; }
};

//////////////////////////////////////////////////////////////////////////

struct FColorBinding : ICustomBinding
{
	using Type = FColor;
	const FMemberId MemberIds[4];

	template<class Ids>
	FColorBinding(TCustomSpecifier<Ids, 4>& Spec)
	: MemberIds{Ids::IndexMember("B"), Ids::IndexMember("G"), Ids::IndexMember("R"), Ids::IndexMember("A")}
	{
		Spec.FillMembers(Specify<decltype(FColor::R)>());
	}
	
	PLAINPROPS_API void	Save(FMemberBuilder& Dst, const FColor& Src, const FColor* Default, const FSaveContext&) const;
	inline static void	Plan(FMemcpyLoadPlan& Out) { Out.Size = sizeof(Type); }
	inline static bool	Diff(FColor A, FColor B, const FBindContext&) { return A != B; }
};

//////////////////////////////////////////////////////////////////////////

struct FLinearColorBinding : ICustomBinding
{
	using Type = FLinearColor;
	const FMemberId MemberIds[4];

	template<class Ids>
	FLinearColorBinding(TCustomSpecifier<Ids, 4>& Spec)
	: MemberIds{Ids::IndexMember("R"), Ids::IndexMember("G"), Ids::IndexMember("B"), Ids::IndexMember("A")}
	{
		Spec.FillMembers(Specify<decltype(FLinearColor::R)>());
	}
	
	PLAINPROPS_API void	Save(FMemberBuilder& Dst, const FLinearColor& Src, const FLinearColor* Default, const FSaveContext&) const;
	inline static void	Plan(FMemcpyLoadPlan& Out) { Out.Size = sizeof(Type); }
	inline static bool	Diff(FLinearColor A, FLinearColor B, const FBindContext&) { return A != B; }
};

//////////////////////////////////////////////////////////////////////////

struct FBaseDeltaBinding
{
	enum class EOp { Del, Add };
	FMemberId MemberIds[2];
	FMemberId DelId() const { return MemberIds[(uint32)EOp::Del]; }
	FMemberId AddId() const { return MemberIds[(uint32)EOp::Add]; }
	
	template<class Ids>	
	FBaseDeltaBinding(TInit<Ids>)
	: MemberIds{Ids::IndexMember("Del"), Ids::IndexMember("Add")}
	{}

	template<class Ids>	
	static FBaseDeltaBinding Cache()
	{
		static FBaseDeltaBinding Out(TInit<Ids>{});
		return Out;
	}
};

template <typename SetType, typename KeyType>
struct TBaseDeltaBinding : ICustomBinding, FBaseDeltaBinding
{
	using Type = SetType;
	using ElemType = typename SetType::ElementType;
	static constexpr ESizeType MaxSize = ESizeType::S32;
	static constexpr bool bIsSet = std::is_same_v<KeyType, ElemType>;
	
	const TMemberSerializer<ElemType> Elems;

	template<class Ids>	
	TBaseDeltaBinding(TCustomInit<Ids>& Init)
	: FBaseDeltaBinding(FBaseDeltaBinding::Cache<Ids>())
	, Elems(Init, bIsSet ? MakeArrayView(MemberIds) : MakeArrayView({AddId()}))
	{}

	template<typename KeyOrElemType>
	static const KeyOrElemType& Get(const ElemType& Elem)
	{
		if constexpr (std::is_same_v<KeyOrElemType, ElemType>)
		{
			return Elem;
		}
		else
		{
			return Elem.Key;
		}
	}

	// Todo: Reimplement with Assign/Remove/Insert like SaveSetDelta in PlainPropsUObjectBindings.cpp
	inline void SaveDelta(FMemberBuilder& Dst, const SetType& Src, const SetType* Default, const FSaveContext& Ctx, const TMemberSerializer<KeyType>& Keys) const
	{
		if (Src.IsEmpty())
		{
			if (Default && !Default->IsEmpty())
			{
				Dst.AddRange(DelId(), SaveAll<KeyType>(*Default, Keys, Ctx));
			}
		}
		else if (Default && !Default->IsEmpty())
		{
			TBitArray<> DelSubset(false, Default->GetMaxIndex());
			for (auto It = Default->CreateConstIterator(); It; ++It)
			{
				DelSubset[It.GetId().AsInteger()] = !Src.Contains(Get<KeyType>(*It));
			}
			if (DelSubset.Find(true) != INDEX_NONE)
			{
				Dst.AddRange(DelId(), SaveSome<KeyType>(*Default, DelSubset, Keys, Ctx));
			}

			TBitArray<> AddSubset(false, Src.GetMaxIndex());
			for (auto It = Src.CreateConstIterator(); It; ++It)
			{
				AddSubset[It.GetId().AsInteger()] = !ContainsValue(*Default, *It);
			}
			if (AddSubset.Find(true) != INDEX_NONE)
			{
				Dst.AddRange(AddId(), SaveSome<ElemType>(Src, AddSubset, Elems, Ctx));
			}
		}
		else
		{
			Dst.AddRange(AddId(), SaveAll<ElemType>(Src, Elems, Ctx));
		}
	}

	template<typename ItemType>
	FTypedRange SaveAll(const SetType& Set, const TMemberSerializer<ItemType>& Schema, const FSaveContext& Ctx) const
	{
		check(!Set.IsEmpty());

		TRangeSaver<ItemType> Items(Ctx, static_cast<uint64>(Set.Num()), Schema);
		for (const ElemType& Elem : Set)
		{
			Items.AddItem(Get<ItemType>(Elem));
		}
		return Items.Finalize(MaxSize);
	}
	
	template<typename ItemType>
	FTypedRange SaveSome(const SetType& Set, const TBitArray<>& Subset, const TMemberSerializer<ItemType>& Schema, const FSaveContext& Ctx) const
	{
		TRangeSaver<ItemType> Items(Ctx, Subset.CountSetBits(), Schema);
		for (int32 Idx = 0, Max = Set.GetMaxIndex(); Idx < Max; ++Idx)
		{
			if (Subset[Idx])
			{
				const ElemType& Elem = Set.Get(FSetElementId::FromInteger(Idx));
				Items.AddItem(Get<ItemType>(Elem));
			}
		}
		return Items.Finalize(MaxSize);
	}

	inline void LoadDelta(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method, const TMemberSerializer<KeyType>& Keys) const
	{
		FMemberLoader Members(Src);

		if (Method == ECustomLoadMethod::Construct)
		{
			::new (&Dst) SetType;
		}
				
		while (Members.HasMore())
		{
			if (Members.PeekNameUnchecked() == AddId())
			{
				ApplyItems<EOp::Add, ElemType>(Dst, Members.GrabRange(), Elems);
				check(!Members.HasMore());
				break;
			}

			check(Members.PeekNameUnchecked() == DelId());
			ApplyItems<EOp::Del, KeyType>(Dst, Members.GrabRange(), Keys);
		}
	}

	template<EOp Op, typename T>
	void ApplyItems(SetType& Out, FRangeLoadView Items, const TMemberSerializer<T>& Schema) const
	{
		check(!Items.IsEmpty());
 
		if constexpr (Op == EOp::Add && !LeafType<T>)
		{
			Out.Reserve(static_cast<int32>(Items.Num()));
		}

		if constexpr (LeafType<T>)
		{
			ApplyLeaves<Op, T>(Out, Items.AsLeaves());
		}
		else if constexpr (TMemberSerializer<T>::Kind == EMemberKind::Struct)
		{
			ApplyStructs<Op, T>(Out, Items.AsStructs());
		}
		else
		{
			ApplyRanges<Op, T>(Out, Items.AsRanges(), Schema);
		}
	}

	template<EOp Op, typename T>
	void ApplyLeaves(SetType& Out, FLeafRangeLoadView Items) const requires (Op == EOp::Add && !std::is_same_v<T, bool>)
	{
		Out.Append(MakeArrayView(Items.As<T>()));
	}

	template<EOp Op, typename T>
	void ApplyLeaves(SetType& Out, FLeafRangeLoadView Items) const
	{
		for (T Item : Items.As<T>())
		{
			ApplyItem<Op>(Out, Item);
		}
	}

	template<EOp Op, typename T>
	void ApplyRanges(SetType& Out, FNestedRangeLoadView Items, const TMemberSerializer<T>& Schema) const
	{
		static_assert(std::is_default_constructible_v<T>, TEXT("Ranges must be default-constructible"));

		TConstArrayView<FRangeBinding> Bindings(Schema.Bindings, Schema.NumRanges);
		for (FRangeLoadView Item : Items)
		{
			T Tmp;
			LoadRange(&Tmp, Item, Bindings);
			ApplyItem<Op>(Out, MoveTemp(Tmp));
		}
	}
	
	template<EOp Op, typename T>
	void ApplyStructs(SetType& Out, FStructRangeLoadView Items) const
	{
		for (FStructLoadView Item : Items)
		{
			if constexpr (std::is_default_constructible_v<T>)
			{
				T Tmp;
				LoadStruct(&Tmp, Item);
				ApplyItem<Op>(Out, MoveTemp(Tmp));
			}
			else
			{
				static_assert(sizeof(T) <= 128, "Switch to e.g. TArray<uint8, TInlineAllocator<32>> + SetNumInitialized(sizeof(T) + over-alignment) + AlignPtr(GetData(), alignof(T))");
				alignas(T) uint8 Buffer[sizeof(T)];
				ConstructAndLoadStruct(Buffer, Item);
				T& Tmp = *reinterpret_cast<T*>(Buffer);
				ApplyItem<Op>(Out, MoveTemp(Tmp));
				Tmp.~T();
			}
		}
	}
	
	template<EOp Op, typename T>
	void ApplyItem(SetType& Out, T&& Item) const
	{
		if constexpr (Op == EOp::Add)
		{
			Out.Add(MoveTemp(Item));
		}
		else
		{
			Out.Remove(Item);
		}
	}

	inline static bool ContainsValue(const Type& Set, const ElemType& Elem)
	{
		if constexpr (bIsSet)
		{
			return Set.Contains(Elem);
		}
		else
		{
			const auto* Value = Set.Find(Elem.Key);
			return Value && *Value == Elem.Value;
		}
	}

	inline static bool Diff(const Type& As, const Type& Bs, const FBindContext&)
	{
		if (As.Num() != Bs.Num())
		{
			return true;
		}

		for (const ElemType& A : As)
		{
			if (!ContainsValue(Bs, A))
			{
				return true;
			}
		}

		return false;
	}
};

template <typename T, typename KeyFuncs, typename SetAllocator>
struct TSetDeltaBinding : TBaseDeltaBinding<TSet<T, KeyFuncs, SetAllocator>, T>
{
	using SetType = TSet<T, KeyFuncs, SetAllocator>;
	using Super = TBaseDeltaBinding<SetType, T>;

	struct FCustomTypename
	{
		inline static constexpr std::string_view DeclName = "SetDelta";
		inline static constexpr std::string_view BindName = Concat<DeclName, ShortTypename<KeyFuncs>, ShortTypename<SetAllocator>>;
		inline static constexpr std::string_view Namespace;
		using Parameters = std::tuple<T>;
	};
		
	template<class Ids>	
	TSetDeltaBinding(TCustomSpecifier<Ids, 2>& Spec)
	: Super(Spec)
	{
		Spec.FillMembers(FMemberSpec(Super::MaxSize, Super::Elems.SpecMember()));
	}

	inline void Save(FMemberBuilder& Dst, const SetType& Src, const SetType* Default, const FSaveContext& Ctx) const
	{
		Super::SaveDelta(Dst, Src, Default, Ctx, /* Keys */ Super::Elems);
	}

	inline void Load(SetType& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
	{
		Super::LoadDelta(Dst, Src, Method, /* Keys */ Super::Elems);
	}
};

template <typename K, typename V, typename SetAllocator, typename KeyFuncs>
struct TMapDeltaBinding : TBaseDeltaBinding<TMap<K, V, SetAllocator, KeyFuncs>, K>
{
	using MapType = TMap<K, V, SetAllocator, KeyFuncs>;
	using Super = TBaseDeltaBinding<MapType, K>;

	struct FCustomTypename
	{
		inline static constexpr std::string_view DeclName = "MapDelta";
		inline static constexpr std::string_view BindName = Concat<DeclName, ShortTypename<KeyFuncs>, ShortTypename<SetAllocator>>;
		inline static constexpr std::string_view Namespace;
		using Parameters = std::tuple<K, V>;
	};
	
	const TMemberSerializer<K> Keys;
	
	template<class Ids>	
	TMapDeltaBinding(TCustomSpecifier<Ids, 2>& Spec)
	: Super(Spec)
	, Keys(Spec, {FBaseDeltaBinding::DelId()})
	{
		Spec.Members[0] = FMemberSpec(Super::MaxSize, Keys.SpecMember());
		Spec.Members[1] = FMemberSpec(Super::MaxSize, Super::Elems.SpecMember());
	}
		
	inline void Save(FMemberBuilder& Dst, const MapType& Src, const MapType* Default, const FSaveContext& Ctx) const
	{
		Super::SaveDelta(Dst, Src, Default, Ctx, Keys);
	}

	inline void Load(MapType& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
	{
		Super::LoadDelta(Dst, Src, Method, Keys);
	}
};

template <typename T, typename... Ts>
struct TVariantMemberSerializer
{
	using Type = TVariant<Ts...>;
	using TSerializers = std::tuple<TMemberSerializer<Ts>...>;

	static FTypedValue SaveMember(const Type& Src, const TSerializers& Serializers, const FSaveContext& Ctx)
	{
		return std::get<TMemberSerializer<T>>(Serializers).SaveMember(Src.template Get<T>(), Ctx);
	}

	static void LoadMember(Type& Dst, const TSerializers& Serializers, FMemberLoader& Members)
	{
		new (&Dst) Type(::UE::Core::Private::TUninitializedType<T>());
		std::get<TMemberSerializer<T>>(Serializers).ConstructAndLoadMember(&Dst.template Get<T>(), Members);
	}

	template<typename ContextType>
	static bool DiffMember(const Type& A, const Type& B, const TSerializers& Serializers, FMemberId MemberId, ContextType& Ctx)
	{
		check(A.GetIndex() == B.GetIndex());
		return std::get<TMemberSerializer<T>>(Serializers).DiffMember(A.template Get<T>(), B.template Get<T>(), MemberId, Ctx);
	}
};

template <typename... Ts>
struct TVariantMemberFunctions
{
	using Type = TVariant<Ts...>;
	using TSerializers = std::tuple<TMemberSerializer<Ts>...>;

	FTypedValue(*SaveMember)(const Type&, const TSerializers&, const FSaveContext&);
	void(*LoadMember)(Type&, const TSerializers&, FMemberLoader&);
	bool(*DiffMember)(const Type&, const Type&, const TSerializers&, FMemberId, const FBindContext&);
	bool(*DiffAndTrackMember)(const Type&, const Type&, const TSerializers&, FMemberId, FDiffContext&);
};

template <typename... Ts>
struct TVariantBinding : ICustomBinding
{
	using Type = TVariant<Ts...>;
	using TSerializers = std::tuple<TMemberSerializer<Ts>...>;
	static constexpr uint32 N = sizeof...(Ts);

	const FMemberId MemberIds[N];
	const TSerializers Serializers;
	static constexpr TVariantMemberFunctions<Ts...> Functions[N] = {{
		&TVariantMemberSerializer<Ts,Ts...>::SaveMember,
		&TVariantMemberSerializer<Ts,Ts...>::LoadMember,
		&TVariantMemberSerializer<Ts,Ts...>::DiffMember,
		&TVariantMemberSerializer<Ts,Ts...>::DiffMember}...};

	template<class Ids>
	TVariantBinding(TCustomSpecifier<Ids, N>& Spec)
	: MemberIds{Ids::IndexNumeral(::UE::Core::Private::TParameterPackTypeIndex<Ts, Ts...>::Value)...}
	, Serializers{{Spec, MakeArrayView({MemberIds[::UE::Core::Private::TParameterPackTypeIndex<Ts, Ts...>::Value]})}...}
	{
		Spec.SetMembers(std::get<TMemberSerializer<Ts>>(Serializers).SpecMember()...);
	}

	void Save(FMemberBuilder& Dst, const Type& Src, const Type* Default, const FSaveContext& Ctx) const
	{
		uint8 MemberIdx = IntCastChecked<uint8>(Src.GetIndex());
		check(MemberIdx < UE_ARRAY_COUNT(Functions));
		FTypedValue Value = Functions[MemberIdx].SaveMember(Src, Serializers, Ctx);
		Dst.Add(MemberIds[MemberIdx], Value);
	}

	void Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
	{
		FMemberLoader Members(Src);
		const FMemberId* DeclaredName = Algo::Find(MemberIds, Members.PeekName());
		check(DeclaredName);
		uint8 MemberIdx = IntCastChecked<uint8>(DeclaredName - &MemberIds[0]);
		check(MemberIdx < UE_ARRAY_COUNT(Functions));
		if (Method == ECustomLoadMethod::Assign)
		{
			Dst.~Type();
		}
		Functions[MemberIdx].LoadMember(Dst, Serializers, Members);
	}

	bool Diff(const Type& A, const Type& B, const FBindContext& Ctx) const
	{
		uint8 MemberIdx = IntCastChecked<uint8>(A.GetIndex());
		check(MemberIdx < UE_ARRAY_COUNT(Functions));
		if (MemberIdx != B.GetIndex())
		{
			return true;
		}
		return Functions[MemberIdx].DiffMember(A, B, Serializers, MemberIds[MemberIdx], Ctx);
	}

	bool Diff(const Type& A, const Type& B, FDiffContext& Ctx) const
	{
		uint8 MemberIdx = IntCastChecked<uint8>(A.GetIndex());
		check(MemberIdx < UE_ARRAY_COUNT(Functions));
		if (MemberIdx != B.GetIndex())
		{
			return true;
		}
		return Functions[MemberIdx].DiffAndTrackMember(A, B, Serializers, MemberIds[MemberIdx], Ctx);
	}
};

PLAINPROPS_API FEnumId DeclarePixelFormat();

} // namespace PlainProps::UE

namespace PlainProps
{

template <> struct TTypename<FName>					{ inline static constexpr std::string_view DeclName = "Name"; };
template <> struct TTypename<FText>					{ inline static constexpr std::string_view DeclName = "Text"; };
template <> struct TTypename<FIntPoint>				{ inline static constexpr std::string_view DeclName = "IntPoint"; };
template <> struct TTypename<FQuat>					{ inline static constexpr std::string_view DeclName = "Quat"; };
template <> struct TTypename<FTransform>			{ inline static constexpr std::string_view DeclName = "Transform"; };
template <> struct TTypename<FVector>				{ inline static constexpr std::string_view DeclName = "Vector"; };
template <> struct TTypename<FVector4>				{ inline static constexpr std::string_view DeclName = "Vector4"; };
template <> struct TTypename<FGuid>					{ inline static constexpr std::string_view DeclName = "Guid"; };
template <> struct TTypename<FColor>				{ inline static constexpr std::string_view DeclName = "Color"; };
template <> struct TTypename<FLinearColor>			{ inline static constexpr std::string_view DeclName = "LinearColor"; };
template <> struct TTypename<FEmptyVariantState>	{ inline static constexpr std::string_view DeclName = "EmptyVariant"; };
template <> struct TTypename<FString>				{ inline static constexpr std::string_view RangeBindName = "String"; };
template <> struct TTypename<FUtf8String>			{ inline static constexpr std::string_view RangeBindName = "Utf8String"; };
template <> struct TTypename<FAnsiString>			{ inline static constexpr std::string_view RangeBindName = "AnsiString"; };


template <typename K, typename V>
struct TTypename<TPair<K,V>>
{
	inline static constexpr std::string_view DeclName = "Pair";
	using Parameters = std::tuple<K, V>;
};

template <Enumeration E>
struct TTypename<TEnumAsByte<E>>
{
	inline static constexpr std::string_view DeclName = "EnumAsByte";
	using Parameters = std::tuple<E>;
};

template <typename... Ts>
struct TTypename<TVariant<Ts...>>
{
	inline static constexpr std::string_view DeclName = "Variant";
	using Parameters = std::tuple<Ts...>;
};

inline static constexpr std::string_view UeArrayName = "Array";
inline static constexpr std::string_view UeSetName = "Set";
inline static constexpr std::string_view UeMapName = "Map";

template<typename T, typename Allocator>
struct TTypename<TArray<T, Allocator>>
{
	inline static constexpr std::string_view RangeBindName = Concat<UeArrayName, ShortTypename<Allocator>>;
};

template<>
struct TShortTypename<FDefaultAllocator> : FOmitTypename {};
template<>
struct TShortTypename<FDefaultSetAllocator> : FOmitTypename {};
template<typename T>
struct TShortTypename<DefaultKeyFuncs<T, false>> : FOmitTypename {};
template<typename K, typename V>
struct TShortTypename<TDefaultMapHashableKeyFuncs<K, V, false>> : FOmitTypename {};

inline constexpr std::string_view InlineAllocatorPrefix = "InlX";
template<int N>
struct TShortTypename<TInlineAllocator<N>>
{
	inline static constexpr std::string_view Value = Concat<InlineAllocatorPrefix, HexString<N>>;
};

template<int N>
struct TShortTypename<TInlineSetAllocator<N>> : TShortTypename<TInlineAllocator<N>> {};

template <typename T, typename KeyFuncs, typename SetAllocator>
struct TTypename<TSet<T, KeyFuncs, SetAllocator>>
{
	inline static constexpr std::string_view RangeBindName = Concat<UeSetName, ShortTypename<KeyFuncs>, ShortTypename<SetAllocator>>;
};

template <typename K, typename V, typename SetAllocator, typename KeyFuncs>
struct TTypename<TMap<K, V, SetAllocator, KeyFuncs>>
{
	inline static constexpr std::string_view RangeBindName = Concat<UeMapName, ShortTypename<SetAllocator>, ShortTypename<KeyFuncs>>;
};

template<>
PLAINPROPS_API void AppendString(FUtf8StringBuilderBase& Out, const UE::FSensitiveName& Name);

template<typename T, typename Allocator>
struct TRangeBind<TArray<T, Allocator>>
{
	using Type = UE::TArrayBinding<T, Allocator>;
};

template<> struct TRangeBind<FString> { using Type = UE::TStringBinding<FString>; };
template<> struct TRangeBind<FAnsiString> {	using Type = UE::TStringBinding<FAnsiString>; };
template<> struct TRangeBind<FUtf8String> {	using Type = UE::TStringBinding<FUtf8String>; };
template<> struct TRangeBind<FVerseString> { using Type = UE::TStringBinding<FVerseString>; };

template<typename T>
struct TRangeBind<TUniquePtr<T>>
{
	using Type = UE::TUniquePtrBinding<T>;
};

template <typename T, typename KeyFuncs, typename SetAllocator>
struct TRangeBind<TSet<T, KeyFuncs, SetAllocator>>
{
	using Type = UE::TSetBinding<T, KeyFuncs, SetAllocator>;
};

template <typename T, typename KeyFuncs, typename SetAllocator>
struct TCustomDeltaBind<TSet<T, KeyFuncs, SetAllocator>>
{
	using Type = UE::TSetDeltaBinding<T, KeyFuncs, SetAllocator>;
};

template <typename K, typename V, typename SetAllocator, typename KeyFuncs>
struct TRangeBind<TMap<K, V, SetAllocator, KeyFuncs>>
{
	using Type = UE::TMapBinding<K, V, SetAllocator, KeyFuncs>;
};

template <typename K, typename V, typename SetAllocator, typename KeyFuncs>
struct TCustomDeltaBind<TMap<K, V, SetAllocator, KeyFuncs>>
{
	using Type = UE::TMapDeltaBinding<K, V, SetAllocator, KeyFuncs>;
};

template<typename T>
struct TRangeBind<TOptional<T>>
{
	using Type = UE::TOptionalBinding<T>;
};

template<> struct TOccupancyOf<FQuat> : FRequireAll {};
template<> struct TOccupancyOf<FVector> : FRequireAll {};
template<> struct TOccupancyOf<FIntPoint> : FRequireAll {};
template<> struct TOccupancyOf<FGuid> : FRequireAll {};
template<> struct TOccupancyOf<FColor> : FRequireAll {};
template<> struct TOccupancyOf<FLinearColor> : FRequireAll {};

template<> struct TCustomBind<FTransform> {	using Type = UE::FTransformBinding; };
template<> struct TCustomBind<FGuid> { using Type = UE::FGuidBinding; };
template<> struct TCustomBind<FColor> { using Type = UE::FColorBinding; };
template<> struct TCustomBind<FLinearColor> { using Type = UE::FLinearColorBinding; };


template<typename K, typename V>
struct TOccupancyOf<TPair<K, V>> : FRequireAll {};

template<typename K, typename V> requires (!std::is_default_constructible_v<TPair<K,V>>)
struct TCustomBind<TPair<K, V>>
{
	using Type = UE::TPairBinding<K, V>;
};

template<Enumeration E>
struct TOccupancyOf<TEnumAsByte<E>> : FRequireAll {};

template<Enumeration E> requires (requires { CttiOfPtr((E*)nullptr); })
struct TCustomBind<TEnumAsByte<E>>
{
	using Type = UE::TAuxEnumAsByteBinding<E>;
};

template<typename... Ts>
struct TCustomBind<TVariant<Ts...>>
{
	using Type = UE::TVariantBinding<Ts...>;
};

} // namespace PlainProps
