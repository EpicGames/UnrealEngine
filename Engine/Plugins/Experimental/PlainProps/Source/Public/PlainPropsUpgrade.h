// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsTypes.h"
#include "PlainPropsRead.h" // FBoolRangeIterator
#include "PlainPropsLooseTypes.h"
#include "Containers/BitArray.h"
#include "Misc/Optional.h"
#include "Templates/UniquePtr.h"


namespace PlainProps
{
struct FBuiltStruct;
class FCustomBindings;
class FIdIndexerBase;
class FLiteralIndexerBase;
class FSchemaBatchId;
struct FSchemaBatch;
class FSchemaBindings;
class FScratchAllocator;
struct FStructDeclaration;
struct FStructView;
struct FUpgradeBindings;
class FWriter;
struct IDeclarations;

namespace Upgrade
{
class FChronicler;
class FContext;	
class FEnumChronicler;
struct FFriend;
struct FHistory;
class FIn;
struct FInAny;
class FInEnum;
class FInEnumerator;
class FInFlags;
struct FInMembers;
struct FInStruct;
struct IOps;
class FOut;
class FOutAny;
class FOutBoolItem;
struct FOutEnum;
class FOutEnumItem;
struct FOutEnumerator;
class FOutEnumeratorItem;
struct FOutFlags;
class FOutFlagsItem;
struct FOutMembers;
struct FOutParameter;
struct FOutStruct;
class FUpgrades;
template<typename T, uint8 N> struct TInIt;
template<typename T, uint8 N> struct TOutIt;
template<typename T, uint8 N = 1> struct TOutRange;

using FIns = TConstArrayView<FIn>;
using FOuts = TArrayView<FOut>;
using Op = void (*)(FIns, FOuts, FContext); 
using AddOps = void (*)(IOps& Out); 

////////////////////////////////////////////////////////////////////////////////////////////////

// Mechanism for compiling out old upgrade code and literals
template<Enumeration ReleaseEnum> 
inline constexpr ReleaseEnum OldestRelevant = {};

template<auto Release> 
inline constexpr bool Relevant = Release >= OldestRelevant<decltype(Release)>;

////////////////////////////////////////////////////////////////////////////////////////////////

enum class EAccess : uint8 { WholeValue, AllMembers, SomeMembers }; // todo: could allow leaving taken members too. could have another flag to leave old member

struct FInnermostHandle
{
	FOptionalInnerId	Id;
	uint16				Version;
	uint8				Slice;
	EAccess				Access;
};

struct FParameter
{
	const char*			Name;
	uint32				NameLen;
	uint8				NumRanges;
	bool				bOptional;
	FInnermostType		InnermostType;
	FInnermostHandle	InnermostHandle;
	
	constexpr FParameter operator~() 			{ return {Name, NameLen, NumRanges, true, InnermostType, InnermostHandle }; }
	constexpr FParameter Range(uint8 Num = 1) 	{ return {Name, NameLen, static_cast<uint8>(NumRanges + Num), bOptional, InnermostType, InnermostHandle }; }
};

template<Arithmetic T>
constexpr FParameter MakeParam(const char* Name, std::size_t Len)
{
	return { Name, static_cast<uint32>(Len), 0, false, ReflectInnermost<T>, {} };
}

////////////////////////////////////////////////////////////////////////////////////////////////

inline constexpr FInnermostType InnermostDecomposedStruct	= {.Struct{._ = EInnermostKind::Struct, .IsDecomposed = 1}};
inline constexpr FInnermostType InnermostDecomposedFlatEnum	= {.Enum{._ = EInnermostKind::Enum, .FlagMode = 0, .IsDecomposed = 1}};
inline constexpr FInnermostType InnermostDecomposedFlagEnum	= {.Enum{._ = EInnermostKind::Enum, .FlagMode = 1, .IsDecomposed = 1}};

template<class T>
constexpr FInnermostType IllegalItem()
{
	static_assert(!sizeof(T), "Illegal item type");
	return InnermostStaticStruct;
}

template<class T> inline constexpr FInnermostType ReflectItem				= IllegalItem<T>();
template<Arithmetic T> inline constexpr FInnermostType ReflectItem<T>		= ReflectInnermost<T>;
template<> inline constexpr FInnermostType ReflectItem<FOutBoolItem>		= ReflectInnermost<bool>;
template<> inline constexpr FInnermostType ReflectItem<FInStruct>			= InnermostStaticStruct;
template<> inline constexpr FInnermostType ReflectItem<FOutStruct>			= InnermostStaticStruct;
template<> inline constexpr FInnermostType ReflectItem<FInAny>				= InnermostDynamicStruct;
template<> inline constexpr FInnermostType ReflectItem<FOutAny>				= InnermostDynamicStruct;
template<> inline constexpr FInnermostType ReflectItem<FInMembers>			= InnermostDecomposedStruct;
template<> inline constexpr FInnermostType ReflectItem<FOutMembers>			= InnermostDecomposedStruct;
template<> inline constexpr FInnermostType ReflectItem<FInEnum>				= InnermostFlatEnum; // Note that FlagMode is unknown
template<> inline constexpr FInnermostType ReflectItem<FOutEnumItem>		= InnermostFlatEnum; // Note that FlagMode is unknown
template<> inline constexpr FInnermostType ReflectItem<FInEnumerator>		= InnermostDecomposedFlatEnum;
template<> inline constexpr FInnermostType ReflectItem<FOutEnumeratorItem>	= InnermostDecomposedFlatEnum;
template<> inline constexpr FInnermostType ReflectItem<FInFlags>			= InnermostDecomposedFlagEnum;
template<> inline constexpr FInnermostType ReflectItem<FOutFlagsItem>		= InnermostDecomposedFlagEnum;

////////////////////////////////////////////////////////////////////////////////////////////////

constexpr FParameter operator ""_Bool(const char* Name, std::size_t Len)	{ return MakeParam<bool>(Name, Len); }
constexpr FParameter operator ""_B1(const char* Name, std::size_t Len)		{ return MakeParam<bool>(Name, Len); }
constexpr FParameter operator ""_S8(const char* Name, std::size_t Len)		{ return MakeParam<int8>(Name, Len); }
constexpr FParameter operator ""_U8(const char* Name, std::size_t Len)		{ return MakeParam<uint8>(Name, Len); }
constexpr FParameter operator ""_S16(const char* Name, std::size_t Len)		{ return MakeParam<int16>(Name, Len); }
constexpr FParameter operator ""_U16(const char* Name, std::size_t Len)		{ return MakeParam<uint16>(Name, Len); }
constexpr FParameter operator ""_S32(const char* Name, std::size_t Len)		{ return MakeParam<int32>(Name, Len); }
constexpr FParameter operator ""_U32(const char* Name, std::size_t Len)		{ return MakeParam<uint32>(Name, Len); }
constexpr FParameter operator ""_U64(const char* Name, std::size_t Len)		{ return MakeParam<uint64>(Name, Len); }
constexpr FParameter operator ""_F32(const char* Name, std::size_t Len)		{ return MakeParam<float>(Name, Len); }
constexpr FParameter operator ""_F64(const char* Name, std::size_t Len)		{ return MakeParam<double>(Name, Len); }
constexpr FParameter operator ""_Char(const char* Name, std::size_t Len)	{ return MakeParam<char8_t>(Name, Len); }
constexpr FParameter operator ""_C8(const char* Name, std::size_t Len)		{ return MakeParam<char8_t>(Name, Len); }
constexpr FParameter operator ""_C16(const char* Name, std::size_t Len)		{ return MakeParam<char16_t>(Name, Len); }
constexpr FParameter operator ""_C32(const char* Name, std::size_t Len)		{ return MakeParam<char32_t>(Name, Len); }
constexpr FParameter operator ""_Bools(const char* Name, std::size_t Len)	{ return MakeParam<bool>(Name, Len).Range(); }
constexpr FParameter operator ""_B1s(const char* Name, std::size_t Len)		{ return MakeParam<bool>(Name, Len).Range(); }
constexpr FParameter operator ""_S8s(const char* Name, std::size_t Len)		{ return MakeParam<int8>(Name, Len).Range(); }
constexpr FParameter operator ""_U8s(const char* Name, std::size_t Len)		{ return MakeParam<uint8>(Name, Len).Range(); }
constexpr FParameter operator ""_S16s(const char* Name, std::size_t Len)	{ return MakeParam<int16>(Name, Len).Range(); }
constexpr FParameter operator ""_U16s(const char* Name, std::size_t Len)	{ return MakeParam<uint16>(Name, Len).Range(); }
constexpr FParameter operator ""_S32s(const char* Name, std::size_t Len)	{ return MakeParam<int32>(Name, Len).Range(); }
constexpr FParameter operator ""_U32s(const char* Name, std::size_t Len)	{ return MakeParam<uint32>(Name, Len).Range(); }
constexpr FParameter operator ""_U64s(const char* Name, std::size_t Len)	{ return MakeParam<uint64>(Name, Len).Range(); }
constexpr FParameter operator ""_F32s(const char* Name, std::size_t Len)	{ return MakeParam<float>(Name, Len).Range(); }
constexpr FParameter operator ""_F64s(const char* Name, std::size_t Len)	{ return MakeParam<double>(Name, Len).Range(); }
constexpr FParameter operator ""_Str(const char* Name, std::size_t Len)		{ return MakeParam<char8_t>(Name, Len).Range(); }
constexpr FParameter operator ""_C8s(const char* Name, std::size_t Len)		{ return MakeParam<char8_t>(Name, Len).Range(); }
constexpr FParameter operator ""_C16s(const char* Name, std::size_t Len)	{ return MakeParam<char16_t>(Name, Len).Range(); }
constexpr FParameter operator ""_C32s(const char* Name, std::size_t Len)	{ return MakeParam<char32_t>(Name, Len).Range(); }

////////////////////////////////////////////////////////////////////////////////////////////////

struct FMemberLiteral
{
	const char*			Str;
	std::size_t			Len;

	inline constexpr FParameter MakeParam(uint8 NumRanges, FInnermostType InnermostType, FInnermostHandle InnermostHandle) const
	{
		return { Str, static_cast<uint32>(Len), NumRanges, false, InnermostType, InnermostHandle };
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////

struct FOpaqueEnum
{
	FEnumId			Id;
	bool			bFlagMode;
};

struct FEnumeratorNames
{
	FEnumId			Id;
	uint8			Slice;
};

struct FEnumFlagNames
{
	FEnumId			Id;
	uint8			Slice;
};

struct FEnumLiteral : FMemberLiteral
{
	uint8 NumRanges = 0;

	inline FParameter operator()(FOpaqueEnum Enum) const
	{
		return MakeParam(NumRanges, Enum.bFlagMode ? InnermostFlagEnum : InnermostFlatEnum, {FInnerId(Enum.Id), 0, 0, EAccess::WholeValue});
	}

	inline FParameter operator()(FEnumeratorNames Enumerators, EAccess Access = EAccess::AllMembers) const
	{
		return MakeParam(NumRanges, InnermostDecomposedFlatEnum, {FInnerId(Enumerators.Id), 0, Enumerators.Slice, Access});
	}

	inline FParameter operator()(FEnumFlagNames Flags, EAccess Access = EAccess::AllMembers) const
	{
		return MakeParam(NumRanges, InnermostDecomposedFlagEnum, {FInnerId(Flags.Id), 0, Flags.Slice, Access});
	}
};
inline constexpr FEnumLiteral operator ""_Enum(const char* Name, std::size_t Len)	{ return { {Name, Len}, 0 }; }
inline constexpr FEnumLiteral operator ""_Enums(const char* Name, std::size_t Len)	{ return { {Name, Len}, 1 }; }

////////////////////////////////////////////////////////////////////////////////////////////////

struct FStructMemberNames
{
	FStructId		Id;
	uint8			Slice;
};

struct FStructLiteral : FMemberLiteral
{
	uint8 NumRanges = 0;

	inline FParameter operator()(FStructId Id, uint16 Version = 0) const
	{
		return MakeParam(NumRanges, InnermostStaticStruct, {FInnerId(Id), Version, 0, EAccess::WholeValue});
	}

	inline FParameter operator()(FStructMemberNames Members, uint16 Version = 0, EAccess Access = EAccess::AllMembers) const
	{
		return MakeParam(NumRanges, InnermostDecomposedStruct, {FInnerId(Members.Id), Version, Members.Slice, Access});
	}

	inline FParameter Any() const
	{
		return MakeParam(NumRanges, InnermostDynamicStruct, {NoId, 0, 0, EAccess::WholeValue});
	}
};
inline constexpr FStructLiteral operator ""_Struct(const char* Name, std::size_t Len)	{ return { {Name, Len}, 0 }; }
inline constexpr FStructLiteral operator ""_Structs(const char* Name, std::size_t Len)	{ return { {Name, Len}, 1 }; }

////////////////////////////////////////////////////////////////////////////////////////////////

inline FLooseType ItemOf(FLooseType Type)
{
	check(Type.NumRanges);
	--Type.NumRanges;
	return Type;
}

////////////////////////////////////////////////////////////////////////////////////////////////

// Will allow logging errors/warnings, wont provide any global access to outside world
// Might also be used to allocate new out ranges during op execution
class FContext
{
	friend FFriend;
	FScratchAllocator& Scratch;
public:
	explicit FContext(FScratchAllocator& In) : Scratch(In) {}
};

////////////////////////////////////////////////////////////////////////////////////////////////

struct FItems
{
	FLooseType			Type;
	FLooseNestedRange	Value;
	uint16				NumNamed;
};

template<typename T, uint8 N = 1>
struct TInItems : protected FItems
{
	TInItems(FItems Items) : FItems(Items) { check(Items.Type.NumRanges == N-1); }
	using Iterator = TInIt<T, N>::Type;
	friend TOutRange<T,N>;

	uint64				Num() const							{ return Value.Num; }
	auto&&				operator[](uint64 Idx) const		{ check(Idx < Value.Num); return *Iterator(*this, Idx); }
	Iterator			begin() const						{ return Iterator(*this, 0); }
	Iterator			end() const							{ return Iterator(*this, Value.Num); }
};

struct FOutItems : FItems
{
	FLooseMember& Owner; // Only needed to RecordDynamicType
};

template<typename T, uint8 N = 1>
struct TOutItems : protected FOutItems
{
	TOutItems(FOutItems Items) : FOutItems(Items) {}
	using Iterator = TOutIt<T,N>::Type;
	using DerefType = decltype(**((Iterator*)nullptr));

	uint64				Num() const							{ return Value.Num; }
	DerefType			operator[](uint64 Idx)				{ check(Idx < Value.Num); return *Iterator(*this, Idx); }
	Iterator			begin() const						{ return Iterator(*this, 0); }
	Iterator			end() const							{ return Iterator(*this, Value.Num); }
};

// Helps fill TOutItems and check that all items are assigned
template<typename T, uint8 N = 1>
class TFiller : protected TOutItems<T, N>
{
	using Super = TOutItems<T, N>;
	uint64				Pos = 0;

public:
	TFiller(Super Items) : Super(Items) {}
	~TFiller() { check(Pos == Super::Num()); }
	
	template<typename InT>
	TFiller&			operator<<(InT InItem)
	{
		Super::operator[](Pos++) = InItem;
		return *this;
	}
	
	template<typename InT>
	TFiller&			operator<<(TInItems<InT, N> Items)
	{
		check(Pos + Items.Num() <= Super::Num());
		typename Super::Iterator OutIt(*this, Pos);
		for (auto Item : Items)
		{
			*OutIt = Item;
			++OutIt;
		}
		Pos += Items.Num();
		return *this;
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////

struct FInRange
{
	FLooseType			Type;
	FLooseValue			Value;
	uint16				NumNamed;

	FItems				Items() const { return {ItemOf(Type), {Value.Meta.Num, Value.Data.Range}, NumNamed}; }
};

#if DO_CHECK
PLAINPROPS_API void CheckRangeType(FLooseType Type, uint16 NumNamed, FInnermostType InnermostType, uint8 NumRanges);
#else
inline void CheckRangeType(FLooseType Type, uint16 NumNamed, FInnermostType InnermostType, uint8 NumRanges) {}
#endif


template<typename T, uint8 N = 1>
struct TInRange : protected FInRange
{
	friend TOutRange<T,N>;
	TInRange(FLooseMember I, uint16 NumNamed) : FInRange({I.GetType(), I.GetValue(), NumNamed})	{ CheckRangeType(Type, NumNamed, ReflectItem<T>, N);	}
	uint64							Num() const			{ return Value.Meta.Num; }
	TInItems<T, N>					Items() const		{ return FInRange::Items(); }
};

template<typename T, uint8 N = 1>
struct TOptionalInRange : protected FInRange
{
	friend TOutRange<T,N>;
	TOptionalInRange(FLooseMember I, uint16 NumNamed) : FInRange({I.GetType(), I.GetValue(), NumNamed})	{ CheckRangeType(Type, NumNamed, ReflectItem<T>, N); }
	explicit						operator bool()		{ return Value; }
	uint64							Num() const			{ return Value.Meta.Num; }
	TInItems<T, N>					Items() const		{ check(Value); return FInRange::Items(); }
};

////////////////////////////////////////////////////////////////////////////////////////////////

struct FOutRange
{
	FLooseMember&					Out;
	const uint16					NumNamed;

	FOutRange(FLooseMember& O, uint16 N) : Out(O), NumNamed(N) {}

	PLAINPROPS_API void				Set(const FInRange& In);
	PLAINPROPS_API void				Set(FLooseNestedRange Value);
	PLAINPROPS_API FRangeData		NewNested(uint64 Num, FContext Ctx) const;
	PLAINPROPS_API FRangeData		NewStructs(uint64 Num, FContext Ctx)  const;
	PLAINPROPS_API FRangeData		NewMembers(uint64 Num, FContext Ctx) const;
	PLAINPROPS_API FRangeData		NewEnums(uint64 Num, FContext Ctx) const;
	PLAINPROPS_API FRangeData		NewEnumerators(uint64 Num, FContext Ctx) const;
	PLAINPROPS_API FRangeData		NewFlags(uint64 Num, FContext Ctx) const;
	PLAINPROPS_API FRangeData		NewBools(uint64 Num, FContext Ctx) const;
	PLAINPROPS_API FRangeData		NewArithmetics(uint64 Num, FContext Ctx, uint32 SizeOf) const;
	template<typename T> FRangeData	NewItems(uint64 Num, FContext Ctx) const;
	FOutItems						SetItems(FLooseNestedRange Value)
	{
		Set(Value);
		return {{ItemOf(Out.GetType()), Value, NumNamed}, Out};
	}
};

template<typename T>
FRangeData FOutRange::NewItems(uint64 Num, FContext Ctx) const
{
	if constexpr (Arithmetic<T> && !std::is_same_v<T, bool>)	return NewArithmetics(Num, Ctx, sizeof(T));
	else if constexpr (std::is_same_v<T, FOutBoolItem>)			return NewBools(Num, Ctx);
	else if constexpr (std::is_same_v<T, FOutStruct>)			return NewStructs(Num, Ctx);
	else if constexpr (std::is_same_v<T, FOutAny>)				return NewStructs(Num, Ctx);
	else if constexpr (std::is_same_v<T, FOutMembers>)			return NewMembers(Num, Ctx);
	else if constexpr (std::is_same_v<T, FOutEnumItem>)			return NewEnums(Num, Ctx);
	else if constexpr (std::is_same_v<T, FOutEnumeratorItem>)	return NewEnumerators(Num, Ctx);
	else if constexpr (std::is_same_v<T, FOutFlagsItem>)		return NewFlags(Num, Ctx);
	else static_assert(!sizeof(T), "Illegal item type");
}

template<typename T, uint8 N>
struct TOutRange : protected FOutRange
{
	TOutRange(FLooseMember& O, uint16 NumNamed) : FOutRange(O, NumNamed) { CheckRangeType(Out.GetType(), NumNamed, ReflectItem<T>, N); }
	
	void				operator=(const TInRange<T, N>& In)	{ Set(In); }
	void				operator=(TInItems<T, N> Items)	{ Set(Items); }
	TOutItems<T, N>		New(uint64 Num, FContext Ctx) requires (N>1)	{ return SetItems({Num, NewNested(Num, Ctx)}); }
	TOutItems<T, N>		New(uint64 Num, FContext Ctx) requires (N==1)	{ return SetItems({Num, NewItems<T>(Num, Ctx)}); }
};

////////////////////////////////////////////////////////////////////////////////////////////////

class FIn : protected FLooseMember
{
public:
	using FLooseMember::FLooseMember;
	FIn(const FLooseMember& In) : FLooseMember(In.GetType(), In.GetValue()) {}
	using FLooseMember::operator bool;

	bool												Bool() const							{ return As<bool>(); }
	int32												S32() const								{ return As<int32>(); }
	uint32												U32() const								{ return As<uint32>(); }
	uint64												U64() const								{ return As<uint64>(); }
	float												F32() const								{ return As<float>(); }
	char8_t												Char() const							{ return As<char8_t>(); }
	char8_t												C8() const								{ return As<char8_t>(); }

	PLAINPROPS_API FInStruct							Struct() const;
	PLAINPROPS_API FInAny								StructAny() const;
	PLAINPROPS_API FInMembers							StructMembers() const;

	PLAINPROPS_API FInEnum								Enum() const; 
	PLAINPROPS_API FInEnumerator						Enumerator() const;
	PLAINPROPS_API FInFlags								EnumFlags() const;

	template<typename T, uint8 N = 1>
	TInRange<T, N>										Range(uint16 NumNamed = 0) const		{ return TInRange<T, N>(*this, NumNamed); };
	TInRange<bool>										Bools() const							{ return Range<bool>(); }
	TInRange<uint8>										U8s() const								{ return Range<uint8>(); }
	TInRange<int32>										S32s() const							{ return Range<int32>(); }
	TInRange<float>										F32s() const							{ return Range<float>(); }
	TInRange<char8_t>									C8s() const								{ return Range<char8_t>(); }
	TInRange<char8_t>									Str() const								{ return Range<char8_t>(); }
	TInRange<FInStruct>									Structs() const							{ return Range<FInStruct>(); }
	TInRange<FInMembers>								StructsMembers(uint16 NumNamed) const	{ return Range<FInMembers>(NumNamed); }
	TInRange<FInEnum>									Enums() const							{ return Range<FInEnum>(); }
	TInRange<FInEnumerator>								Enumerators(uint16 NumNamed) const		{ return Range<FInEnumerator>(NumNamed); }
	TInRange<FInFlags>									EnumsFlags(uint16 NumNamed) const		{ return Range<FInFlags>(NumNamed); }

	TOptional<bool>										OptBool() const							{ return AsOpt<bool>(); }
	TOptional<int32>									OptS32() const							{ return AsOpt<int32>(); }
	TOptional<uint32>									OptU32() const							{ return AsOpt<uint32>(); }
	TOptional<uint64>									OptU64() const							{ return AsOpt<uint64>(); }
	TOptional<float>									OptF32() const							{ return AsOpt<float>(); }
	PLAINPROPS_API FInStruct							OptStruct() const;

private:	
	template<Arithmetic T>
	T								As() const
	{
		check(Type.Compare(ReflectLoose<T>));
		check(Value);
		return ArithmeticCast<T>(Value.Data.Arithmetic);
	}

	template<Arithmetic T>
	TOptional<T>					AsOpt() const
	{
		check(Type.Compare(ReflectLoose<T>));
		check(IsOptional());
		return Value ? TOptional<T>(ArithmeticCast<T>(Value.Data.Arithmetic)) : NullOpt;
	}

	friend FFriend;
};

// Opaque struct value, assignable to a type-matching FOutStruct
struct FInStruct : protected FLooseMember
{
	using FLooseMember::FLooseMember;
	friend FOutStruct; friend FOutAny;
};

struct FInAny : protected FLooseMember
{
	using FLooseMember::FLooseMember;
	friend FOutStruct;
};

struct FInMembers : protected FLooseMember
{
	using FLooseMember::FLooseMember;
	friend FIn; friend FOutStruct;

	FIn operator[](uint32 Idx) const { check(Idx < Value.Meta.Num); return FIn(Value.Data.Struct.Loose[Idx]); }
};

// Opaque enum value, assignable to a type-matching FOutEnum
class FInEnum
{
	friend FOutEnum; friend FOutEnumItem;
	FLooseType			Type;
	FEnumeratorData		Value;
public:
	FInEnum(FLooseType T, FEnumeratorData V) : Type(T), Value(V) {}
};

// Enum value represented as enumerator index in NameEnumerators() order
// * Conceptually a string
// * Not assignable to FOutEnumerator, which might use different NameEnumerators() indices
// * Not convertible to FInEnum nor assignable to FOutEnum either
class FInEnumerator
{
	FEnumeratorIndex	Name;
public:
	explicit FInEnumerator(FEnumeratorIndex N, uint16 Max) : Name(N) { check(Name.Index < Max); }
	uint16				GetIndex() const { return Name.Index; }
};

// Bitset of enumerator indices in NameEnumFlags() order
// * Conceptually a string set
// * Not assignable to FOutEnumerator, which might use different NameEnumerators() indices
class FInFlags
{
	FEnumeratorIndexSet	Flags;
	const uint32		Max;
public:
	explicit FInFlags(FEnumeratorIndexSet InFlags, uint32 NumNamedFlags) : Flags(InFlags), Max(NumNamedFlags) {}
	uint64				GetIndices() const;
	bool				IsSet(uint32 Index) const;
};

////////////////////////////////////////////////////////////////////////////////////////////////

template <Arithmetic T>
class TOut
{
	FLooseValue& Out;
public:
	TOut(FLooseType Type, FLooseValue& Value UE_LIFETIMEBOUND)
	: Out(Value)
	{
		check(Type.Compare(ReflectLoose<T>));
	}

	void operator=(T In)
	{
		// todo: normalize floats or check is normalized
		Out.Meta.IsSet = 1;
		Out.Data.Arithmetic = ValueCast(In);
	}

	void operator=(TOptional<T> In)
	{
		Out.Meta.IsSet = !!In;
		Out.Data.Arithmetic =  ValueCast(In.Get(T{}));
	}
};

class FOut : protected FLooseMember
{
public:
	FOut(FOut&&) = default;
	FOut(const FOut&) = delete;
	FOut(FOutParameter Parameter);
	FOut(FOutParameter Parameter, FOut* StructMembers);
	FOut& operator=(FOut&&) = default;
	FOut& operator=(const FOut&) = delete;

	TOut<bool>						Bool()							{ return { Type, Value }; }
	TOut<int32>						S32()							{ return { Type, Value }; }
	TOut<uint32>					U32()							{ return { Type, Value }; }
	TOut<uint64>					U64()							{ return { Type, Value }; }
	TOut<float>						F32()							{ return { Type, Value }; }
	TOut<char8_t>					Char()							{ return { Type, Value }; }
	TOut<char8_t>					C8()							{ return { Type, Value }; }
	
	PLAINPROPS_API FOutStruct		Struct();
	PLAINPROPS_API FOutAny			StructAny();
	PLAINPROPS_API FOutMembers		StructMembers();

	PLAINPROPS_API FOutEnum			Enum();
	PLAINPROPS_API FOutEnumerator	Enumerator();
	PLAINPROPS_API FOutFlags		EnumFlags();

	template<typename T, uint8 N = 1>
	TOutRange<T, N>					Range(uint16 NumNamed = 0)		{ return TOutRange<T, N>(*this, NumNamed); }
	
	TOutRange<FOutBoolItem>			Bools()							{ return Range<FOutBoolItem>(); }
	TOutRange<uint8>				U8s()							{ return Range<uint8>(); }
	TOutRange<int32>				S32s()							{ return Range<int32>(); }
	TOutRange<float>				F32s()							{ return Range<float>(); }
	TOutRange<char8_t>				C8s()							{ return Range<char8_t>(); }
	TOutRange<char8_t>				Str()							{ return Range<char8_t>(); }
	TOutRange<FOutStruct>			Structs()						{ return Range<FOutStruct>(); }
	TOutRange<FOutAny>				StructsAny()					{ return Range<FOutAny>(); }
	TOutRange<FOutMembers>			StructsMembers(uint16 NumNamed) { return Range<FOutMembers>(NumNamed); }
	TOutRange<FOutEnumItem>			Enums()							{ return Range<FOutEnumItem>(); }
	TOutRange<FOutEnumeratorItem>	Enumerators(uint16 NumNamed)	{ return Range<FOutEnumeratorItem>(NumNamed); }
	TOutRange<FOutFlagsItem>		EnumsFlags(uint16 NumNamed)		{ return Range<FOutFlagsItem>(NumNamed); }

	friend FFriend; friend FOutMembers;
};

struct FOutMember
{
	const FLooseType		Type;
	FLooseValue&			Value;
};

struct FOutStruct : protected FOutMember
{
	FOutStruct(FLooseType Type, FLooseValue& Out) : FOutMember{Type, Out} {}
	PLAINPROPS_API void operator=(FInStruct In);
};

class FOutAny : protected FOutMember
{
	FLooseMember&			Owner;
public:
	FOutAny(FLooseType Type, FLooseValue& Out, FLooseMember& InOwner) : FOutMember{Type, Out}, Owner(InOwner) {}
	PLAINPROPS_API void operator=(FInStruct In);
};

struct FOutMembers : protected FOutMember
{
	FOutMembers(FLooseType Type, FLooseValue& Out) : FOutMember{Type, Out} {}
	FOut& operator[](uint32 Idx) { check(Idx < Value.Meta.Num);	return static_cast<FOut&>(Value.Data.Struct.Loose[Idx]); }
};

struct FOutEnum : protected FOutMember
{
	FOutEnum(FLooseType Type, FLooseValue& Out) : FOutMember{Type, Out} {}
	PLAINPROPS_API void operator=(FInEnum In);
};

struct FOutEnumerator : protected FOutMember
{
	FOutEnumerator(FLooseType Type, FLooseValue& Out) : FOutMember{Type, Out} {}
	PLAINPROPS_API void operator=(uint16 Index);
};

class FOutFlag
{
	FLooseValue&			Out;
	const uint32			Idx;
public:
	FOutFlag(FLooseValue& O, uint32 I) : Out(O), Idx(I) { check(Idx < Out.Meta.Num); }
	PLAINPROPS_API void operator=(bool bValue);
	PLAINPROPS_API void operator=(TOptional<bool> bValue);
};

struct FOutFlags : protected FOutMember
{
	FOutFlags(FLooseType Type, FLooseValue& Out) : FOutMember{Type, Out} {}
	// @param Index in NameFlags order
	FOutFlag operator[](uint32 Idx) { return FOutFlag(Value, Idx); }
	// @param Indices bitset in NameFlags order
	PLAINPROPS_API void operator=(uint64 Indices);
};

////////////////////////////////////////////////////////////////////////////////////////////////

class FOutBoolItem
{
	uint8&					Out;
	const uint8				Mask;
public:
	FOutBoolItem(uint8& Byte, uint8 InMask) : Out(Byte), Mask(InMask) {}
	PLAINPROPS_API void operator=(bool bValue);
};

class FOutEnumItem
{
	FEnumeratorData&		Out;
	const FLooseType		Type;
public:
	FOutEnumItem(FLooseType T, FEnumeratorData& D) : Out(D), Type(T) {}
	PLAINPROPS_API void operator=(FInEnum In);
};

class FOutEnumeratorItem
{
	FEnumeratorIndex&		Out;
	const uint32			Max;
public:
	FOutEnumeratorItem(FEnumeratorIndex& Name, uint32 InMax) : Out(Name), Max(InMax) {}
	PLAINPROPS_API void operator=(uint16 Index);
};

class FOutFlagItem
{
	FEnumeratorIndexSet&	Out;
	const uint32			Idx;
public:
	FOutFlagItem(FEnumeratorIndexSet& O, uint32 InIdx) : Out(O), Idx(InIdx) {}
	PLAINPROPS_API void operator=(bool bValue);
};

class FOutFlagsItem
{
	FEnumeratorIndexSet&	Out;
	const uint32			Max;
public:
	FOutFlagsItem(FEnumeratorIndexSet& Flags, uint32 InMax) : Out(Flags), Max(InMax) {}
	// @param Index in NameFlags order
	FOutFlagItem operator[](uint32 Index) { check(Index < Max); return FOutFlagItem(Out, Index); }
	// @param Indices bitset in NameFlags order
	PLAINPROPS_API void operator=(uint64 Indices);
};

////////////////////////////////////////////////////////////////////////////////////////////////

struct FInBoolIt : public FBoolRangeIterator
{
	FInBoolIt(FItems Items, uint64 Idx) : FBoolRangeIterator(static_cast<const uint8*>(Items.Value.Data.Arithmetics.Source), Idx) {}
};

class FOutBoolIt
{
	uint8*				Byte;
	uint8				Mask;
	PLAINPROPS_API void	Increment();
public:
	FOutBoolIt(FItems Items, uint64 Idx)
	: Byte(static_cast<uint8*>(Items.Value.Data.Arithmetics.Loose) + Idx / 8)
	, Mask(1u << (Idx % 8))
	{}

	FOutBoolItem		operator*() const { return FOutBoolItem(*Byte, Mask); }
	bool				operator==(const FOutBoolIt& Rhs) const = default;
	bool				operator!=(const FOutBoolIt& Rhs) const = default;
	FOutBoolIt&			operator++() { Increment(); return *this; }
};

template<Arithmetic T>
class TInArithmeticIt 
{
	const T*			Ptr;
public:
	TInArithmeticIt(FItems Items, uint64 Idx) : Ptr(static_cast<const T*>(Items.Value.Data.Arithmetics.Source) + Idx) {}
	T					operator*()								{ return *Ptr; }
	TInArithmeticIt&	operator++()							{ ++Ptr; return *this; }
	bool				operator!=(TInArithmeticIt O) const		{ return Ptr != O.Ptr; }
};

template<Arithmetic T>
class TOutArithmeticIt
{
	T*					Ptr;
public:
	TOutArithmeticIt(FItems Items, uint64 Idx) : Ptr(static_cast<T*>(Items.Value.Data.Arithmetics.Loose) + Idx) {}
	T&					operator*()								{ return *Ptr; }
	TOutArithmeticIt&	operator++()							{ ++Ptr; return *this; }
	bool				operator!=(TOutArithmeticIt O) const	{ return Ptr != O.Ptr; }
};

template<typename ValueType>
class TItemIt
{
public:
	TItemIt&			operator++()						{ ++It; return *this; }
	bool				operator!=(TItemIt O) const			{ return It != O.It; }
protected:
	TItemIt(FItems Items, ValueType* Value) : Type(Items.Type), NumNamed(Items.NumNamed), It(Value) {}

	const FLooseType	Type;
	uint16				NumNamed;
	ValueType*			It;
};

struct FInStructIt : TItemIt<const FLooseValue>
{
	FInStructIt(FItems Items, uint64 Idx) : TItemIt(Items, Items.Value.Data.Structs.Loose + Idx) { checkSlow(Type.IsStruct() && NumNamed == 0); }
	FInStruct operator*() const { return FInStruct(Type, *It); }
};

struct FOutStructIt : TItemIt<FLooseValue>
{
	FOutStructIt(FItems Items, uint64 Idx) : TItemIt(Items, Items.Value.Data.Structs.Loose + Idx) { checkSlow(Type.IsStruct() && NumNamed == 0); }
	FOutStruct operator*() const { return FOutStruct(Type, *It); }
};

struct FInAnyIt : TItemIt<const FLooseValue>
{
	FInAnyIt(FItems Items, uint64 Idx) : TItemIt(Items, Items.Value.Data.Structs.Loose + Idx) {}
	FInAny operator*() const { return FInAny(Type, *It); }
};

struct FOutAnyIt : TItemIt<FLooseValue>
{
	FOutAnyIt(FOutItems Items, uint64 Idx) : TItemIt(Items, Items.Value.Data.Structs.Loose + Idx), Owner(Items.Owner) { checkSlow(Type.IsStruct() && NumNamed == 0); }
	FOutAny operator*() const { return FOutAny(Type, *It, Owner); }
private:
	FLooseMember& Owner;
};

struct FInMemberIt : TItemIt<const FLooseValue>
{
	FInMemberIt(FItems Items, uint64 Idx) : TItemIt(Items, Items.Value.Data.Structs.Loose + Idx) { checkSlow(Type.IsStruct() && NumNamed > 0); }
	FInMembers operator*() const { return FInMembers(Type, *It); }
};

struct FOutMemberIt : TItemIt<FLooseValue>
{
	FOutMemberIt(FItems Items, uint64 Idx) : TItemIt(Items, Items.Value.Data.Structs.Loose + Idx) { checkSlow(Type.IsStruct() && NumNamed > 0);}
	FOutMembers operator*() const { return FOutMembers(Type, *It); }
};

struct FInEnumIt : TItemIt<const FEnumLooseData>
{
	FInEnumIt(FItems Items, uint64 Idx) : TItemIt(Items, Items.Value.Data.Enums.Loose + Idx) { checkSlow(Type.IsEnum() && NumNamed == 0);}
	FInEnum operator*() const { return FInEnum(Type, It->Enumerator); }
};

struct FOutEnumIt : TItemIt<FEnumLooseData>
{
	FOutEnumIt(FItems Items, uint64 Idx) : TItemIt(Items, Items.Value.Data.Enums.Loose + Idx) { checkSlow(Type.IsEnum() && NumNamed == 0);}
	FOutEnumItem operator*() const { return FOutEnumItem(Type, It->Enumerator); }
};

struct FInEnumeratorIt : TItemIt<const FEnumLooseData>
{
	FInEnumeratorIt(FItems Items, uint64 Idx) : TItemIt(Items, Items.Value.Data.Enums.Loose + Idx) { checkSlow(Type.InnermostType == InnermostDecomposedFlatEnum && !Type.IsRange() && NumNamed > 0); }
	FInEnumerator operator*() const { return FInEnumerator(It->Decomposed.Flat, NumNamed); }
};

struct FOutEnumeratorIt : TItemIt<FEnumLooseData>
{
	FOutEnumeratorIt(FItems Items, uint64 Idx) : TItemIt(Items, Items.Value.Data.Enums.Loose + Idx) { checkSlow(Type.InnermostType == InnermostDecomposedFlatEnum && !Type.IsRange() && NumNamed > 0 && NumNamed < 64); }
	FOutEnumeratorItem operator*() const { return FOutEnumeratorItem(It->Decomposed.Flat, NumNamed); }
};

struct FInFlagsIt : TItemIt<const FEnumLooseData>
{
	FInFlagsIt(FItems Items, uint64 Idx) : TItemIt(Items, Items.Value.Data.Enums.Loose + Idx) { checkSlow(Type.InnermostType == InnermostDecomposedFlagEnum && !Type.IsRange() && NumNamed > 0 && NumNamed < 64); }
	FInFlags operator*() const { return FInFlags(It->Decomposed.Flags, static_cast<uint8>(NumNamed)); }
};

struct FOutFlagsIt : TItemIt<FEnumLooseData>
{
	FOutFlagsIt(FItems Items, uint64 Idx) : TItemIt(Items, Items.Value.Data.Enums.Loose + Idx) { checkSlow(Type.InnermostType == InnermostDecomposedFlagEnum && !Type.IsRange() && NumNamed > 0 && NumNamed < 64); }
	FOutFlagsItem operator*() const { return FOutFlagsItem(It->Decomposed.Flags, NumNamed); }
};

template<typename T, uint8 N>
struct TInNestedRangeIt : TItemIt<const FLooseNestedRange>
{
	TInNestedRangeIt(FItems Items, uint64 Idx) : TItemIt(Items, Items.Value.Data.Ranges + Idx) { checkSlow(Type.NumRanges == N-1); }
	TInItems<T, N-1> operator*() const { return FItems{ItemOf(Type), *It, NumNamed}; }
};

template<typename T, uint8 N>
struct TOutNestedRangeIt : TItemIt<FLooseNestedRange>
{
	TOutNestedRangeIt(FItems Items, uint64 Idx) : TItemIt(Items, Items.Value.Data.Ranges + Idx) { checkSlow(Type.NumRanges == N-1); }
	TOutItems<T, N-1> operator*() const { return FItems{ItemOf(Type), *It, NumNamed}; }
};

template<typename T, uint8 N> requires(N==1) struct TInIt<T,N>	{ static_assert(!sizeof(T), "Illegal in item type"); };
template<typename T, uint8 N> requires(N==1) struct TOutIt<T,N>	{ static_assert(!sizeof(T), "Illegal out item type"); };
template<typename T, uint8 N> requires(N>1) struct TInIt<T,N>	{ using Type = TInNestedRangeIt<T,N>; }; 
template<typename T, uint8 N> requires(N>1) struct TOutIt<T,N> 	{ using Type = TOutNestedRangeIt<T,N>; };
template<Arithmetic T> struct TInIt<T,1>						{ using Type = TInArithmeticIt<T>; };
template<Arithmetic T> struct TOutIt<T,1>						{ using Type = TOutArithmeticIt<T>; };
template<> struct TInIt<bool,1>									{ using Type = FInBoolIt; };
template<> struct TOutIt<FOutBoolItem,1>						{ using Type = FOutBoolIt; };
template<> struct TInIt<FInStruct,1>							{ using Type = FInStructIt; };
template<> struct TOutIt<FOutStruct,1>							{ using Type = FOutStructIt; };
template<> struct TInIt<FInAny,1>								{ using Type = FInAnyIt; };
template<> struct TOutIt<FOutAny,1>								{ using Type = FOutAnyIt; };
template<> struct TInIt<FInMembers,1>							{ using Type = FInMemberIt; };
template<> struct TOutIt<FInMembers,1>							{ using Type = FOutMemberIt; };
template<> struct TInIt<FInEnum,1>								{ using Type = FInEnumIt; };
template<> struct TOutIt<FOutEnumItem,1>						{ using Type = FOutEnumIt; };
template<> struct TInIt<FInEnumerator,1>						{ using Type = FInEnumeratorIt; };
template<> struct TOutIt<FOutEnumeratorItem,1>					{ using Type = FOutEnumeratorIt; };
template<> struct TInIt<FInFlags,1>								{ using Type = FInFlagsIt; };
template<> struct TOutIt<FOutFlagsItem,1>						{ using Type = FOutFlagsIt; };

// Remap TOutItems/TFiller<Xyz> ->  TOutItems/TFiller<XyzItem> to enable e.g. TOutItems<FOutEnum> instead of TOutItems<FOutEnumItem>
template<> struct TOutItems<bool>			: TOutItems<FOutBoolItem>		{ TOutItems(TOutItems<FOutBoolItem> Super) : TOutItems<FOutBoolItem>(Super) {} };
template<> struct TOutItems<FOutEnum>		: TOutItems<FOutEnumItem>		{ TOutItems(TOutItems<FOutEnumItem> Super) : TOutItems<FOutEnumItem>(Super) {} };
template<> struct TOutItems<FOutEnumerator> : TOutItems<FOutEnumeratorItem> { TOutItems(TOutItems<FOutEnumeratorItem> Super) : TOutItems<FOutEnumeratorItem>(Super) {} };
template<> struct TOutItems<FOutFlags>		: TOutItems<FOutFlagsItem>		{ TOutItems(TOutItems<FOutFlagsItem> Super) : TOutItems<FOutFlagsItem>(Super) {} };
template<> struct TFiller<bool>				: TFiller<FOutBoolItem>			{ using TFiller<FOutBoolItem>::TFiller; };
template<> struct TFiller<FOutEnum>			: TFiller<FOutEnumItem>			{ using TFiller<FOutEnumItem>::TFiller; };
template<> struct TFiller<FOutEnumerator>	: TFiller<FOutEnumeratorItem>	{ using TFiller<FOutEnumeratorItem>::TFiller; };
template<> struct TFiller<FOutFlags>		: TFiller<FOutFlagsItem>		{ using TFiller<FOutFlagsItem>::TFiller; };

////////////////////////////////////////////////////////////////////////////////////////////////

// Unindexed typename
struct FTypename
{
	const char*		Scope;
	const char*		Name;
};

// All upgrade ops for a particular version of a particular struct
struct IOps
{
	virtual ~IOps() {}
	
	inline	FOpaqueEnum			NameFlatEnum(FTypename Enum) { return { .Id = NameEnum(Enum), .bFlagMode = false }; }
	inline	FOpaqueEnum			NameFlagEnum(FTypename Enum) { return { .Id = NameEnum(Enum), .bFlagMode = true  }; }
	virtual FEnumId				NameEnum(FTypename Enum) = 0;
	virtual FEnumeratorNames	NameEnumerators(FEnumId Enum, std::initializer_list<const char*> Enumerators) = 0;
	inline  FEnumeratorNames	NameEnumerators(FTypename Enum, std::initializer_list<const char*> Enumerators)		{ return NameEnumerators(NameEnum(Enum), Enumerators); }
	virtual FEnumFlagNames		NameFlags(FEnumId Enum, std::initializer_list<const char*> Flags) = 0;
	inline  FEnumFlagNames		NameFlags(FTypename Enum, std::initializer_list<const char*> Flags)					{ return NameFlags(NameEnum(Enum), Flags); }
	virtual FStructId			NameStruct(FTypename Struct) = 0;
	virtual FStructMemberNames	NameMembers(FStructId Struct, TConstArrayView<FParameter> Members) = 0;
	inline  FStructMemberNames	NameMembers(FTypename Struct, TConstArrayView<FParameter> Members)					{ return NameMembers(NameStruct(Struct), Members); }

	virtual void				Add(TConstArrayView<FParameter> Ins, TConstArrayView<FParameter> Outs, Op Upgrade) = 0;

	template<uint32 N, uint32 M>
	void						Add(FParameter (&&Ins)[N], FParameter (&&Outs)[M], Op Upgrade)
	{
		Add(MakeArrayView(Ins), MakeArrayView(Outs), Upgrade);
	}
	
	// Only one version bump or rename is allowed and must happen last
	virtual void				Bump(uint16 NextVersion) = 0;

	// Only one version bump or rename is allowed and must happen last
	virtual void				Rename(FTypename Redirect, uint16 ToVersion) = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////

class FChronicles
{
	FHistory&					History;
	FStructId					Owner;
	uint16						PriorVersion = 0xFFFF;
	bool						bFinalVersion = false;

public:
	FChronicles(FHistory& Out, FStructId InOwner) : History(Out), Owner(InOwner) {}

	PLAINPROPS_API IOps*	CreateVersion(FTypename Name, uint16 Version);
	PLAINPROPS_API void		CommitVersion(IOps* Version);
};

// Records relevant ops for a particular struct version in its destructor
// Helps compile out irrelevant literals and ops
class FChronicler
{
public:
	FORCEINLINE FChronicler(FChronicles Out, const char* Scope, const char* Name, uint16 Ver = 0) : Name{Scope, Name}, Version(Ver), Chronicles(Out) {}
	FORCEINLINE ~FChronicler()
	{
		if (Out)
		{
			Chronicles.CommitVersion(/* move */ Out);
		}	
	}
	
	template<auto Release>
	FORCEINLINE void In(AddOps Add)
	{
		if constexpr (Relevant<Release>)
		{
			Out = Out ? Out : Chronicles.CreateVersion(Name, Version);
			Add(/* out */ *Out);
		}
	}

	template<auto Release>
	FORCEINLINE void Rename(FTypename To, uint16 ToVersion = 0)
	{
		if constexpr (Relevant<Release>)
		{
			Out = Out ? Out : Chronicles.CreateVersion(Name, Version);
			Out->Rename(To, ToVersion);
		}
	}

	FTypename GetName() const { return Name; }

private:
	const FTypename		Name;
	const uint16		Version;
	FChronicles			Chronicles;
	IOps*				Out = nullptr;
};

// todo
class FEnumChronicler
{};

////////////////////////////////////////////////////////////////////////////////////////////////

// Enables overloading void Record(TChroniclesOf<FMyStruct> Out) 
template <typename T>
class TChroniclesOf : public FChronicles {};

template<typename T>
void Record(TChroniclesOf<T>) {}

PLAINPROPS_API void EraseChroniclesOf(FHistory& History, FStructId Owner);

////////////////////////////////////////////////////////////////////////////////////////////////

struct FHistoryDeleter
{
	PLAINPROPS_API void operator()(FHistory* Ptr) const;
};
using FHistoryPtr = TUniquePtr<FHistory, FHistoryDeleter>;

PLAINPROPS_API FHistoryPtr			MakeHistory(IDeclarations& Latest, FLiteralIndexerBase& Ids);
PLAINPROPS_API FMemberId			ResolveName(const FLooseMember& Member, const FHistory& History);
PLAINPROPS_API FNameId				ResolveEnumerator(const FLooseMember& FlatEnum, const FHistory& History);
PLAINPROPS_API TArray<FNameId>		ResolveFlags(const FLooseMember& FlagEnum, const FHistory& History);

////////////////////////////////////////////////////////////////////////////////////////////////

struct FUpgradedInstance
{
	FDeclId										Type;
	TConstArrayView<FLooseMember>				Loose; // Owned by scratch, empty if no upgraders ran
	const FBuiltStruct*							Fixed; // Owned by scratch, null if Loose has any mismatches
};

class FBatchUpgrader
{
public:
	PLAINPROPS_API FBatchUpgrader(const FSchemaBatch& InBatch UE_LIFETIMEBOUND, TConstArrayView<FInnerId> InRuntimeIds UE_LIFETIMEBOUND, FIdIndexerBase& Ids);

	// @param Types must return declarations that outlive this
	PLAINPROPS_API bool							MatchSchemas(const IDeclarations& Types);

	// todo: add unknown members api and error handler
	PLAINPROPS_API TArray<FUpgradedInstance>	Upgrade(TConstArrayView<FStructView> Structs, FHistory& Upgrades, FScratchAllocator& Scratch);
private:
	const FSchemaBatch&							Batch;
	TConstArrayView<FInnerId> 					RuntimeIds;
	TBitArray<>									MatchingSchemas;
	TArray<const FStructDeclaration*>			Declarations;
};

}} // namespace PlainProps::Upgrade