// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsBind.h"

namespace PlainProps 
{

// Iterates over member bindings
class FMemberVisitor
{
public:
	PLAINPROPS_API explicit FMemberVisitor(const FSchemaBinding& InSchema);

	bool									HasMore() const			{ return MemberIdx < NumMembers; }
	uint16									GetIndex() const		{ return MemberIdx; }
	
	PLAINPROPS_API EMemberKind				PeekKind() const;		// @pre HasMore()
	PLAINPROPS_API FMemberBindType			PeekType() const;		// @pre HasMore()
	PLAINPROPS_API uint32					PeekOffset() const;		// @pre HasMore()

	PLAINPROPS_API FLeafMemberBinding		GrabLeaf();				// @pre PeekKind() == EMemberKind::Leaf
	PLAINPROPS_API FRangeMemberBinding		GrabRange();			// @pre PeekKind() == EMemberKind::Range
	PLAINPROPS_API FStructMemberBinding		GrabStruct();			// @pre PeekKind() == EMemberKind::Struct
	PLAINPROPS_API FBindId					GrabSuper();			// @pre First grab and has declared super
	PLAINPROPS_API void						SkipMember();

protected: // for unit tests
	const FSchemaBinding&		Schema;
	const uint16				NumMembers;
	uint16						MemberIdx = 0;
	uint16						InnerRangeIdx = 0;		// Types of [nested] ranges
	uint16						InnerIdIdx = 0;			// Types of static structs and enums

	using FMemberBindTypeRange = TConstArrayView<FMemberBindType>;

	PLAINPROPS_API uint64					GrabMemberOffset();
	PLAINPROPS_API FMemberBindTypeRange		GrabInnerTypes();
	PLAINPROPS_API FInnerId					GrabInnerId();
	PLAINPROPS_API FBindId					GrabStructSchema(FStructType Type);
	PLAINPROPS_API FOptionalInnerId			GrabRangeSchema(FMemberType InnermostType);
	PLAINPROPS_API FEnumId					GrabEnumSchema()		{ return GrabInnerId().AsEnum(); }
};

} // namespace PlainProps