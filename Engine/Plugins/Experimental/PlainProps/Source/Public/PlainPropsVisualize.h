// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsBuild.h"
#include "PlainPropsIndex.h"
#include "PlainPropsRead.h"
#include "Containers/Set.h"

namespace PlainProps
{

// Mirror publically exposed forward-declared internal structs for full visualizer support in the context of other modules.
namespace DbgVis
{

// InternalBuild.h

struct FBuiltStruct
{
	uint16 NumMembers;
	FBuiltMember Members[0];
};

struct FBuiltRange
{
	uint64 Num;
	uint8 Data[0];
};


// InternalFormat.h

struct FSchemaBatch
{
	uint32 NumNestedScopes;
	uint32 NestedScopesOffset;
	uint32 NumParametricTypes;
	uint32 NumSchemas;
	uint32 NumStructSchemas;
	uint32 SchemaOffsets[0];
};

enum class ESuper : uint8 { No, Unused, Used, Reused };

struct FStructSchema
{
	FType Type;
	uint16 Version;
	uint16 NumMembers;
	uint16 NumRangeTypes;
	uint16 NumInnerSchemas;
	ESuper Inheritance : 2;
	uint8 IsDense : 1;
	FMemberType Footer[0];
};

struct FEnumSchema
{
	FType Type;
	uint8 FlagMode : 1;
	uint8 ExplicitConstants : 1;
	ELeafWidth Width;
	uint16 Num;
	FNameId Footer[0];
};


// Load.cpp

struct FLoadStructPlan
{
	uint64					Handle;
};

struct FLateLoadPlans
{
	FLateLoadPlans* Next;
	uint8 Data[0];
};

struct FLoadBatch
{
	FSchemaBatchId			BatchId;
	uint32					NumReadSchemas;
	uint32					NumPlans;
	uint32					MaxPlans;
	FLateLoadPlans*			LatePlans;
	FLoadStructPlan			Plans[0];
};

////////////////////////////////////////////////////////////////////////////////

struct FIdVisualizer
{
	const FIdIndexerBase* Indexer;
	const char* NameType;

	static void KeepDebugInfo(DbgVis::FBuiltStruct*) {};
	static void KeepDebugInfo(DbgVis::FBuiltRange*) {};
	static void KeepDebugInfo(DbgVis::ESuper) {};
	static void KeepDebugInfo(DbgVis::FSchemaBatch*) {};
	static void KeepDebugInfo(DbgVis::FStructSchema*) {};
	static void KeepDebugInfo(DbgVis::FEnumSchema*) {};
	static void KeepDebugInfo(DbgVis::FLoadStructPlan*) {};
	static void KeepDebugInfo(DbgVis::FLoadBatch*) {};
};

class FIdScope
{
public:
	// NameType must be a unique static string identifer for NameT,
	// it is used by natvis in a strncmp(str1,str2,N) expression to select the correct typed name indexer, N <= 8
	template<typename NameT>
	FIdScope(const TIdIndexer<NameT>& Indexer, const char NameType[8] = nullptr)
	: Current{ static_cast<const FIdIndexerBase*>(&Indexer), &NameType[0] }
	, Previous(Global)
	{
		AssignDebuggingState();
	}

	PLAINPROPS_API ~FIdScope();

private:
	PLAINPROPS_API void AssignDebuggingState();

	inline static FIdVisualizer*	Global = nullptr;
	FIdVisualizer					Current;
	FIdVisualizer*					Previous;
};

void AssignReadSchemasDebuggingState(FSchemaBatch** Slots);

} // namespace DbgVis

} // namespace PlainProps
