// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsTypes.h"
#include "Concepts/ContiguousRange.h"
#include "Containers/Array.h"
#include "Templates/UniquePtr.h"
#include <type_traits>

namespace PlainProps
{

struct FBuiltSchemas;
struct FBuiltStruct;
class FIdIndexerBase;
class FNestedScopeIndexer;
class FParametricTypeIndexer;
struct FWriteIds;
struct IDeclarations;

class FWriter
{
public:
	PLAINPROPS_API FWriter(const FIdIndexerBase& AllIds, const IDeclarations& Declarations, const FBuiltSchemas& InSchemas, ESchemaFormat Format);
	PLAINPROPS_API ~FWriter();
	
	PLAINPROPS_API TConstArrayView<FNameId>			GetUsedNames() const;
	PLAINPROPS_API FOptionalStructSchemaId			GetWriteId(FStructId BuiltId) const;

	PLAINPROPS_API void								WriteSchemas(TArray64<uint8>& Out) const;
	PLAINPROPS_API FStructSchemaId					WriteMembers(TArray64<uint8>& Out, FStructId BuiltId, const FBuiltStruct& Struct) const;

private:
	const FBuiltSchemas&							Schemas;
	FDebugIds										Debug;
	TUniquePtr<FWriteIds>							NewIds;
};

//////////////////////////////////////////////////////////////////////////

inline void WriteData(TArray64<uint8>& Out, const void* Data, int64 Size)
{
	Out.Append(static_cast<const uint8*>(Data), Size);
}

template<::UE::CContiguousRange ArrayType>
void WriteArray(TArray64<uint8>& Out, const ArrayType& In)
{
	WriteData(Out, In.GetData(), sizeof(typename ArrayType::ElementType) * In.Num());
}

template<typename T>
void WriteAlignmentPadding(TArray64<uint8>& Out)
{
	Out.AddZeroed(Align(Out.Num(), alignof(T)) - Out.Num());
}	

template<typename T>
void WriteAlignedArray(TArray64<uint8>& Out, TArrayView<T> In)
{
	WriteAlignmentPadding<T>(Out);
	WriteArray(Out, In);
}

template<typename T>
inline void WriteInt(TArray64<uint8>& Out, T Number) requires (std::is_integral_v<T> && !!PLATFORM_LITTLE_ENDIAN)
{
	WriteData(Out, &Number, sizeof(T));
}

PLAINPROPS_API uint64 WriteSkippableSlice(TArray64<uint8>& Out, TConstArrayView64<uint8> Slice);


} // namespace PlainProps
