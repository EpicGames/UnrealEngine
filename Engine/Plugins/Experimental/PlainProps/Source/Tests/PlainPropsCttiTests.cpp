// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsCtti.h"
#include <type_traits>
#include <string_view>

namespace PlainProps::Test
{

enum class F1 : uint8_t { None, A = 2, B = 1 << 7 };

enum U1 : uint8_t { U1_A, U1_B };

enum class E1 : uint16_t { A, B };

enum class E2 : int16_t
{
	C0, C1, C2, C3, C4, C5, C6, C7, C8, C9, C10, C11, C12, C13, C14, C15, C16, C17, C18, C19,
	C20, C21, C22, C23, C24, C25, C26, C27, C28, C29, C30, C31, C32, C33, C34, C35, C36, C37, C38, C39,
	C40, C41, C42, C43, C44, C45, C46, C47, C48, C49, C50, C51, C52, C53, C54, C55, C56, C57, C58, C59,
	C60, C61, C62, C63, C64, C65, C66, C67, C68, C69, C70, C71, C72, C73, C74, C75, C76, C77, C78, C79,
	C80, C81, C82, C83, C84, C85, C86, C87, C88, C89, C90, C91, C92, C93, C94, C95, C96, C97, C98, C99,
};


namespace Actual
{
	PP_REFLECT_FLAG_ENUM(PlainProps::Test, F1, None, A, B);
	PP_REFLECT_ENUM(PlainProps::Test, U1, U1_A, U1_B);
	PP_REFLECT_ENUM(PlainProps::Test, E1, A, B);
	PP_REFLECT_ENUM(PlainProps::Test, E2,
		C0, C1, C2, C3, C4, C5, C6, C7, C8, C9, C10, C11, C12, C13, C14, C15, C16, C17, C18, C19,
		C20, C21, C22, C23, C24, C25, C26, C27, C28, C29, C30, C31, C32, C33, C34, C35, C36, C37, C38, C39,
		C40, C41, C42, C43, C44, C45, C46, C47, C48, C49, C50, C51, C52, C53, C54, C55, C56, C57, C58, C59,
		C60, C61, C62, C63, C64, C65, C66, C67, C68, C69, C70, C71, C72, C73, C74, C75, C76, C77, C78, C79,
		C80, C81, C82, C83, C84, C85, C86, C87, C88, C89, C90, C91, C92, C93, C94, C95, C96, C97, C98, C99);
}

namespace Expect
{
	struct F1_Ctti
	{
		static constexpr char Namespace[] = "PlainProps::Test";
		static constexpr char Name[] = "F1";
		using Type = ::PlainProps::Test::F1;
		using enum Type;
		static constexpr uint32 NumEnumerators = 3;
		inline static constexpr std::array<std::string_view, NumEnumerators> Names = { "None", "A", "B" };
		static constexpr Type Constants[NumEnumerators] = { None, A, B };
	};

	struct U1_Ctti
	{
		static constexpr char Namespace[] = "PlainProps::Test";
		static constexpr char Name[] = "U1";
		using Type = ::PlainProps::Test::U1;
		static constexpr uint32 NumEnumerators = 2;
		inline static constexpr std::array<std::string_view, NumEnumerators> Names = { "U1_A", "U1_B"};
		static constexpr Type Constants[NumEnumerators] = { U1_A, U1_B };
	};

	struct E1_Ctti
	{
		static constexpr char Namespace[] = "PlainProps::Test";
		static constexpr char Name[] = "E1";
		using Type = ::PlainProps::Test::E1;
		using enum Type;
		static constexpr uint32 NumEnumerators = 2;
		inline static constexpr std::array<std::string_view, NumEnumerators> Names = { "A", "B" };
		static constexpr Type Constants[NumEnumerators] = { A, B };
	};

	struct E2_Ctti
	{
		static constexpr char Namespace[] = "PlainProps::Test";
		static constexpr char Name[] = "E2";
		using Type = ::PlainProps::Test::E2;
		using enum Type;
		static constexpr uint32 NumEnumerators = 100;
		inline static constexpr std::array<std::string_view, NumEnumerators> Names =
		{
			"C0","C1","C2","C3","C4","C5","C6","C7","C8","C9",
			"C10","C11","C12","C13","C14","C15","C16","C17","C18","C19",
			"C20","C21","C22","C23","C24","C25","C26","C27","C28","C29",
			"C30","C31","C32","C33","C34","C35","C36","C37","C38","C39",
			"C40","C41","C42","C43","C44","C45","C46","C47","C48","C49",
			"C50","C51","C52","C53","C54","C55","C56","C57","C58","C59",
			"C60","C61","C62","C63","C64","C65","C66","C67","C68","C69",
			"C70","C71","C72","C73","C74","C75","C76","C77","C78","C79",
			"C80","C81","C82","C83","C84","C85","C86","C87","C88","C89",
			"C90","C91","C92","C93","C94","C95","C96","C97","C98","C99"
		};
		static constexpr Type Constants[NumEnumerators] =
		{
			C0, C1, C2, C3, C4, C5, C6, C7, C8, C9,
			C10, C11, C12, C13, C14, C15, C16, C17, C18, C19,
			C20, C21, C22, C23, C24, C25, C26, C27, C28, C29,
			C30, C31, C32, C33, C34, C35, C36, C37, C38, C39,
			C40, C41, C42, C43, C44, C45, C46, C47, C48, C49,
			C50, C51, C52, C53, C54, C55, C56, C57, C58, C59,
			C60, C61, C62, C63, C64, C65, C66, C67, C68, C69,
			C70, C71, C72, C73, C74, C75, C76, C77, C78, C79,
			C80, C81, C82, C83, C84, C85, C86, C87, C88, C89,
			C90, C91, C92, C93, C94, C95, C96, C97, C98, C99,
		};
	};
}

static_assert(std::string_view(Actual::F1_Ctti::Name) == std::string_view(Expect::F1_Ctti::Name));
static_assert(std::is_same_v<Actual::F1_Ctti::Type, Expect::F1_Ctti::Type>);
static_assert(Actual::F1_Ctti::NumEnumerators == Expect::F1_Ctti::NumEnumerators);
static_assert(Actual::F1_Ctti::Constants[0]	== Expect::F1_Ctti::Constants[0]);
static_assert(Actual::F1_Ctti::Constants[1]	== Expect::F1_Ctti::Constants[1]);
static_assert(Actual::F1_Ctti::Constants[2]	== Expect::F1_Ctti::Constants[2]);
static_assert(std::string_view(Actual::F1_Ctti::Names[0]) == std::string_view(Expect::F1_Ctti::Names[0]));
static_assert(std::string_view(Actual::F1_Ctti::Names[1]) == std::string_view(Expect::F1_Ctti::Names[1]));
static_assert(std::string_view(Actual::F1_Ctti::Names[2]) == std::string_view(Expect::F1_Ctti::Names[2]));

static_assert(std::string_view(Actual::U1_Ctti::Name) == std::string_view(Expect::U1_Ctti::Name));
static_assert(std::is_same_v<Actual::U1_Ctti::Type, Expect::U1_Ctti::Type>);
static_assert(Actual::U1_Ctti::NumEnumerators == Expect::U1_Ctti::NumEnumerators);
static_assert(Actual::U1_Ctti::Constants[0]	== Expect::U1_Ctti::Constants[0]);
static_assert(Actual::U1_Ctti::Constants[1]	== Expect::U1_Ctti::Constants[1]);
static_assert(std::string_view(Actual::U1_Ctti::Names[0]) == std::string_view(Expect::U1_Ctti::Names[0]));
static_assert(std::string_view(Actual::U1_Ctti::Names[1]) == std::string_view(Expect::U1_Ctti::Names[1]));

static_assert(std::string_view(Actual::E1_Ctti::Name) == std::string_view(Expect::E1_Ctti::Name));
static_assert(std::is_same_v<Actual::E1_Ctti::Type, Expect::E1_Ctti::Type>);
static_assert(Actual::E1_Ctti::NumEnumerators == Expect::E1_Ctti::NumEnumerators);
static_assert(Actual::E1_Ctti::Constants[0]	== Expect::E1_Ctti::Constants[0]);
static_assert(Actual::E1_Ctti::Constants[1]	== Expect::E1_Ctti::Constants[1]);
static_assert(std::string_view(Actual::E1_Ctti::Names[0]) == std::string_view(Expect::E1_Ctti::Names[0]));
static_assert(std::string_view(Actual::E1_Ctti::Names[1]) == std::string_view(Expect::E1_Ctti::Names[1]));

static_assert(std::string_view(Actual::E2_Ctti::Name) == std::string_view(Expect::E2_Ctti::Name));
static_assert(std::is_same_v<Actual::E2_Ctti::Type, Expect::E2_Ctti::Type>);
static_assert(Actual::E2_Ctti::NumEnumerators == Expect::E2_Ctti::NumEnumerators);
static_assert(Actual::E2_Ctti::Constants[0]	== Expect::E2_Ctti::Constants[0]);
static_assert(Actual::E2_Ctti::Constants[1]	== Expect::E2_Ctti::Constants[1]);
static_assert(Actual::E2_Ctti::Constants[Actual::E2_Ctti::NumEnumerators-2]	== Expect::E2_Ctti::Constants[Expect::E2_Ctti::NumEnumerators-2]);
static_assert(Actual::E2_Ctti::Constants[Actual::E2_Ctti::NumEnumerators-1]	== Expect::E2_Ctti::Constants[Expect::E2_Ctti::NumEnumerators-1]);
static_assert(std::string_view(Actual::E2_Ctti::Names[0]) == std::string_view(Expect::E2_Ctti::Names[0]));
static_assert(std::string_view(Actual::E2_Ctti::Names[1]) == std::string_view(Expect::E2_Ctti::Names[1]));
static_assert(std::string_view(Actual::E2_Ctti::Names[Actual::E2_Ctti::NumEnumerators-2]) == std::string_view(Expect::E2_Ctti::Names[Expect::E2_Ctti::NumEnumerators-2]));
static_assert(std::string_view(Actual::E2_Ctti::Names[Actual::E2_Ctti::NumEnumerators-1]) == std::string_view(Expect::E2_Ctti::Names[Expect::E2_Ctti::NumEnumerators-1]));

//////////////////////////////////////////////////////////////////////////

struct S1
{
	float x;
	int y;
};

namespace Actual
{	
	PP_REFLECT_STRUCT(PlainProps::Test, S1, void, x, y);
}

namespace Expect
{
	struct S1_Ctti
	{
		static constexpr char Name[] = "S1";
		using Type = ::PlainProps::Test::S1;
		using Super = void;
		static constexpr int NumVars = 2;
		template<int> struct Var;
	};

	template<> struct S1_Ctti::Var<2-2>
	{
		static constexpr char Name[] = "x";
		using Type = decltype(::PlainProps::Test::S1::x);
		static constexpr auto Pointer = &::PlainProps::Test::S1::x;
		static constexpr std::size_t Offset = offsetof(::PlainProps::Test::S1, x);
		static constexpr int Index = 2-2;
	};

	template<> struct S1_Ctti::Var<2-1>
	{
		static constexpr char Name[] = "y";
		using Type = decltype(::PlainProps::Test::S1::y);
		static constexpr auto Pointer = &::PlainProps::Test::S1::y;
		static constexpr std::size_t Offset = offsetof(::PlainProps::Test::S1, y);
		static constexpr int Index = 2-1;
	};
}

template<class Actual, class Expect>
static constexpr bool AssertVarEquivalence()
{
	static_assert(std::string_view(Actual::Name) == std::string_view(Expect::Name));
	static_assert(std::is_same_v<typename Actual::Type, typename Expect::Type>);
	static_assert(Actual::Offset == Expect::Offset);
	static_assert(Actual::Pointer == Expect::Pointer);
	static_assert(Actual::Index == Expect::Index);
	return true;
}

static_assert(std::string_view(Actual::S1_Ctti::Name) == std::string_view(Expect::S1_Ctti::Name));
static_assert(std::is_same_v<Actual::S1_Ctti::Type, Expect::S1_Ctti::Type>);
static_assert(std::is_same_v<Actual::S1_Ctti::Super, Expect::S1_Ctti::Super>);
static_assert(Actual::S1_Ctti::NumVars == Expect::S1_Ctti::NumVars);
static_assert(AssertVarEquivalence<Actual::S1_Ctti::Var<0>, Expect::S1_Ctti::Var<0>>());
static_assert(AssertVarEquivalence<Actual::S1_Ctti::Var<1>, Expect::S1_Ctti::Var<1>>());

// CttiOf only works when CTTI exists in same or parent namespace, it uses argument dependent lookup
// to find the "canonical" CTTI if exists. Regenerate S1_Ctti in S1's namespace to test it.
PP_REFLECT_STRUCT(PlainProps::Test, S1, void, x, y);
static_assert(std::is_same_v<CttiOf<S1>, S1_Ctti>);

//////////////////////////////////////////////////////////////////////////

template<class T>
struct S2
{
	bool _; // unreflected
	T a;
};

PP_REFLECT_STRUCT_TEMPLATE(PlainProps::Test, S2, void, a);

static_assert(std::string_view(CttiOf<S2<int>>::Name) == std::string_view("S2"));
static_assert(std::is_same_v<CttiOf<S2<int>>::Type, S2<int>>);
static_assert(std::is_same_v<CttiOf<S2<int>>::TemplateArgs, std::tuple<int>>);
static_assert(CttiOf<S2<int>>::NumVars == 1);
static_assert(CttiOf<S2<int>>::Var<0>::Name == std::string_view("a"));
static_assert(CttiOf<S2<int>>::Var<0>::Offset == offsetof(S2<int>, a));

}
