// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsStringUtil.h"

namespace PlainProps::Private
{
	using namespace std::literals;

	// Concat tests
	static constexpr std::string_view Hello = "Hello";
	static constexpr std::string_view Space = " ";
	static constexpr std::string_view World = "World";
	static_assert("Hello World"sv == Concat<Hello, Space, World>);
	
	// HexString tests
	static_assert("0"sv == HexString<0>);
	static_assert("1"sv == HexString<1>);
	static_assert("9"sv == HexString<9>);
	static_assert("A"sv == HexString<0xA>);
	static_assert("F"sv == HexString<0xF>);
	static_assert("10"sv == HexString<0x10>);
	static_assert("FF"sv == HexString<0xFF>);
	static_assert("100"sv == HexString<0x100>);
	static_assert("FFF"sv == HexString<0xFFF>);
	static_assert("1000"sv == HexString<0x1000>);
	static_assert("FEDCBA9876543210"sv == HexString<0xFEDCBA9876543210>);

	// ParseArgs tests (separated by comma and whitespace)
	static constexpr std::string_view Many = "A, B,  C,D  ";
	static constexpr std::string_view One = "\tOne";
	static constexpr std::string_view Two = "One,\nTwo";
	static constexpr std::string_view Whitespace = " \v\f\t\r\n";
	static constexpr std::string_view Empty = "";
	static constexpr std::array<std::string_view, 4> Args = ParseArgs<Many>;
	static_assert(Args.size() == 4);
	static_assert("A"sv == Args[0]);
	static_assert("B"sv == Args[1]);
	static_assert("C"sv == Args[2]);
	static_assert("D"sv == Args[3]);
	static_assert(ParseArgs<One>.size() == 1);
	static_assert(ParseArgs<One>[0] == "One"sv);
	static_assert(ParseArgs<Two>[0] == "One"sv);
	static_assert(ParseArgs<Two>[1] == "Two"sv);
	static_assert(ParseArgs<Whitespace>.size() == 0);
	static_assert(ParseArgs<Empty>.size() == 0);
}
