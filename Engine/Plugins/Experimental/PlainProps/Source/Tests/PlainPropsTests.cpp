// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS
#include "Algo/Compare.h"
#include "Containers/AnsiString.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/StaticArray.h"
#include "Containers/StringConv.h"
#include "Containers/StringView.h"
#include "Logging/LogMacros.h"
#include "PlainPropsBuildSchema.h"
#include "PlainPropsDiff.h"
#include "PlainPropsIndex.h"
#include "PlainPropsInternalBuild.h"
#include "PlainPropsInternalBuild.h"
#include "PlainPropsInternalFormat.h"
#include "PlainPropsInternalParse.h"
#include "PlainPropsInternalPrint.h"
#include "PlainPropsInternalRead.h"
#include "PlainPropsRead.h"
#include "PlainPropsSpecify.h"
#include "PlainPropsVisualize.h"
#include "PlainPropsWrite.h"
#include "Templates/UnrealTemplate.h"
#include "Tests/TestHarnessAdapter.h"

DEFINE_LOG_CATEGORY_STATIC(LogPlainPropsTests, Log, All);

namespace PlainProps
{
	
static bool operator==(FScopeId A, FNestedScopeId B) { return A == FScopeId(B); }
static bool operator==(FScopeId A, FFlatScopeId B) { return A == FScopeId(B); }
static bool operator==(FParametricTypeView A, FParametricTypeView B)
{
	return A.Name == B.Name && A.NumParameters == B.NumParameters && !FMemory::Memcmp(A.Parameters, B.Parameters, A.NumParameters * sizeof(FType));
}

////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FPlainPropsIndexTest, "System::Core::Serialization::PlainProps::Index", "[Core][PlainProps][SmokeFilter]")
{
	SECTION("NestedScope")
	{
		FScopeId S0{FFlatScopeId{{0}}};
		FScopeId S1{FFlatScopeId{{1}}};
		FScopeId S2{FFlatScopeId{{2}}};

		FNestedScope N01{S0, S1.AsFlat()};
		FNestedScope N10{S1, S0.AsFlat()};
		FNestedScope N12{S1, S2.AsFlat()};

		FNestedScopeIndexer Indexer;

		FScopeId S01(Indexer.Index(N01));
		FScopeId S10(Indexer.Index(N10));
		FScopeId S12(Indexer.Index(N12));

		FNestedScope N012{S01, S2.AsFlat()};
		FScopeId S012(Indexer.Index(N012));

		FNestedScope N0120{S012, S0.AsFlat()};
		FScopeId S0120(Indexer.Index(N0120));
		
		CHECK(S01	== Indexer.Index(N01));
		CHECK(S10	== Indexer.Index(N10));
		CHECK(S12	== Indexer.Index(N12));
		CHECK(S012	== Indexer.Index(N012));
		CHECK(S0120 == Indexer.Index(N0120));
		CHECK(N01	== Indexer.Resolve(S01.AsNested()));
		CHECK(N10	== Indexer.Resolve(S10.AsNested()));
		CHECK(N12	== Indexer.Resolve(S12.AsNested()));
		CHECK(N012	== Indexer.Resolve(S012.AsNested()));
		CHECK(N0120 == Indexer.Resolve(S0120.AsNested()));
		CHECK(Indexer.Num() == 5);
	}
	
	SECTION("ParametricType")
	{
		FScopeId S0{FFlatScopeId{{0}}};
		FScopeId S1{FFlatScopeId{{1}}};
		FScopeId S2{FFlatScopeId{{2}}};
		
		FConcreteTypenameId T3{{3}};
		FConcreteTypenameId T4{{4}};
		FConcreteTypenameId T5{{5}};
		
		FType S0T3 = {S0, FTypenameId{T3}};
		FType S1T3 = {S1, FTypenameId{T3}};
		FType S2T3 = {S2, FTypenameId{T3}};

		FParametricTypeIndexer Indexer;

		FParametricTypeId T4_S0T3 = Indexer.Index({T4, 1, &S0T3});
		FParametricTypeId T4_S1T3 = Indexer.Index({T4, 1, &S1T3});
		
		CHECK(Indexer.Resolve(T4_S0T3) == FParametricTypeView{T4, 1, &S0T3});
		CHECK(Indexer.Resolve(T4_S1T3) == FParametricTypeView{T4, 1, &S1T3});
		
		FType S1T4_S0T3 = {S1,  FTypenameId{T4_S0T3}};
		FType S2T4_S1T3 = {S2,  FTypenameId{T4_S1T3}};
		
		CHECK(S1T4_S0T3.Name.AsParametric() == T4_S0T3);
		CHECK(S2T4_S1T3.Name.AsParametric() == T4_S1T3);

		FParametricTypeId T5_S0T3_S2T3 = Indexer.Index({T5, {S1T4_S0T3, S2T4_S1T3}});
		CHECK(Indexer.Resolve(T5_S0T3_S2T3) == FParametricTypeView{T5, {S1T4_S0T3, S2T4_S1T3}});
		
		CHECK(T4_S0T3		== Indexer.Index({T4, 1, &S0T3}));
		CHECK(T4_S1T3		== Indexer.Index({T4, 1, &S1T3}));
		CHECK(T5_S0T3_S2T3	== Indexer.Index({T5, {S1T4_S0T3, S2T4_S1T3}}));

		CHECK(Indexer.Num() == 3);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////

inline constexpr uint32 TestMagics[] = {0xFEEDF00D, 0xABCD1234, 0xDADADAAA, 0x99887766, 0xF0F1F2F3 };

template<>
void AppendString(FUtf8StringBuilderBase& Out, const FAnsiString& Str)
{
	Out.Append(Str);
}

class FTestBatchBuilder final : public TIdIndexer<FAnsiString>, public IDeclarations
{
public:
	FTestBatchBuilder(FScratchAllocator& InScratch) : EnumDeclarations(FDebugIds(*this)), Scratch(InScratch) {}

	FEnumId						DeclareEnum(FType Type, EEnumMode Mode, ELeafWidth Width, std::initializer_list<const char*> Names, std::initializer_list<uint64> Constants);
	FEnumId						DeclareEnum(const char* Scope, const char* Name, EEnumMode Mode, ELeafWidth Width, std::initializer_list<const char*> Names, std::initializer_list<uint64> Constants);
	FDeclId						DeclareStruct(FType Type, std::initializer_list<const char*> MemberOrder, TConstArrayView<FMemberSpec> MemberTypes, EMemberPresence Occupancy, FOptionalDeclId Super = NoId);
	FDeclId						DeclareStruct(const char* Scope, const char* Name, std::initializer_list<const char*> MemberNames, TConstArrayView<FMemberSpec> MemberTypes, EMemberPresence Occupancy, FOptionalDeclId Super = NoId);
	
	const FEnumDeclaration&		Get(FEnumId Id) const { return EnumDeclarations.Get(Id); }
	const FStructDeclaration&	Get(FDeclId Id) const { return *StructDeclarations.FindChecked(Id); }

	void						AddObject(FDeclId Id, FMemberBuilder&& Members);

	TArray64<uint8>				Write();

	FDebugIds					GetDebug() const { return FDebugIds(*this); }

private:
	TArray<TPair<FDeclId, FBuiltStruct*>>		Objects;
	TMap<FDeclId, FStructDeclarationPtr>		StructDeclarations;
	FEnumDeclarations							EnumDeclarations;
	FScratchAllocator&							Scratch;

	TArray<FMemberId> NameMembers(std::initializer_list<const char*> Members)
	{
		TArray<FMemberId> Out;
		Out.Reserve(Members.size());
		for (const char* Member : Members)
		{
			Out.Add(NameMember(Member));
		}
		return Out;
	}
	
	TArray<FEnumerator> MakeEnumerators(TConstArrayView<const char*> InNames, TConstArrayView<uint64> Constants)
	{
		check(InNames.Num() == Constants.Num());
		TArray<FEnumerator> Out;
		Out.SetNumUninitialized(InNames.Num());
		for (uint32 Idx = 0, Num = Out.Num(); Idx < Num; ++Idx)
		{
			Out[Idx] = { MakeName(InNames[Idx]), Constants[Idx] };
		}
		return Out;
	}

	TArray<char> GetNameData() const;

	virtual const FEnumDeclaration*		Find(FEnumId Id) const override { return EnumDeclarations.Find(Id); }
	virtual const FStructDeclaration*	Find(FStructId Id) const override { return StructDeclarations.FindChecked(FDeclId(Id)); }
	virtual FDeclId						Lower(FBindId Id) const override
	{
		checkf(false, TEXT("All struct ids should be declared, nothing is bound with different names in this test suite"));
		return LowerCast(Id);
	}
};

FDeclId FTestBatchBuilder::DeclareStruct(FType Type, std::initializer_list<const char*> MemberNames, TConstArrayView<FMemberSpec> MemberTypes,  EMemberPresence Occupancy, FOptionalDeclId Super)
{
	FDeclId Id = IndexDeclId(Type);
	StructDeclarations.Emplace(Id, Declare({Id, Super, /* v */ 0, Occupancy, NameMembers(MemberNames), MemberTypes}));
	return Id;
}

FDeclId FTestBatchBuilder::DeclareStruct(const char* Scope, const char* Name, std::initializer_list<const char*> MemberNames, TConstArrayView<FMemberSpec> MemberTypes, EMemberPresence Occupancy, FOptionalDeclId Super)
{
	return DeclareStruct(MakeType(Scope, Name), MemberNames, MemberTypes, Occupancy, Super);
}

FEnumId FTestBatchBuilder::DeclareEnum(FType Type, EEnumMode Mode, ELeafWidth Width, std::initializer_list<const char*> InNames, std::initializer_list<uint64> Constants)
{
	FEnumId Id = IndexEnum(Type);
	EnumDeclarations.Declare(Id, Type, Mode, Width, MakeEnumerators(InNames, Constants), EEnumAliases::Fail);
	return Id;
}

FEnumId FTestBatchBuilder::DeclareEnum(const char* Scope, const char* Name, EEnumMode Mode, ELeafWidth Width, std::initializer_list<const char*> InNames, std::initializer_list<uint64> Constants)
{
	return DeclareEnum(MakeType(Scope, Name), Mode, Width, InNames, Constants);
}

void FTestBatchBuilder::AddObject(FDeclId Id, FMemberBuilder&& Members)
{
	Objects.Emplace(Id, Members.BuildAndReset(Scratch, Get(Id), GetDebug()));
}

TArray64<uint8> FTestBatchBuilder::Write()
{
	// Build partial schemas
	FSchemasBuilder SchemaBuilders(*this,  *this, Scratch, ESchemaFormat::StableNames);
	for (const TPair<FDeclId, FBuiltStruct*>& Object : Objects)
	{
		SchemaBuilders.NoteStructAndMembers(Object.Key, *Object.Value);
	}
	FBuiltSchemas Schemas = SchemaBuilders.Build(); 

	// Filter out declared but unused names and ids
	FWriter Writer(*this, *this, Schemas, ESchemaFormat::StableNames);

	// Write names
	TArray64<uint8> Out;
	TArray64<uint8> Tmp;
	for (FNameId Name : Writer.GetUsedNames())
	{
		const FAnsiString& Str = ResolveName(Name);
		WriteData(Tmp, *Str, Str.Len() + 1);
	}
	WriteInt(Out, TestMagics[0]);
	WriteSkippableSlice(Out, Tmp);
	Tmp.Reset();

	// Write schemas
	WriteInt(Out, TestMagics[1]);
	Writer.WriteSchemas(/* Out */ Tmp);
	WriteAlignmentPadding<uint32>(Out);
	WriteInt(Out, IntCastChecked<uint32>(Tmp.Num()));
	WriteArray(Out, Tmp);
	Tmp.Reset();

	// Write objects
	WriteInt(Out, TestMagics[2]);
	for (const TPair<FDeclId, FBuiltStruct*>& Object : Objects)
	{
		WriteInt(/* out */ Tmp, TestMagics[3]);
		WriteInt(/* out */ Tmp, Writer.GetWriteId(Object.Key).Get().Idx);
		Writer.WriteMembers(/* out */ Tmp, Object.Key, *Object.Value);
		WriteSkippableSlice(Out, Tmp);
		Tmp.Reset();
	}

	// Write object terminator
	WriteSkippableSlice(Out, TConstArrayView64<uint8>());
	WriteInt(Out, TestMagics[4]);
		
	return Out;
}

TArray<char> FTestBatchBuilder::GetNameData() const
{
	TArray<char> Out;
	Out.Reserve(Names.Num() * 100);
	for (const FAnsiString& Name : Names)
	{
		Out.Append(Name.GetCharArray());
		check(Out.Last() == '\0');
	}
	return Out;
}

////////////////////////////////////////////////////////////////////////////////////////////////

class FTestNameReader
{
public:
	void Read(FMemoryView Data)
	{
		check(Names.IsEmpty() && !Data.IsEmpty());
		TConstArrayView<char> AllChars(static_cast<const char*>(Data.GetData()), IntCastChecked<int32>(Data.GetSize()));
		
		const char* NextStr = AllChars.GetData();
		for (const char& Char : AllChars)
		{
			if (!Char)
			{
				Names.Add(NextStr);
				NextStr = &Char + 1;
			}
		}
		
		check(Names.Num() >= 3); // 1 FType and 1 member id at least
		check(NextStr == Data.GetDataEnd()); // end with null-terminator
	}
	
	uint32 NumNames() const { return IntCastChecked<uint32>(Names.Num()); }
	FAnsiStringView operator[](FNameId Id) const { return MakeStringView(Names[Id.Idx]); }
	FAnsiStringView operator[](FMemberId Name) const { return MakeStringView(Names[Name.Id.Idx]); }
	FAnsiStringView operator[](FOptionalMemberId Name) const { return Name ? operator[](Name.Get()) : "Super"; }
	FAnsiStringView operator[](FScopeId Scope) const { return operator[](Scope.AsFlat().Name); }
	FAnsiStringView operator[](FTypenameId Name) const { return operator[](Name.AsConcrete().Id); }

private:
	TArray<const char*> Names;
};

class FTestBatchIds final : public FStableBatchIds
{
	const FTestNameReader& Names;
public:
	FTestBatchIds(const FTestNameReader& InNames, FSchemaBatchId Batch) : FStableBatchIds(Batch), Names(InNames) {}
	
	using FStableBatchIds::AppendString;
	virtual uint32 NumNames() const override { return Names.NumNames(); }
	virtual void AppendString(FUtf8StringBuilderBase& Out, FNameId Name) const override { Out.Append(Names[Name]); }
};

static FSchemaBatchId ParseBatch(TArray64<uint8>& OutData, TArray<FStructView>& OutObjects)
{
	// Grab and mount parsed schemas
	FByteReader It(MakeMemoryView(OutData));
	const uint32 SchemasSize = It.Grab<uint32>();
	FMemoryView SchemasView = It.GrabSlice(SchemasSize);
	const FSchemaBatch* Schemas = ValidateSchemas(SchemasView);
	FSchemaBatchId Batch = MountReadSchemas(Schemas);

	// Grab parsed objects
	while (uint64 NumBytes = It.GrabVarIntU())
	{	
		FByteReader ObjIt(It.GrabSlice(NumBytes));
		FStructSchemaId Schema = { ObjIt.Grab<uint32>() };
		OutObjects.Add({ { Schema, Batch }, ObjIt });
	}
	
	return Batch;
};

[[nodiscard]] FSchemaBatchId ParseYamlBatch(TArray64<uint8>& OutData, TArray<FStructView>& OutObjects, FUtf8StringView YamlView)
{
	ParseYamlBatch(OutData, YamlView);
	return ParseBatch(OutData, OutObjects);
};

[[nodiscard]] FSchemaBatchId ParseJsonBatch(TArray64<uint8>& OutData, TArray<FStructView>& OutObjects, FUtf8StringView JsonView)
{
	ParseJsonBatch(OutData, JsonView);
	return ParseBatch(OutData, OutObjects);
}

enum class ERoundtripFormat { Yaml, Json };

static void RoundtripText(
	const FBatchIds& BatchIds,
	TConstArrayView<FStructView> Objects,
	ERoundtripFormat Format=ERoundtripFormat::Yaml)
{
	// Print text
	TUtf8StringBuilder<4096> Text;
	switch (Format)
	{
	case ERoundtripFormat::Yaml: PrintYamlBatch(Text, BatchIds, Objects); break;
	case ERoundtripFormat::Json: PrintJsonBatch(Text, BatchIds, Objects); break;
	}
	FUtf8StringView TextView = Text.ToView();

	// Log text
	auto Wide = StringCast<TCHAR>(TextView.GetData(), TextView.Len());
	UE_LOGF(LogPlainPropsTests, Log, "Schemas with StableNames:\n%.*ls", Wide.Length(), Wide.Get());

	// Parse text
	TArray64<uint8> Data;
	TArray<FStructView> ParsedObjects;
	FSchemaBatchId ParsedBatch;
	switch (Format)
	{
	case ERoundtripFormat::Yaml: ParsedBatch = ParseYamlBatch(Data, ParsedObjects, TextView); break;
	case ERoundtripFormat::Json: ParsedBatch = ParseJsonBatch(Data, ParsedObjects, TextView); break;
	}

	// Diff schemas
	CHECK_FALSE(DiffSchemas(BatchIds.GetBatchId(), ParsedBatch));

	// Diff objects
	CHECK(Objects.Num() == ParsedObjects.Num());
	const int32 NumObjects = FMath::Min(Objects.Num(), ParsedObjects.Num());
	for (int32 I = 0; I < NumObjects; ++I)
	{
		FStructView In = Objects[I];
		FStructView Parsed = ParsedObjects[I];
		FReadDiffPath DiffPath;
		if (DiffStruct(In, Parsed, DiffPath))
		{
			TUtf8StringBuilder<256> Diff;
			PrintDiff(Diff, BatchIds, DiffPath);
			FAIL_CHECK(FString::Printf(TEXT("Diff in '%s' in Objects[%d]"), *Print(Diff.ToString()), I));
		}
	}

	// Unmount parsed schemas
	UnmountReadSchemas(ParsedBatch);
}
////////////////////////////////////////////////////////////////////////////////////////////////

class FTestBatchReader
{
public:
	FTestBatchReader(FMemoryView Data)
	{
		// Read names
		FByteReader It(Data);
		CHECK(It.Grab<uint32>() == TestMagics[0]);
		Names.Read(It.GrabSkippableSlice());
		
		// Read schemas
		CHECK(It.Grab<uint32>() == TestMagics[1]);
		It.SkipAlignmentPadding<uint32>();
		uint32 SchemasSize = It.Grab<uint32>();
		FMemoryView SchemasView = It.GrabSlice(SchemasSize);
		const FSchemaBatch* Schemas = ValidateSchemas(SchemasView);
		FSchemaBatchId Batch = MountReadSchemas(Schemas);
		CHECK(It.Grab<uint32>() == TestMagics[2]);
		
		// Read objects
		while (uint64 NumBytes = It.GrabVarIntU())
		{	
			FByteReader ObjIt(It.GrabSlice(NumBytes));
			CHECK(ObjIt.Grab<uint32>() == TestMagics[3]);
			FStructSchemaId Schema = { ObjIt.Grab<uint32>() };
			Objects.Add({ { Schema, Batch }, ObjIt });
		}
		
		CHECK(It.Grab<uint32>() == TestMagics[4]);
		CHECK(!Objects.IsEmpty());

		BatchIds.Emplace(Names, Batch);
	}

	void RoundtripText(PlainProps::ERoundtripFormat Format)
	{
		PlainProps::RoundtripText(BatchIds.GetValue(), Objects, Format);
	}

	~FTestBatchReader()
	{
		UnmountReadSchemas(Objects[0].Schema.Batch);
	}

	TConstArrayView<FStructView>		GetObjects() const { return Objects; }
	const FTestNameReader&				GetNames() const { return Names; }
	const FTestBatchIds&				GetBatchIds() const { return BatchIds.GetValue(); }
	
private:
	FTestNameReader Names;
	TOptional<FTestBatchIds> BatchIds;
	TArray<FStructView> Objects;
};

static void TestSerialize(void (*BuildObjects)(FTestBatchBuilder&, FScratchAllocator&), void (*CheckObjects)(TConstArrayView<FStructView>, const FTestNameReader&))
{
	TArray64<uint8> Data;
	{
		FScratchAllocator Scratch;
		FTestBatchBuilder Batch(Scratch);
		DbgVis::FIdScope _(Batch, "AnsiStr");
		BuildObjects(Batch, Scratch);
		Data = Batch.Write();
	}

	{
		FTestBatchReader Batch(MakeMemoryView(Data));
		Batch.RoundtripText(PlainProps::ERoundtripFormat::Yaml);
		CheckObjects(Batch.GetObjects(), Batch.GetNames());
	}

	{
		FTestBatchReader Batch(MakeMemoryView(Data));
		Batch.RoundtripText(PlainProps::ERoundtripFormat::Json);
		CheckObjects(Batch.GetObjects(), Batch.GetNames());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////

// Tests that everything was read
struct FTestMemberReader : public FMemberReader
{
	using FMemberReader::FMemberReader;
	
	~FTestMemberReader()
	{
		CHECK(MemberIdx == NumMembers); // Must read all members
		CHECK(RangeTypeIdx == NumRangeTypes); // Must read all ranges
#if DO_CHECK
		CHECK(InnerSchemaIdx == NumInnerSchemas); // Must read all schema ids
#endif
	}
};

template<typename OutType, typename InType>
TArray<OutType> MakeArray(const InType& Items)
{
	TArray<OutType> Out;
	Out.Reserve(IntCastChecked<int32>(Items.Num()));
	for (const auto& Item : Items)
	{
		Out.Emplace(Item);
	}
	return Out;
}

////////////////////////////////////////////////////////////////////////////////////////////////

template<uint32 N>
TStaticArray<FMemberSpec, N> Respec(FMemberSpec Spec)
{
	return MakeUniformStaticArray<FMemberSpec, N>(Spec);
}

static FMemberSpec UniRange(FMemberSpec Item)	{ return FMemberSpec(ESizeType::Uni, Item); }
static FMemberSpec S8Range(FMemberSpec Item)	{ return FMemberSpec(ESizeType::S8, Item); }
static FMemberSpec U8Range(FMemberSpec Item)	{ return FMemberSpec(ESizeType::U8, Item); }
static FMemberSpec S16Range(FMemberSpec Item)	{ return FMemberSpec(ESizeType::S16, Item); }
static FMemberSpec U16Range(FMemberSpec Item)	{ return FMemberSpec(ESizeType::U16, Item); }
static FMemberSpec S32Range(FMemberSpec Item)	{ return FMemberSpec(ESizeType::S32, Item); }
static FMemberSpec U32Range(FMemberSpec Item)	{ return FMemberSpec(ESizeType::U32, Item); }
static FMemberSpec S64Range(FMemberSpec Item)	{ return FMemberSpec(ESizeType::S64, Item); }
static FMemberSpec U64Range(FMemberSpec Item)	{ return FMemberSpec(ESizeType::U64, Item); }

inline FMemberSpec Enum8(FEnumId Id)			{ return FMemberSpec({ELeafType::Enum, ELeafWidth::B8}, Id); }
inline FMemberSpec Enum16(FEnumId Id)			{ return FMemberSpec({ELeafType::Enum, ELeafWidth::B16}, Id); }
inline FMemberSpec Enum32(FEnumId Id)			{ return FMemberSpec({ELeafType::Enum, ELeafWidth::B32}, Id); }
inline FMemberSpec Enum64(FEnumId Id)			{ return FMemberSpec({ELeafType::Enum, ELeafWidth::B64}, Id); }

////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FPlainPropsReadWriteTest, "System::Core::Serialization::PlainProps::ReadWrite", "[Core][PlainProps][SmokeFilter]")
{
	SECTION("Bool")
	{
		static constexpr std::initializer_list<const char*> MemberNames =  {"b0", "b1", "b2", "b3", "b4", "b5", "b6", "b7", "b8", "b9", "b10", "b11"};	

		TestSerialize([](FTestBatchBuilder& Batch, FScratchAllocator& Scratch)
		{
			FDeclId Id = Batch.DeclareStruct("Testing", "Bools", MemberNames, Respec<12>(SpecBool), EMemberPresence::AllowSparse);

			FMemberBuilder B1T;
			B1T.Add(Batch.NameMember("b3"),	true);

			FMemberBuilder B1F;
			B1F.Add(Batch.NameMember("b1"),	false);

			FMemberBuilder B8M;
			B8M.Add(Batch.NameMember("b1"),	true);
			B8M.Add(Batch.NameMember("b2"),	false);
			B8M.Add(Batch.NameMember("b3"),	true);
			B8M.Add(Batch.NameMember("b4"),	false);
			B8M.Add(Batch.NameMember("b5"),	true);
			B8M.Add(Batch.NameMember("b6"),	false);
			B8M.Add(Batch.NameMember("b8"),	false);
			B8M.Add(Batch.NameMember("b9"), true);
			
			FMemberBuilder B9T;
			B9T.Add(Batch.NameMember("b1"),	true);
			B9T.Add(Batch.NameMember("b2"),	true);
			B9T.Add(Batch.NameMember("b3"),	true);
			B9T.Add(Batch.NameMember("b4"),	true);
			B9T.Add(Batch.NameMember("b5"),	true);
			B9T.Add(Batch.NameMember("b6"),	true);
			B9T.Add(Batch.NameMember("b8"),	true);
			B9T.Add(Batch.NameMember("b9"), true);
			B9T.Add(Batch.NameMember("b10"),true);

			Batch.AddObject(Id, MoveTemp(B1T));
			Batch.AddObject(Id, MoveTemp(B1F));
			Batch.AddObject(Id, MoveTemp(B8M));
			Batch.AddObject(Id, MoveTemp(B9T));
		}, 
		[](TConstArrayView<FStructView> Objects, const FTestNameReader& Names)
		{
			CHECK(Objects.Num() == 4);
			FTestMemberReader B1T(Objects[0]);
			FTestMemberReader B1F(Objects[1]);
			FTestMemberReader B8M(Objects[2]);
			FTestMemberReader B9T(Objects[3]);
			CHECK(Objects[0].Schema.Id == Objects[3].Schema.Id);

			// Check schema
			const FStructSchema& Schema =  Objects[0].Schema.Resolve();
			CHECK(Names[Schema.Type.Scope] == "Testing");
			CHECK(Names[Schema.Type.Name] == "Bools");
			CHECK(Schema.NumMembers == 9); // b0, b7 and b11 unused
			CHECK(Schema.NumRangeTypes == 0);
			CHECK(Schema.NumInnerSchemas == 0);
			CHECK(Schema.IsDense == 0);
			CHECK(Schema.Inheritance == ESuper::No);
			CHECK(FStructSchema::GetMemberTypes(Schema.Footer)[0] == FUnpackedLeafType(ELeafType::Bool, ELeafWidth::B8).Pack());
			CHECK(FStructSchema::GetMemberTypes(Schema.Footer)[8] == FUnpackedLeafType(ELeafType::Bool, ELeafWidth::B8).Pack());
			TConstArrayView<FMemberId> MemberIds = Schema.GetMemberNames();
			CHECK(Names[MemberIds[0]] == "b1");
			CHECK(Names[MemberIds[1]] == "b2");
			CHECK(Names[MemberIds[2]] == "b3");
			CHECK(Names[MemberIds[3]] == "b4");
			CHECK(Names[MemberIds[4]] == "b5");
			CHECK(Names[MemberIds[5]] == "b6");
			CHECK(Names[MemberIds[6]] == "b8");
			CHECK(Names[MemberIds[7]] == "b9");
			CHECK(Names[MemberIds[8]] == "b10");

			CHECK(Names[B1T.PeekName()] == "b3");
			CHECK(B1T.GrabLeaf().AsBool() == true);

			CHECK(Names[B1F.PeekName()] == "b1");
			CHECK(B1F.GrabLeaf().AsBool() == false);
			
			CHECK(B8M.GrabLeaf().AsBool() == true);
			CHECK(B8M.GrabLeaf().AsBool() == false);
			CHECK(B8M.GrabLeaf().AsBool() == true);
			CHECK(B8M.GrabLeaf().AsBool() == false);
			CHECK(B8M.GrabLeaf().AsBool() == true);
			CHECK(B8M.GrabLeaf().AsBool() == false);
			CHECK(B8M.GrabLeaf().AsBool() == false);
			CHECK(B8M.GrabLeaf().AsBool() == true);

			for (int32 Idx = 0; Idx < 9; ++Idx)
			{
				CHECK(B9T.GrabLeaf().AsBool() == true);
			}
		});
	}

	SECTION("Number")
	{
		static constexpr std::initializer_list<const char*> MemberNames =  {"F32", "F64", "S8", "U8", "S16", "U16", "S32", "U32", "S64", "U64"};

		TestSerialize([](FTestBatchBuilder& Batch, FScratchAllocator& Scratch)
		{
			FMemberSpec MemberTypes[] = {SpecF32, SpecF64, SpecS8, SpecU8, SpecS16, SpecU16, SpecS32, SpecU32, SpecS64, SpecU64};
			FDeclId Id = Batch.DeclareStruct("Test", "Numbers", MemberNames, MemberTypes, EMemberPresence::AllowSparse);

			FMemberBuilder Misc, Mins, Maxs, Some;

			Misc.Add(Batch.NameMember("F32"),	32.f);
			Misc.Add(Batch.NameMember("F64"),	64.0);
			Misc.Add(Batch.NameMember("S8"),	int8(-8));
			Misc.Add(Batch.NameMember("U8"),	uint8(8));
			Misc.Add(Batch.NameMember("S16"),	int16(-16));
			Misc.Add(Batch.NameMember("U16"),	uint16(16));
			Misc.Add(Batch.NameMember("S32"),	int32(-32));
			Misc.Add(Batch.NameMember("U32"),	uint32(32));
			Misc.Add(Batch.NameMember("S64"),	int64(-64));
			Misc.Add(Batch.NameMember("U64"),	uint64(64));
			
			Mins.Add(Batch.NameMember("F32"),	std::numeric_limits< float>::min());
			Mins.Add(Batch.NameMember("F64"),	std::numeric_limits<double>::min());
			Mins.Add(Batch.NameMember("S8"),	std::numeric_limits<  int8>::min());
			Mins.Add(Batch.NameMember("U8"),	std::numeric_limits< uint8>::min());
			Mins.Add(Batch.NameMember("S16"),	std::numeric_limits< int16>::min());
			Mins.Add(Batch.NameMember("U16"),	std::numeric_limits<uint16>::min());
			Mins.Add(Batch.NameMember("S32"),	std::numeric_limits< int32>::min());
			Mins.Add(Batch.NameMember("U32"),	std::numeric_limits<uint32>::min());
			Mins.Add(Batch.NameMember("S64"),	std::numeric_limits< int64>::min());
			Mins.Add(Batch.NameMember("U64"),	std::numeric_limits<uint64>::min());

			Maxs.Add(Batch.NameMember("F32"),	std::numeric_limits< float>::max());
			Maxs.Add(Batch.NameMember("F64"),	std::numeric_limits<double>::max());
			Maxs.Add(Batch.NameMember("S8"),	std::numeric_limits<  int8>::max());
			Maxs.Add(Batch.NameMember("U8"),	std::numeric_limits< uint8>::max());
			Maxs.Add(Batch.NameMember("S16"),	std::numeric_limits< int16>::max());
			Maxs.Add(Batch.NameMember("U16"),	std::numeric_limits<uint16>::max());
			Maxs.Add(Batch.NameMember("S32"),	std::numeric_limits< int32>::max());
			Maxs.Add(Batch.NameMember("U32"),	std::numeric_limits<uint32>::max());
			Maxs.Add(Batch.NameMember("S64"),	std::numeric_limits< int64>::max());
			Maxs.Add(Batch.NameMember("U64"),	std::numeric_limits<uint64>::max());

			Some.Add(Batch.NameMember("S32"),	0);
			
			Batch.AddObject(Id, MoveTemp(Misc));
			Batch.AddObject(Id, MoveTemp(Mins));
			Batch.AddObject(Id, MoveTemp(Maxs));
			Batch.AddObject(Id, MoveTemp(Some));
		}, 
		[](TConstArrayView<FStructView> Objects, const FTestNameReader& Names)
		{
			for (const FStructView& Object : Objects.Left(3))
			{
				FTestMemberReader Members(Object);
				for (const char* MemberName : MemberNames)
				{
					CHECK(Members.HasMore());
					CHECK(Names[Members.PeekName()] == MemberName);
					CHECK(Members.PeekKind() == EMemberKind::Leaf);
					(void)Members.GrabLeaf();
				}
			}
			
			FTestMemberReader Misc(Objects[0]);
			CHECK(Misc.GrabLeaf().AsFloat()		== 32.f);
			CHECK(Misc.GrabLeaf().AsDouble()	== 64.0);
			CHECK(Misc.GrabLeaf().AsS8()		== int8(-8));
			CHECK(Misc.GrabLeaf().AsU8()		== uint8(8));
			CHECK(Misc.GrabLeaf().AsS16()		== int16(-16));
			CHECK(Misc.GrabLeaf().AsU16()		== uint16(16));
			CHECK(Misc.GrabLeaf().AsS32()		== int32(-32));
			CHECK(Misc.GrabLeaf().AsU32()		== uint32(32));
			CHECK(Misc.GrabLeaf().AsS64()		== int64(-64));
			CHECK(Misc.GrabLeaf().AsU64()		== uint64(64));

			FTestMemberReader Mins(Objects[1]);
			CHECK(Mins.GrabLeaf().AsFloat()		== std::numeric_limits< float>::min());
			CHECK(Mins.GrabLeaf().AsDouble()	== std::numeric_limits<double>::min());
			CHECK(Mins.GrabLeaf().AsS8()		== std::numeric_limits<  int8>::min());
			CHECK(Mins.GrabLeaf().AsU8()		== std::numeric_limits< uint8>::min());
			CHECK(Mins.GrabLeaf().AsS16()		== std::numeric_limits< int16>::min());
			CHECK(Mins.GrabLeaf().AsU16()		== std::numeric_limits<uint16>::min());
			CHECK(Mins.GrabLeaf().AsS32()		== std::numeric_limits< int32>::min());
			CHECK(Mins.GrabLeaf().AsU32()		== std::numeric_limits<uint32>::min());
			CHECK(Mins.GrabLeaf().AsS64()		== std::numeric_limits< int64>::min());
			CHECK(Mins.GrabLeaf().AsU64()		== std::numeric_limits<uint64>::min());

			FTestMemberReader Maxs(Objects[2]);
			CHECK(Maxs.GrabLeaf().AsFloat()		== std::numeric_limits< float>::max());
			CHECK(Maxs.GrabLeaf().AsDouble()	== std::numeric_limits<double>::max());
			CHECK(Maxs.GrabLeaf().AsS8()		== std::numeric_limits<  int8>::max());
			CHECK(Maxs.GrabLeaf().AsU8()		== std::numeric_limits< uint8>::max());
			CHECK(Maxs.GrabLeaf().AsS16()		== std::numeric_limits< int16>::max());
			CHECK(Maxs.GrabLeaf().AsU16()		== std::numeric_limits<uint16>::max());
			CHECK(Maxs.GrabLeaf().AsS32()		== std::numeric_limits< int32>::max());
			CHECK(Maxs.GrabLeaf().AsU32()		== std::numeric_limits<uint32>::max());
			CHECK(Maxs.GrabLeaf().AsS64()		== std::numeric_limits< int64>::max());
			CHECK(Maxs.GrabLeaf().AsU64()		== std::numeric_limits<uint64>::max());

			FTestMemberReader Some(Objects[3]);
			CHECK(Names[Some.PeekName()] == "S32");
			CHECK(Some.GrabLeaf().AsS32() == 0);
		});
	}

	SECTION("Hex")
	{
		static constexpr std::initializer_list<const char*> MemberNames =  {"Hex8", "Hex16", "Hex32", "Hex64"};

		TestSerialize([](FTestBatchBuilder& Batch, FScratchAllocator& Scratch)
		{
			FMemberSpec MemberTypes[] = {SpecHex8, SpecHex16, SpecHex32, SpecHex64};
			FDeclId Id = Batch.DeclareStruct("Test", "Hex", MemberNames, MemberTypes, EMemberPresence::AllowSparse);

			FMemberBuilder Misc, Mins, Maxs, Some;

			Misc.AddHex(Batch.NameMember("Hex8"),	uint8(8));
			Misc.AddHex(Batch.NameMember("Hex16"),	uint16(16));
			Misc.AddHex(Batch.NameMember("Hex32"),	uint32(32));
			Misc.AddHex(Batch.NameMember("Hex64"),	uint64(64));
			
			Mins.AddHex(Batch.NameMember("Hex8"),	std::numeric_limits< uint8>::min());
			Mins.AddHex(Batch.NameMember("Hex16"),	std::numeric_limits<uint16>::min());
			Mins.AddHex(Batch.NameMember("Hex32"),	std::numeric_limits<uint32>::min());
			Mins.AddHex(Batch.NameMember("Hex64"),	std::numeric_limits<uint64>::min());

			Maxs.AddHex(Batch.NameMember("Hex8"),	std::numeric_limits< uint8>::max());
			Maxs.AddHex(Batch.NameMember("Hex16"),	std::numeric_limits<uint16>::max());
			Maxs.AddHex(Batch.NameMember("Hex32"),	std::numeric_limits<uint32>::max());
			Maxs.AddHex(Batch.NameMember("Hex64"),	std::numeric_limits<uint64>::max());

			Some.AddHex(Batch.NameMember("Hex32"),	uint32(0));
			
			Batch.AddObject(Id, MoveTemp(Misc));
			Batch.AddObject(Id, MoveTemp(Mins));
			Batch.AddObject(Id, MoveTemp(Maxs));
			Batch.AddObject(Id, MoveTemp(Some));
		}, 
		[](TConstArrayView<FStructView> Objects, const FTestNameReader& Names)
		{
			for (const FStructView& Object : Objects.Left(3))
			{
				FTestMemberReader Members(Object);
				for (const char* MemberName : MemberNames)
				{
					CHECK(Members.HasMore());
					CHECK(Names[Members.PeekName()] == MemberName);
					CHECK(Members.PeekKind() == EMemberKind::Leaf);
					(void)Members.GrabLeaf();
				}
			}
			
			FTestMemberReader Misc(Objects[0]);
			CHECK(Misc.GrabLeaf().AsHex8()	== uint8(8));
			CHECK(Misc.GrabLeaf().AsHex16()	== uint16(16));
			CHECK(Misc.GrabLeaf().AsHex32()	== uint32(32));
			CHECK(Misc.GrabLeaf().AsHex64()	== uint64(64));

			FTestMemberReader Mins(Objects[1]);
			CHECK(Mins.GrabLeaf().AsHex8()	== std::numeric_limits< uint8>::min());
			CHECK(Mins.GrabLeaf().AsHex16()	== std::numeric_limits<uint16>::min());
			CHECK(Mins.GrabLeaf().AsHex32()	== std::numeric_limits<uint32>::min());
			CHECK(Mins.GrabLeaf().AsHex64()	== std::numeric_limits<uint64>::min());

			FTestMemberReader Maxs(Objects[2]);
			CHECK(Maxs.GrabLeaf().AsHex8()	== std::numeric_limits< uint8>::max());
			CHECK(Maxs.GrabLeaf().AsHex16()	== std::numeric_limits<uint16>::max());
			CHECK(Maxs.GrabLeaf().AsHex32()	== std::numeric_limits<uint32>::max());
			CHECK(Maxs.GrabLeaf().AsHex64()	== std::numeric_limits<uint64>::max());

			FTestMemberReader Some(Objects[3]);
			CHECK(Names[Some.PeekName()] == "Hex32");
			CHECK(Some.GrabLeaf().AsHex32() == 0);
		});
	}

	SECTION("Unicode")
	{
		TestSerialize([](FTestBatchBuilder& Batch, FScratchAllocator& Scratch)
		{
			FDeclId Char8Id  = Batch.DeclareStruct("Test", "Char8",  {"A", "B", "C", "D", "E", "F"}, Respec<6>(SpecUtf8),  EMemberPresence::AllowSparse);
			FDeclId Char16Id = Batch.DeclareStruct("Test", "Char16", {"A", "B", "C", "D", "E", "F"}, Respec<6>(SpecUtf16), EMemberPresence::AllowSparse);
			FDeclId Char32Id = Batch.DeclareStruct("Test", "Char32", {"A", "B", "C", "D", "E", "F"}, Respec<6>(SpecUtf32), EMemberPresence::AllowSparse);

			FMemberBuilder Char8, Char16, Char32;

			Char8.Add(Batch.NameMember("A"), u8'\0');		// NUL, first valid code unit
			Char8.Add(Batch.NameMember("B"), u8'\x1');		// SOH, a control character
			Char8.Add(Batch.NameMember("C"), u8'\n');		// LF, an escaped character
			Char8.Add(Batch.NameMember("D"), u8'%');		// %, a printable character
			Char8.Add(Batch.NameMember("E"), u8'E');		// E, an alphabetic character
			Char8.Add(Batch.NameMember("F"), u8'\x7F');		// DEL, last valid code unit

			Char16.Add(Batch.NameMember("A"), u'\0');		// First valid code unit
			Char16.Add(Batch.NameMember("B"), u'\x0024');	// Dollar sign, single byte code unit
			Char16.Add(Batch.NameMember("C"), u'\xD7FF');	// Last single byte code unit
			Char16.Add(Batch.NameMember("D"), u'\xE000');	// First double byte code unit
			Char16.Add(Batch.NameMember("E"), u'\x20AC');	// Euro sign, double byte code unit
			Char16.Add(Batch.NameMember("F"), u'\xFFFD');	// Last valid single code unit character
			// UE::Core::Private::IsValidCodepoint() treats the non-characters FFFE and FFFF as invalid
						
			Char32.Add(Batch.NameMember("A"), U'\0');
			Char32.Add(Batch.NameMember("B"), U'\x1');
			Char32.Add(Batch.NameMember("C"), U'C');

			Batch.AddObject(Char8Id, MoveTemp(Char8));
			Batch.AddObject(Char16Id, MoveTemp(Char16));
			Batch.AddObject(Char32Id, MoveTemp(Char32));
		}, 
		[](TConstArrayView<FStructView> Objects, const FTestNameReader& Names)
		{
			CHECK(Objects.Num() == 3);
			
			FTestMemberReader Char8(Objects[0]);
			CHECK(Char8.GrabLeaf().AsChar8() == u8'\0');
			CHECK(Char8.GrabLeaf().AsChar8() == u8'\x1');
			CHECK(Char8.GrabLeaf().AsChar8() == u8'\n');
			CHECK(Char8.GrabLeaf().AsChar8() == u8'%');
			CHECK(Char8.GrabLeaf().AsChar8() == u8'E');
			CHECK(Char8.GrabLeaf().AsChar8() == u8'\x7F');

			FTestMemberReader Char16(Objects[1]);
			CHECK(Char16.GrabLeaf().AsChar16() == u'\0');
			CHECK(Char16.GrabLeaf().AsChar16() == u'\x0024');
			CHECK(Char16.GrabLeaf().AsChar16() == u'\xD7FF');
			CHECK(Char16.GrabLeaf().AsChar16() == u'\xE000');
			CHECK(Char16.GrabLeaf().AsChar16() == u'\x20AC');
			CHECK(Char16.GrabLeaf().AsChar16() == u'\xFFFD');

			FTestMemberReader Char32(Objects[2]);
			CHECK(Char32.GrabLeaf().AsChar32() == U'\0');
			CHECK(Char32.GrabLeaf().AsChar32() == U'\x1');
			CHECK(Char32.GrabLeaf().AsChar32() == U'C');
		});
	}

	SECTION("Dense")
	{
		static constexpr std::initializer_list<const char*> ExplicitNames = {"A", "B", "C"};
		static constexpr std::initializer_list<const char*> ImplicitNames = {"0", "A", "1", "B", "2", "C", "3"};

		TestSerialize([](FTestBatchBuilder& Batch, FScratchAllocator& Scratch)
		{
			FMemberSpec ExplicitTypes[] = {SpecUtf8, SpecUtf16, SpecUtf32};
			FMemberSpec ImplicitTypes[] = {SpecBool, SpecUtf8, SpecBool, SpecUtf16, SpecBool, SpecUtf32, SpecBool};

			FDeclId ExplicitId = Batch.DeclareStruct("Test", "ExplicitDense", ExplicitNames, ExplicitTypes, EMemberPresence::RequireAll);
			FDeclId ImplicitId = Batch.DeclareStruct("Test", "ImplicitDense", ImplicitNames, ImplicitTypes, EMemberPresence::AllowSparse);

			FMemberBuilder X;
			X.Add(Batch.NameMember("A"), char8_t('a'));
			X.Add(Batch.NameMember("B"), char16_t('b'));
			X.Add(Batch.NameMember("C"), char32_t('c'));

			FMemberBuilder Y;
			Y.Add(Batch.NameMember("A"), char8_t('1'));
			Y.Add(Batch.NameMember("B"), char16_t('2'));
			Y.Add(Batch.NameMember("C"), char32_t('3'));
						
			Batch.AddObject(ExplicitId, MoveTemp(X));
			Batch.AddObject(ImplicitId, MoveTemp(Y));
		}, 
		[](TConstArrayView<FStructView> Objects, const FTestNameReader& Names)
		{
			CHECK(Objects.Num() == 2);

			const FStructSchema& ExplicitSchema = Objects[0].Schema.Resolve();
			const FStructSchema& ImplicitSchema = Objects[1].Schema.Resolve();
			CHECK(Names[ExplicitSchema.Type.Name] == "ExplicitDense");
			CHECK(ExplicitSchema.NumMembers == 3);
			CHECK(ExplicitSchema.IsDense == 1);

			CHECK(Names[ImplicitSchema.Type.Name] == "ImplicitDense");
			CHECK(ImplicitSchema.NumMembers == 3);
			CHECK(ImplicitSchema.IsDense == 1);

			FTestMemberReader X(Objects[0]);
			FTestMemberReader Y(Objects[1]);

			CHECK(Names[X.PeekName()]		== "A");
			CHECK(X.GrabLeaf().AsChar8()	== 'a');
			CHECK(Names[X.PeekName()]		== "B");
			CHECK(X.GrabLeaf().AsChar16()	== 'b');
			CHECK(Names[X.PeekName()]		== "C");
			CHECK(X.GrabLeaf().AsChar32()	== 'c');

			CHECK(Names[Y.PeekName()]		== "A");
			CHECK(Y.GrabLeaf().AsChar8()	== '1');
			CHECK(Names[Y.PeekName()]		== "B");
			CHECK(Y.GrabLeaf().AsChar16()	== '2');
			CHECK(Names[Y.PeekName()]		== "C");
			CHECK(Y.GrabLeaf().AsChar32()	== '3');
		});
	}

	SECTION("Struct")
	{
		static constexpr std::initializer_list<const char*> ObjectMembers = {"L1", "S", "N", "L2"};
		static constexpr std::initializer_list<const char*> StructMembers = {"Nested", "Leaf"};
		static constexpr std::initializer_list<const char*> NestedMembers = {"I1", "I2"};
		static constexpr std::initializer_list<const char*> UnusedMembers = {"Unused1", "Unused2"};
		static constexpr std::initializer_list<const char*> EmptyMembers  = {};

		TestSerialize([](FTestBatchBuilder& Batch, FScratchAllocator& Scratch)
		{
			FDeclId NestedId = Batch.DeclareStruct("Test", "Nested", NestedMembers, {SpecS32, SpecS32}, EMemberPresence::AllowSparse);
			FDeclId StructId = Batch.DeclareStruct("Test", "Struct", StructMembers, {NestedId, SpecBool}, EMemberPresence::AllowSparse);
			FDeclId ObjectId = Batch.DeclareStruct("Test", "Object", ObjectMembers, {SpecF32, StructId, NestedId, SpecF32}, EMemberPresence::AllowSparse);
			FDeclId UnusedId = Batch.DeclareStruct("Test", "Unused", UnusedMembers,	{SpecBool, SpecBool}, EMemberPresence::AllowSparse);
			FDeclId EmptyId  = Batch.DeclareStruct("Test", "Empty",  EmptyMembers, {}, EMemberPresence::AllowSparse);
		
			FMemberBuilder Members;
			Members.Add(Batch.NameMember("I1"), 100);
			FBuiltStruct* NestedInStruct = Members.BuildAndReset(Scratch, Batch.Get(NestedId), Batch.GetDebug());

			Members.AddStruct(Batch.NameMember("Nested"), NestedId, MoveTemp(NestedInStruct));
			Members.Add(Batch.NameMember("Leaf"), true);
			FBuiltStruct* Struct = Members.BuildAndReset(Scratch, Batch.Get(StructId), Batch.GetDebug());
		
			Members.Add(Batch.NameMember("I2"), 200);
			FBuiltStruct* NestedInObject = Members.BuildAndReset(Scratch, Batch.Get(NestedId), Batch.GetDebug());

			Members.Add(Batch.NameMember("L1"), 123.f);
			Members.AddStruct(Batch.NameMember("S"), StructId, MoveTemp(Struct));
			Members.AddStruct(Batch.NameMember("N"), NestedId, MoveTemp(NestedInObject));
			Members.Add(Batch.NameMember("L2"), -45.f);

			Batch.AddObject(ObjectId, MoveTemp(Members));
			Batch.AddObject(UnusedId, {});
			Batch.AddObject(EmptyId, {});
		}, 
		[](TConstArrayView<FStructView> Objects, const FTestNameReader& Names)
		{
			CHECK(Objects.Num() == 3);

			CHECK(Objects[0].Schema.Resolve().IsDense == 1);
			CHECK(Objects[1].Schema.Resolve().IsDense == 1);
			CHECK(Objects[2].Schema.Resolve().IsDense == 1);

			FTestMemberReader Object(Objects[0]);
			CHECK(Object.GrabLeaf().AsFloat() == 123.f);
			FTestMemberReader Struct(Object.GrabStruct());
			FTestMemberReader NestedInObject(Object.GrabStruct());
			CHECK(Object.GrabLeaf().AsFloat()	== -45.f);
			
			FTestMemberReader NestedInStruct(Struct.GrabStruct());
			CHECK(Struct.GrabLeaf().AsBool() == true);

			CHECK(NestedInObject.GrabLeaf().AsS32() == 200);

			CHECK(NestedInStruct.GrabLeaf().AsS32() == 100);

			FTestMemberReader Unused(Objects[1]);  
			FTestMemberReader Empty(Objects[2]);
		});
	}

	SECTION("Enum")
	{
		enum class EFlatSparse8 : int8 { A = 1, B = 2, C = 3 };

		static constexpr std::initializer_list<const char*> MemberNames = 
		{ "A2", "A0", "B0", "B4", "B5", "B7", "C3", "D0", "D34", "Max8", "Max16", "Max32", "Max64", "IF" };

		TestSerialize([](FTestBatchBuilder& Batch, FScratchAllocator& Scratch)
		{
			// Test create holes in the original FNameId, FDeclId and FEnumId index range
			FDeclId UnusedId = Batch.DeclareStruct("Test", "UnusedStruct", {"U1", "U2"}, {SpecBool, SpecBool}, EMemberPresence::AllowSparse);
			
			FEnumId U	= Batch.DeclareEnum("Test", "UnusedEnum1",	EEnumMode::Flag, ELeafWidth::B8,	{"U3"}, {1}); // Hole
			FEnumId A	= Batch.DeclareEnum("Test", "FlatDense8",	EEnumMode::Flat, ELeafWidth::B8,	{"A", "B", "C"}, {0, 1, 2});
			FEnumId X	= Batch.DeclareEnum("Test", "UnusedEnum2",	EEnumMode::Flag, ELeafWidth::B8,	{"U4"}, {1}); // Hole
			FEnumId B	= Batch.DeclareEnum("Test", "FlagDense8",	EEnumMode::Flag, ELeafWidth::B8,	{"A", "B", "C"}, {1, 2, 4});
			FEnumId C	= Batch.DeclareEnum("Test", "FlatSparse8",	EEnumMode::Flat, ELeafWidth::B8,	{"A", "B", "C"}, {1, 2, 3});
			FEnumId D	= Batch.DeclareEnum("Test", "FlagSparse8",	EEnumMode::Flag, ELeafWidth::B8,	{"None", "A", "B", "C"}, {0, 2, 16, 32});
			FEnumId E	= Batch.DeclareEnum("Test", "FlatLimit8",	EEnumMode::Flat, ELeafWidth::B8,	{"Min", "Max"}, {0, 0xFF});
			FEnumId F	= Batch.DeclareEnum("Test", "FlatLimit16",	EEnumMode::Flat, ELeafWidth::B16,	{"Min", "Max"}, {0, 0xFFFF});
			FEnumId G	= Batch.DeclareEnum("Test", "FlatLimit32",	EEnumMode::Flat, ELeafWidth::B32,	{"Min", "Max"}, {0, 0xFFFFFFFF});
			FEnumId H	= Batch.DeclareEnum("Test", "FlatLimit64",	EEnumMode::Flat, ELeafWidth::B64,	{"Min", "Max"}, {0, 0xFFFFFFFFFFFFFFFF});
			FEnumId I	= Batch.DeclareEnum("Test", "FlagLimit64",	EEnumMode::Flag, ELeafWidth::B64,	{"One", "Max"}, {1, 0x8000000000000000});
			
			FMemberSpec MemberTypes[] = {
				Enum8(A), Enum8(A), Enum8(B), Enum8(B), Enum8(B), Enum8(B), 
				Enum8(C), Enum8(D), Enum8(D), Enum8(E), Enum16(F), Enum32(G), Enum64(H), Enum64(I) };
			FDeclId ObjectId = Batch.DeclareStruct("Test", "Enums", MemberNames, MemberTypes, EMemberPresence::AllowSparse);

			FMemberBuilder Members;
			Members.AddEnum(Batch.NameMember("A2"),				A, (uint8)2);
			Members.AddEnum(Batch.NameMember("A0"),				A, (uint8)0);
			Members.AddEnum(Batch.NameMember("B0"),				B, (uint8)0);
			Members.AddEnum(Batch.NameMember("B4"),				B, (uint8)4);
			Members.AddEnum(Batch.NameMember("B5"),				B, (uint8)5);
			Members.AddEnum(Batch.NameMember("B7"),				B, (uint8)7);
			Members.AddEnum(Batch.NameMember("C3"),				C, EFlatSparse8::C);
			Members.AddEnum(Batch.NameMember("D0"),				D, (uint8)0);
			Members.AddEnum(Batch.NameMember("D34"),			D, (uint8)34);
			Members.AddEnum(Batch.NameMember("Max8"),			E, (uint8)0xFF);
			Members.AddEnum(Batch.NameMember("Max16"),			F, (uint16)0xFFFF);
			Members.AddEnum(Batch.NameMember("Max32"),			G, (uint32)0xFFFFFFFF);
			Members.AddEnum(Batch.NameMember("Max64"),			H, (uint64)0xFFFFFFFFFFFFFFFF);
			Members.AddEnum(Batch.NameMember("IF"),				I, (uint64)0x8000000000000001);

			Batch.AddObject(ObjectId, MoveTemp(Members));
		}, 
		[](TConstArrayView<FStructView> Objects, const FTestNameReader& Names)
		{
			CHECK(Objects.Num() == 1);

			FSchemaBatchId Batch = Objects[0].Schema.Batch;
			auto GetEnumName = [Batch, &Names](FLeafView Leaf) { return Names[ResolveEnumSchema(Batch, Leaf.Enum.Get()).Type.Name]; };
			auto EqualEnumNames = [&Names](TConstArrayView<FNameId> Ids, TConstArrayView<const char*> Strings)
			{
				return Algo::Compare(Ids, Strings, [&Names](auto& X, auto& Y) { return Names[X] == Y; });
			};

			FTestMemberReader It1(Objects[0]);
			CHECK(It1.GrabLeaf().AsUnderlyingValue<uint8>() == 2);
			CHECK(It1.GrabLeaf().AsUnderlyingValue<uint8>() == 0);
			CHECK(It1.GrabLeaf().AsUnderlyingValue<uint8>() == 0);
			CHECK(It1.GrabLeaf().AsUnderlyingValue<uint8>() == 4);
			CHECK(It1.GrabLeaf().AsUnderlyingValue<uint8>() == 5);
			CHECK(It1.GrabLeaf().AsUnderlyingValue<uint8>() == 7);
			CHECK(It1.GrabLeaf().As<EFlatSparse8>() == EFlatSparse8::C);
			CHECK(It1.GrabLeaf().AsUnderlyingValue<uint8>() == 0);
			CHECK(It1.GrabLeaf().AsUnderlyingValue<uint8>() == 34);
			CHECK(It1.GrabLeaf().AsUnderlyingValue<uint8>() == 0xFF);
			CHECK(It1.GrabLeaf().AsUnderlyingValue<uint16>() == 0xFFFF);
			CHECK(It1.GrabLeaf().AsUnderlyingValue<uint32>() == 0xFFFFFFFF);
			CHECK(It1.GrabLeaf().AsUnderlyingValue<uint64>() == 0xFFFFFFFFFFFFFFFF);
			CHECK(It1.GrabLeaf().AsUnderlyingValue<uint64>() == 0x8000000000000001);	

			FTestMemberReader It2(Objects[0]);
			CHECK(GetEnumName(It2.GrabLeaf()) == "FlatDense8");
			CHECK(GetEnumName(It2.GrabLeaf()) == "FlatDense8");
			CHECK(GetEnumName(It2.GrabLeaf()) == "FlagDense8");
			CHECK(GetEnumName(It2.GrabLeaf()) == "FlagDense8");
			CHECK(GetEnumName(It2.GrabLeaf()) == "FlagDense8");
			CHECK(GetEnumName(It2.GrabLeaf()) == "FlagDense8");
			CHECK(GetEnumName(It2.GrabLeaf()) == "FlatSparse8");
			CHECK(GetEnumName(It2.GrabLeaf()) == "FlagSparse8");
			CHECK(GetEnumName(It2.GrabLeaf()) == "FlagSparse8");
			CHECK(GetEnumName(It2.GrabLeaf()) == "FlatLimit8");
			CHECK(GetEnumName(It2.GrabLeaf()) == "FlatLimit16");
			CHECK(GetEnumName(It2.GrabLeaf()) == "FlatLimit32");
			CHECK(GetEnumName(It2.GrabLeaf()) == "FlatLimit64");
			CHECK(GetEnumName(It2.GrabLeaf()) == "FlagLimit64");			

			FTestMemberReader It3(Objects[0]);
			const FEnumSchema& FlatDense8 = ResolveEnumSchema(Batch, It3.GrabLeaf().Enum.Get());
			(void)It3.GrabLeaf();
			const FEnumSchema& FlagDense8 = ResolveEnumSchema(Batch, It3.GrabLeaf().Enum.Get());
			(void)It3.GrabLeaf();
			(void)It3.GrabLeaf();
			(void)It3.GrabLeaf();
			const FEnumSchema& FlatSparse8 = ResolveEnumSchema(Batch, It3.GrabLeaf().Enum.Get());
			const FEnumSchema& FlagSparse8 = ResolveEnumSchema(Batch, It3.GrabLeaf().Enum.Get());
			(void)It3.GrabLeaf();
			const FEnumSchema& FlatLimit8  = ResolveEnumSchema(Batch, It3.GrabLeaf().Enum.Get());
			const FEnumSchema& FlatLimit16 = ResolveEnumSchema(Batch, It3.GrabLeaf().Enum.Get());
			const FEnumSchema& FlatLimit32 = ResolveEnumSchema(Batch, It3.GrabLeaf().Enum.Get());
			const FEnumSchema& FlatLimit64 = ResolveEnumSchema(Batch, It3.GrabLeaf().Enum.Get());
			const FEnumSchema& FlagLimit64 = ResolveEnumSchema(Batch, It3.GrabLeaf().Enum.Get());

			CHECK(FlatDense8.ExplicitConstants);
			CHECK(!FlagDense8.ExplicitConstants);
			CHECK(FlatSparse8.ExplicitConstants);
			CHECK(FlagSparse8.ExplicitConstants);
			CHECK(FlatLimit8.ExplicitConstants);
			CHECK(FlatLimit16.ExplicitConstants);
			CHECK(FlatLimit32.ExplicitConstants);
			CHECK(FlatLimit64.ExplicitConstants);
			CHECK(FlagLimit64.ExplicitConstants);

			CHECK(EqualEnumNames(MakeConstArrayView(FlatDense8.Footer, FlatDense8.Num), MakeConstArrayView({"A", "C"})));
			CHECK(EqualEnumNames(MakeConstArrayView(FlagDense8.Footer, FlagDense8.Num), MakeConstArrayView({"A", "B", "C"})));
			CHECK(EqualEnumNames(MakeConstArrayView(FlatSparse8.Footer, FlatSparse8.Num), MakeConstArrayView({"C"})));
			CHECK(EqualEnumNames(MakeConstArrayView(FlagSparse8.Footer, FlagSparse8.Num), MakeConstArrayView({"None", "A", "C"})));
			CHECK(Names[FlatLimit8.Footer[0]] == "Max");
			CHECK(Names[FlatLimit16.Footer[0]] == "Max");
			CHECK(Names[FlatLimit32.Footer[0]] == "Max");
			CHECK(Names[FlatLimit64.Footer[0]] == "Max");
			CHECK(Names[FlagLimit64.Footer[0]] == "One");
			CHECK(Names[FlagLimit64.Footer[1]] == "Max");

			CHECK(EqualItems(GetConstants<uint8>(FlatDense8), MakeConstArrayView({0, 2})));
			CHECK(EqualItems(GetConstants<uint8>(FlagDense8), TConstArrayView<uint8>()));
			CHECK(EqualItems(GetConstants<EFlatSparse8>(FlatSparse8), MakeConstArrayView({EFlatSparse8::C})));
			CHECK(EqualItems(GetConstants<uint8>(FlagSparse8), MakeConstArrayView({0, 2, 32})));
			CHECK(GetConstants<uint8>(FlatLimit8)[0] == 0xFF);
			CHECK(GetConstants<uint16>(FlatLimit16)[0] == 0xFFFF);
			CHECK(GetConstants<uint32>(FlatLimit32)[0] == 0xFFFFFFFF);
			CHECK(GetConstants<uint64>(FlatLimit64)[0] == 0xFFFFFFFFFFFFFFFF);
			CHECK(GetConstants<uint64>(FlagLimit64)[0] == 1);
			CHECK(GetConstants<uint64>(FlagLimit64)[1] == 0x8000000000000000);
		});
	}

	SECTION("LeafRange")
	{
		enum class EABCD : uint16 { A, B, C, D };
		enum class EUnused1 : uint8 { X };
		enum class EUnused2 : uint8 { Y };

		TestSerialize([](FTestBatchBuilder& Batch, FScratchAllocator& Scratch)
		{
			FEnumId ABCD	= Batch.DeclareEnum("Test", "ABCD", EEnumMode::Flat, ELeafWidth::B16, {"A", "B", "C", "D"}, {0, 1, 2, 3});
			FEnumId Unused1	= Batch.DeclareEnum("Test", "Unused1", EEnumMode::Flat, ELeafWidth::B8, {"X"}, {0});
			FEnumId Unused2	= Batch.DeclareEnum("Test", "Unused2", EEnumMode::Flat, ELeafWidth::B8, {"Y"}, {0});

			static constexpr std::initializer_list<const char*> MemberNames = { "B0", "B1", "B8", "B9", "D0", "D3", "H0", "E3", "E0", "Hi" };
			FMemberSpec MemberTypes[] = {	S32Range(SpecBool), S32Range(SpecBool), S64Range(SpecBool), S64Range(SpecBool), 
											S32Range(SpecF64), S32Range(SpecF64),
											S32Range(SpecHex16), S32Range(Enum16(ABCD)), S32Range(Enum8(Unused1)),
											S32Range(SpecUtf8)};
			FDeclId ObjectId = Batch.DeclareStruct("Test", "Object", MemberNames, MemberTypes, EMemberPresence::AllowSparse);
		
			FMemberBuilder Members;
			Members.AddRange(Batch.NameMember("B0"), BuildLeafRange(Scratch, TConstArrayView<bool>()));
			Members.AddRange(Batch.NameMember("B1"), BuildLeafRange(Scratch, MakeArrayView({true})));
			Members.AddRange(Batch.NameMember("B8"), BuildLeafRange(Scratch, TConstArrayView<bool,int64>({false, true, false, true, false, true, false, true})));
			Members.AddRange(Batch.NameMember("B9"), BuildLeafRange(Scratch, TConstArrayView<bool,int64>({true, false, true, false, true, false, true, false, true})));
			Members.AddRange(Batch.NameMember("D0"), BuildLeafRange(Scratch, TConstArrayView<double>()));
			Members.AddRange(Batch.NameMember("D3"), BuildLeafRange(Scratch, MakeArrayView({DBL_MIN, 0.0, DBL_MAX})));
			Members.AddRange(Batch.NameMember("H0"), BuildHexRange(Scratch,  TConstArrayView<uint16>({0x00, 0x01, 0xEF, 0xFF})));
			Members.AddRange(Batch.NameMember("E3"), BuildEnumRange(Scratch, ABCD, MakeArrayView({EABCD::B, EABCD::A, EABCD::D})));
			Members.AddRange(Batch.NameMember("E0"), BuildEnumRange(Scratch, Unused1, TConstArrayView<EUnused1>()));
			Members.AddRange(Batch.NameMember("Hi"), BuildLeafRange(Scratch, MakeArrayView(u8"Hello!")));

			Batch.AddObject(ObjectId, MoveTemp(Members));
		}, 
		[](TConstArrayView<FStructView> Objects, const FTestNameReader& Names)
		{
			CHECK(Objects.Num() == 1);

			FTestMemberReader It(Objects[0]);
			FLeafRangeView B0 = It.GrabRange().AsLeaves();
			FLeafRangeView B1 = It.GrabRange().AsLeaves();
			FLeafRangeView B8 = It.GrabRange().AsLeaves();
			FLeafRangeView B9 = It.GrabRange().AsLeaves();
			FLeafRangeView D0 = It.GrabRange().AsLeaves();
			FLeafRangeView D3 = It.GrabRange().AsLeaves();
			FLeafRangeView H0 = It.GrabRange().AsLeaves();
			FLeafRangeView E3 = It.GrabRange().AsLeaves();
			FLeafRangeView E0 = It.GrabRange().AsLeaves();
			FLeafRangeView Hi = It.GrabRange().AsLeaves();
			
			CHECK(B0.Num() == 0);
			CHECK(EqualItems(B1.AsBools(),		MakeArrayView({true})));
			CHECK(EqualItems(B8.AsBools(),		MakeArrayView({false, true, false, true, false, true, false, true})));
			CHECK(EqualItems(B9.AsBools(),		MakeArrayView({true, false, true, false, true, false, true, false, true})));
			CHECK(EqualItems(D0.AsDoubles(),	TConstArrayView<double>()));
			CHECK(EqualItems(D3.AsDoubles(),	MakeArrayView({DBL_MIN, 0.0, DBL_MAX})));
			CHECK(EqualItems(H0.AsHex16s(),		TConstArrayView<uint16>({0x00, 0x01, 0xEF, 0xFF})));
			CHECK(EqualItems(E3.As<EABCD>(),	MakeArrayView({EABCD::B, EABCD::A, EABCD::D})));
			CHECK(EqualItems(E0.As<EUnused1>(),	TConstArrayView<EUnused1>()));
			CHECK(EqualItems(Hi.AsUtf8(),		u8"Hello!"));
		});
	}

	SECTION("UnicodeRange")
	{
		TestSerialize([](FTestBatchBuilder& Batch, FScratchAllocator& Scratch)
		{
			FDeclId Utf8Id  = Batch.DeclareStruct("Test", "Utf8",  {"Null", "Empty", "Escape", "Latin1", "CJK", "Symbols"}, Respec<6>(S32Range(SpecUtf8)), EMemberPresence::AllowSparse);
			FDeclId Utf16Id = Batch.DeclareStruct("Test", "Utf16", {"Null", "Empty", "Escape", "Latin1", "CJK", "Symbols"}, Respec<6>(S32Range(SpecUtf16)), EMemberPresence::AllowSparse);

			FMemberBuilder Utf8, Utf16;

			Utf8.AddRange(Batch.NameMember("Null"),		BuildLeafRange(Scratch, TConstArrayView<char8_t>()));
			Utf8.AddRange(Batch.NameMember("Empty"),	BuildLeafRange(Scratch, MakeArrayView(u8"")));
			Utf8.AddRange(Batch.NameMember("Escape"),	BuildLeafRange(Scratch, MakeArrayView(u8"\"\\ \x01 \x1f\" \"\b \f \n \r \t \"\\").LeftChop(1)));
			Utf8.AddRange(Batch.NameMember("Latin1"),	BuildLeafRange(Scratch, MakeArrayView(u8"\u00E5 \u00E4 \u00F6")));	// å ä ö
			Utf8.AddRange(Batch.NameMember("CJK"),		BuildLeafRange(Scratch, MakeArrayView(u8"\u3300 \uFE30")));			// ㌀ ︰
			Utf8.AddRange(Batch.NameMember("Symbols"),	BuildLeafRange(Scratch, MakeArrayView(u8"\u2665 \u01F34C")));		// ♥ 🍌

			Utf16.AddRange(Batch.NameMember("Null"),	BuildLeafRange(Scratch, TConstArrayView<char16_t>()));
			Utf16.AddRange(Batch.NameMember("Empty"),	BuildLeafRange(Scratch, MakeArrayView(u"")));
			Utf16.AddRange(Batch.NameMember("Escape"),	BuildLeafRange(Scratch, MakeArrayView(u"\\\" \x01 \x1f\" \"\b \f \n \r \t \\\"").LeftChop(1)));
			Utf16.AddRange(Batch.NameMember("Latin1"),	BuildLeafRange(Scratch, MakeArrayView(u"\xC5 \xC4 \xD6")));			// Å Ä Ö
			Utf16.AddRange(Batch.NameMember("CJK"),		BuildLeafRange(Scratch, MakeArrayView(u"\x3300 \xFE30")));			// ㌀ ︰
			Utf16.AddRange(Batch.NameMember("Symbols"),	BuildLeafRange(Scratch, MakeArrayView(u"\x2665 \xD83C\xDF4C")));	// ♥ 🍌

			Batch.AddObject(Utf8Id, MoveTemp(Utf8));
			Batch.AddObject(Utf16Id, MoveTemp(Utf16));
		}, 
		[](TConstArrayView<FStructView> Objects, const FTestNameReader& Names)
		{
			CHECK(Objects.Num() == 2);

			FTestMemberReader It1(Objects[0]);
			CHECK(EqualItems(It1.GrabRange().AsLeaves().AsUtf8(), TConstArrayView<char8_t>()));
			CHECK(EqualItems(It1.GrabRange().AsLeaves().AsUtf8(), u8""));
			CHECK(EqualItems(It1.GrabRange().AsLeaves().AsUtf8(), MakeArrayView(u8"\"\\ \x01 \x1f\" \"\b \f \n \r \t \"\\").LeftChop(1)));
			CHECK(EqualItems(It1.GrabRange().AsLeaves().AsUtf8(), u8"\u00E5 \u00E4 \u00F6"));	// å ä ö
			CHECK(EqualItems(It1.GrabRange().AsLeaves().AsUtf8(), u8"\u3300 \uFE30"));			// ㌀ ︰
			CHECK(EqualItems(It1.GrabRange().AsLeaves().AsUtf8(), u8"\u2665 \u01F34C"));		// ♥ 🍌

			FTestMemberReader It2(Objects[1]);
			CHECK(EqualItems(It2.GrabRange().AsLeaves().AsUtf16(), TConstArrayView<char16_t>()));
			CHECK(EqualItems(It2.GrabRange().AsLeaves().AsUtf16(), u""));
			CHECK(EqualItems(It2.GrabRange().AsLeaves().AsUtf16(), MakeArrayView(u"\\\" \x01 \x1f\" \"\b \f \n \r \t \\\"").LeftChop(1)));
			CHECK(EqualItems(It2.GrabRange().AsLeaves().AsUtf16(), u"\xC5 \xC4 \xD6"));			// Å Ä Ö
			CHECK(EqualItems(It2.GrabRange().AsLeaves().AsUtf16(), u"\x3300 \xFE30"));			// ㌀ ︰
			CHECK(EqualItems(It2.GrabRange().AsLeaves().AsUtf16(), u"\x2665 \xD83C\xDF4C"));	// ♥ 🍌
		});
	}
		
	SECTION("StructRange")
	{
		TestSerialize([](FTestBatchBuilder& Batch, FScratchAllocator& Scratch)
		{
			FDeclId StructId = Batch.DeclareStruct("Test", "Struct", {"I", "F"}, {SpecS32, SpecF32}, EMemberPresence::AllowSparse);
			FDeclId ObjectId = Batch.DeclareStruct("Test", "Object", {"Structs"}, {S32Range(StructId)}, EMemberPresence::AllowSparse);
			 
			FStructRangeBuilder Structs(4);
			Structs[0].Add(Batch.NameMember("I"), 0);
			Structs[1].Add(Batch.NameMember("F"), 1.f);
			Structs[2].Add(Batch.NameMember("I"), 2);
			Structs[2].Add(Batch.NameMember("F"), 2.f);
		
			FMemberBuilder Members;
			Members.AddRange(Batch.NameMember("Structs"), Structs.BuildAndReset(Scratch, Batch.Get(StructId), Batch.GetDebug()));

			Batch.AddObject(ObjectId, MoveTemp(Members));
		}, 
		[](TConstArrayView<FStructView> Objects, const FTestNameReader& Names)
		{
			CHECK(Objects.Num() == 1);

			FTestMemberReader It(Objects[0]);
			TArray<FTestMemberReader> Structs = MakeArray<FTestMemberReader>(It.GrabRange().AsStructs());
			CHECK(Structs.Num() == 4);
			CHECK(Structs[0].GrabLeaf().AsS32() == 0);
			CHECK(Structs[1].GrabLeaf().AsFloat() == 1.f);
			CHECK(Structs[2].GrabLeaf().AsS32() == 2);
			CHECK(Structs[2].GrabLeaf().AsFloat() == 2.f);
			CHECK(!Structs[3].HasMore());
		});
	}
	
	SECTION("NestedRange")
	{
		enum class EAB : uint8 { A = 1, B = 4 };
		enum class EUnused : uint8 { X };

		TestSerialize([](FTestBatchBuilder& Batch, FScratchAllocator& Scratch)
		{
			FDeclId XY = Batch.DeclareStruct("Test", "XY", {"X", "Y"}, {SpecF32, SpecF32}, EMemberPresence::RequireAll);
			FDeclId ZW = Batch.DeclareStruct("Test", "ZW", {"Z", "W"}, {SpecF32, SpecF32}, EMemberPresence::AllowSparse);
			FEnumId Enum = Batch.DeclareEnum("Test", "AB", EEnumMode::Flag, ELeafWidth::B8, {"A", "B"}, {1, 4});
			FEnumId UnusedEnum = Batch.DeclareEnum("Test", "Unused", EEnumMode::Flat, ELeafWidth::B8, {"X"}, {0});
			static constexpr std::initializer_list<const char*> MemberNames = { "IntRs", "EmptyRs", "FloatRs", "EnumRs", "UnusedEnumRs", "UnicodeRs", "StructRs", "StructRRs" };
			FMemberSpec MemberTypes[] = { S32Range(S32Range(SpecS32)), S32Range(S32Range(SpecS32)), S64Range(S64Range(SpecF32)),
				U8Range(S32Range(Enum8(Enum))), U8Range(S32Range(Enum8(UnusedEnum))), S32Range(S32Range(SpecUtf8)),
				U64Range(U64Range(XY)), U32Range(S16Range(S16Range(ZW)))};
			FDeclId Object = Batch.DeclareStruct("Test", "Object", MemberNames, MemberTypes, EMemberPresence::AllowSparse);
	
			FNestedRangeBuilder IntRs(MakeLeafRangeSchema<int32, int32>(), 3);
			IntRs.Add(BuildLeafRange(Scratch, MakeArrayView({1})));
			IntRs.Add({});
			IntRs.Add(BuildLeafRange(Scratch, MakeArrayView({2, 3})));

			FNestedRangeBuilder FloatRs(MakeLeafRangeSchema<float, int64>(), 3);
			FloatRs.Add(BuildLeafRange(Scratch, TConstArrayView<float, int64>({1.f})));
			FloatRs.Add({});
			FloatRs.Add(BuildLeafRange(Scratch, TConstArrayView<float, int64>({2.f, 3.f})));

			FNestedRangeBuilder EnumRs(MakeEnumRangeSchema<EAB, int32>(Enum), 2);
			EnumRs.Add({});
			EnumRs.Add(BuildEnumRange(Scratch, Enum, MakeArrayView({EAB::A, EAB(0), EAB::B})));

			FNestedRangeBuilder UnusedEnumRs(MakeEnumRangeSchema<EUnused, int32>(UnusedEnum), 2);
			UnusedEnumRs.Add({});
			UnusedEnumRs.Add(BuildEnumRange(Scratch, UnusedEnum, TConstArrayView<EUnused>()));

			FNestedRangeBuilder UnicodeRs(MakeLeafRangeSchema<char8_t, int32>(), 3);
			UnicodeRs.Add(BuildLeafRange(Scratch, MakeArrayView(u8"Hello")));
			UnicodeRs.Add({});
			UnicodeRs.Add(BuildLeafRange(Scratch, MakeArrayView(u8"World!")));

			FStructRangeBuilder XYs(uint64(2));
			XYs[0].Add(Batch.NameMember("X"), 1.f);
			XYs[0].Add(Batch.NameMember("Y"), 2.f);
			XYs[1].Add(Batch.NameMember("X"), 3.f);
			XYs[1].Add(Batch.NameMember("Y"), 4.f);
			FNestedRangeBuilder StructRs(MakeStructRangeSchema(ESizeType::U64, XY), 1);
			StructRs.Add(XYs.BuildAndReset(Scratch, Batch.Get(XY), Batch.GetDebug()));

			FStructRangeBuilder ZWs(int16(3));
			ZWs[0].Add(Batch.NameMember("Z"), 1.5f);
			ZWs[2].Add(Batch.NameMember("Z"), 2.5f);
			ZWs[2].Add(Batch.NameMember("W"), 3.5f);
			FMemberSchema ZWRangeSchema = MakeStructRangeSchema(ESizeType::S16, ZW);
			FNestedRangeBuilder ZWRs(ZWRangeSchema, 1);
			ZWRs.Add(ZWs.BuildAndReset(Scratch, Batch.Get(ZW), Batch.GetDebug()));
			FNestedRangeBuilder StructRRs(MakeNestedRangeSchema(Scratch, ESizeType::S16, ZWRangeSchema), 1);
			StructRRs.Add(ZWRs.BuildAndReset(Scratch, ESizeType::S16));

			FMemberBuilder Members;
			Members.AddRange(Batch.NameMember("IntRs"), IntRs.BuildAndReset(Scratch, ESizeType::S32));
			Members.AddRange(Batch.NameMember("EmptyRs"), IntRs.BuildAndReset(Scratch, ESizeType::S32));
			Members.AddRange(Batch.NameMember("FloatRs"), FloatRs.BuildAndReset(Scratch, ESizeType::S64));
			Members.AddRange(Batch.NameMember("EnumRs"), EnumRs.BuildAndReset(Scratch, ESizeType::U8));
			Members.AddRange(Batch.NameMember("UnusedEnumRs"), UnusedEnumRs.BuildAndReset(Scratch, ESizeType::U8));
			Members.AddRange(Batch.NameMember("UnicodeRs"), UnicodeRs.BuildAndReset(Scratch, ESizeType::S32));
			Members.AddRange(Batch.NameMember("StructRs"), StructRs.BuildAndReset(Scratch, ESizeType::U64));
			Members.AddRange(Batch.NameMember("StructRRs"), StructRRs.BuildAndReset(Scratch, ESizeType::U32));

			Batch.AddObject(Object, MoveTemp(Members));
		}, 
		[](TConstArrayView<FStructView> Objects, const FTestNameReader& Names)
		{
			CHECK(Objects.Num() == 1);

			FTestMemberReader It(Objects[0]);
			TArray<FRangeView> IntRs = MakeArray<FRangeView>(It.GrabRange().AsRanges());
			FNestedRangeView EmptyRs = It.GrabRange().AsRanges();
			TArray<FRangeView> FloatRs = MakeArray<FRangeView>(It.GrabRange().AsRanges());
			TArray<FRangeView> EnumRs = MakeArray<FRangeView>(It.GrabRange().AsRanges());
			TArray<FRangeView> UnusedEnumRs = MakeArray<FRangeView>(It.GrabRange().AsRanges());
			TArray<FRangeView> UnicodeRs = MakeArray<FRangeView>(It.GrabRange().AsRanges());
			TArray<FRangeView> StructRs = MakeArray<FRangeView>(It.GrabRange().AsRanges());
			TArray<FRangeView> StructRRs = MakeArray<FRangeView>(It.GrabRange().AsRanges());

			CHECK(IntRs.Num() == 3);
			CHECK(EqualItems(IntRs[0].AsLeaves().AsS32s(), MakeArrayView({1})));
			CHECK(IntRs[1].IsEmpty());
			CHECK(EqualItems(IntRs[2].AsLeaves().AsS32s(), MakeArrayView({2, 3})));

			CHECK(EmptyRs.Num() == 0);
			
			CHECK(EnumRs.Num() == 2);
			CHECK(EnumRs[0].IsEmpty());
			CHECK(EqualItems(EnumRs[1].AsLeaves().As<EAB>(), MakeArrayView({EAB::A, EAB(0), EAB::B})));

			CHECK(UnusedEnumRs.Num() == 2);
			CHECK(UnusedEnumRs[0].IsEmpty());
			CHECK(EqualItems(UnusedEnumRs[1].AsLeaves().As<EUnused>(), TConstArrayView<EUnused>()));

			CHECK(FloatRs.Num() == 3);
			CHECK(EqualItems(FloatRs[0].AsLeaves().AsFloats(), MakeArrayView({1.f})));
			CHECK(FloatRs[1].IsEmpty());
			CHECK(EqualItems(FloatRs[2].AsLeaves().AsFloats(), MakeArrayView({2.f, 3.f})));

			CHECK(StructRs.Num() == 1);
			TArray<FTestMemberReader> XYs = MakeArray<FTestMemberReader>(StructRs[0].AsStructs());
			CHECK(Names[XYs[0].PeekName()] == "X");
			CHECK(XYs[0].GrabLeaf().AsFloat() == 1.f);
			CHECK(Names[XYs[0].PeekName()] == "Y");
			CHECK(XYs[0].GrabLeaf().AsFloat() == 2.f);
			CHECK(Names[XYs[1].PeekName()] == "X");
			CHECK(XYs[1].GrabLeaf().AsFloat() == 3.f);
			CHECK(Names[XYs[1].PeekName()] == "Y");
			CHECK(XYs[1].GrabLeaf().AsFloat() == 4.f);

			CHECK(StructRRs.Num() == 1);
			TArray<FRangeView> ZWRs = MakeArray<FRangeView>(StructRRs[0].AsRanges());
			CHECK(ZWRs.Num() == 1);
			TArray<FTestMemberReader> ZWs = MakeArray<FTestMemberReader>(ZWRs[0].AsStructs());
			CHECK(ZWs.Num() == 3);
			CHECK(Names[ZWs[0].PeekName()] == "Z");
			CHECK(ZWs[0].GrabLeaf().AsFloat() == 1.5f);
			CHECK(Names[ZWs[2].PeekName()] == "Z");
			CHECK(ZWs[2].GrabLeaf().AsFloat() == 2.5f);
			CHECK(Names[ZWs[2].PeekName()] == "W");
			CHECK(ZWs[2].GrabLeaf().AsFloat() == 3.5f);
		});

	}
	
	SECTION("UniRange")
	{
		TestSerialize([](FTestBatchBuilder& Batch, FScratchAllocator& Scratch)
		{
			FMemberSpec StructMemberTypes[] = { UniRange(SpecBool), S32Range(SpecBool), S32Range(UniRange(SpecBool)), SpecBool };
			FDeclId Struct = Batch.DeclareStruct("Test", "Struct", {"MaybeB", "Bs", "MaybeBs", "B"}, StructMemberTypes, EMemberPresence::AllowSparse);
			FMemberSpec ObjectMemberTypes[] = { UniRange(SpecBool), S32Range(Struct), SpecBool, SpecBool };
			FDeclId Object = Batch.DeclareStruct("Test", "Object", {"Bools", "Structs", "BF", "BT" }, ObjectMemberTypes, EMemberPresence::AllowSparse);
						
			const bool True = true;
			const bool False = false;
			FNestedRangeBuilder MaybeBs(MakeLeafRangeSchema<bool, bool>(), 1);
			FStructRangeBuilder Structs(10);
			Structs[5].AddRange(Batch.NameMember("MaybeB"), BuildLeafRange(Scratch, &False, true));
			Structs[6].AddRange(Batch.NameMember("MaybeB"), BuildLeafRange(Scratch, &True, false));
			Structs[7].AddRange(Batch.NameMember("MaybeB"), BuildLeafRange(Scratch, &True, true));
			Structs[7].AddRange(Batch.NameMember("Bs"),		BuildLeafRange(Scratch, MakeArrayView({true, true, false, false, true, true, false, false, true, true})));
			MaybeBs.Add(BuildLeafRange(Scratch, &True, true));
			Structs[7].AddRange(Batch.NameMember("MaybeBs"), MaybeBs.BuildAndReset(Scratch, ESizeType::S32));
			Structs[7].Add(Batch.NameMember("B"), true);
			MaybeBs.Add(BuildLeafRange(Scratch, &True, false));
			Structs[8].AddRange(Batch.NameMember("MaybeBs"), MaybeBs.BuildAndReset(Scratch, ESizeType::S32));
			Structs[9].Add(Batch.NameMember("B"), false);

			FMemberBuilder Members;
			Members.AddRange(Batch.NameMember("Bools"), BuildLeafRange(Scratch, &True, true));
			Members.AddRange(Batch.NameMember("Structs"), Structs.BuildAndReset(Scratch, Batch.Get(Struct), Batch.GetDebug()));
			Members.Add(Batch.NameMember("BF"), false);
			Members.Add(Batch.NameMember("BT"), true);

			Batch.AddObject(Object, MoveTemp(Members));
		}, 
		[](TConstArrayView<FStructView> Objects, const FTestNameReader& Names)
		{
			CHECK(Objects.Num() == 1);
			FTestMemberReader It(Objects[0]);
			
			FBoolRangeView Bools = It.GrabRange().AsLeaves().AsBools();
			TArray<FTestMemberReader> Structs = MakeArray<FTestMemberReader>(It.GrabRange().AsStructs());
			CHECK(It.GrabLeaf().AsBool() == false);
			CHECK(It.GrabLeaf().AsBool() == true);

			CHECK(Bools.Num() == 1);
			CHECK(Bools[0] == true);
			
			CHECK(EqualItems(Structs[5].GrabRange().AsLeaves().AsBools(), MakeArrayView({false})));
			CHECK(Structs[6].GrabRange().AsLeaves().AsBools().Num() == 0);
			CHECK(EqualItems(Structs[7].GrabRange().AsLeaves().AsBools(), MakeArrayView({true})));
			CHECK(EqualItems(Structs[7].GrabRange().AsLeaves().AsBools(), MakeArrayView({true, true, false, false, true, true, false, false, true, true})));
			TArray<FRangeView> MaybeBs7 = MakeArray<FRangeView>(Structs[7].GrabRange().AsRanges());
			CHECK(MaybeBs7.Num() == 1);
			CHECK(EqualItems(MaybeBs7[0].AsLeaves().AsBools(), MakeArrayView({true})));
			CHECK(Structs[7].GrabLeaf().AsBool() == true);
			TArray<FRangeView> MaybeBs8 = MakeArray<FRangeView>(Structs[8].GrabRange().AsRanges());
			CHECK(MaybeBs8.Num() == 1);
			CHECK(MaybeBs8[0].AsLeaves().AsBools().Num() == 0);
			CHECK(Structs[9].GrabLeaf().AsBool() == false);
		});
	}
	
	SECTION("DynamicStruct")
	{
		TestSerialize([](FTestBatchBuilder& Batch, FScratchAllocator& Scratch)
		{
			FDeclId Unused1 = Batch.DeclareStruct("Test", "Unused1", {"X"}, {SpecS32}, EMemberPresence::AllowSparse);
			FDeclId SA = Batch.DeclareStruct("Test", "SA", {"X"}, {SpecS32}, EMemberPresence::AllowSparse);
			FDeclId Unused2 = Batch.DeclareStruct("Test", "Unused2", {"X"}, {SpecF32}, EMemberPresence::AllowSparse);
			FDeclId SB = Batch.DeclareStruct("Test", "SB", {"X"}, {SpecF32}, EMemberPresence::AllowSparse);
			FDeclId Object = Batch.DeclareStruct("Test", "Object", {"Same", "Some", "None", "Diff"}, {SA, SA, SpecBool, SpecDynamicStruct}, EMemberPresence::AllowSparse);
			FDeclId Unused3 = Batch.DeclareStruct("Test", "Unused3", {"X"}, {SpecBool}, EMemberPresence::AllowSparse);
		
			auto BuildStruct = [&](FDeclId Struct, auto X)
				{
					FMemberBuilder Members;
					Members.Add(Batch.NameMember("X"), X);
					return Members.BuildAndReset(Scratch, Batch.Get(Struct), Batch.GetDebug());
				};

			FMemberBuilder O1;
			O1.AddStruct(Batch.NameMember("Same"), SA, BuildStruct(SA, 0));
			O1.AddStruct(Batch.NameMember("Some"), SA, BuildStruct(SA, 1));
			O1.AddStruct(Batch.NameMember("Diff"), SA, BuildStruct(SA, 2));
			FMemberBuilder O2;
			O2.AddStruct(Batch.NameMember("Same"), SA, BuildStruct(SA, 3));
			O2.AddStruct(Batch.NameMember("Diff"), SB, BuildStruct(SB, 4.f));
			
			Batch.AddObject(Object, MoveTemp(O1));
			Batch.AddObject(Object, MoveTemp(O2));
		}, 
		[](TConstArrayView<FStructView> Objects, const FTestNameReader& Names)
		{
			CHECK(Objects.Num() == 2);

			FTestMemberReader O1(Objects[0]);
			CHECK(O1.PeekType().AsStruct().IsDynamic == 0);
			CHECK(FTestMemberReader(O1.GrabStruct()).GrabLeaf().AsS32() == 0);
			CHECK(O1.PeekType().AsStruct().IsDynamic == 0);
			CHECK(FTestMemberReader(O1.GrabStruct()).GrabLeaf().AsS32() == 1);
			CHECK(O1.PeekType().AsStruct().IsDynamic == 1);
			CHECK(FTestMemberReader(O1.GrabStruct()).GrabLeaf().AsS32() == 2);
			
			FTestMemberReader O2(Objects[1]);
			CHECK(O2.PeekType().AsStruct().IsDynamic == 0);
			CHECK(FTestMemberReader(O2.GrabStruct()).GrabLeaf().AsS32() == 3);
			CHECK(O2.PeekType().AsStruct().IsDynamic == 1);
			CHECK(FTestMemberReader(O2.GrabStruct()).GrabLeaf().AsFloat() == 4.f);
		});
	}

	SECTION("DynamicStructRange")
	{
		TestSerialize([](FTestBatchBuilder& Batch, FScratchAllocator& Scratch)
		{
			FDeclId SA = Batch.DeclareStruct("Test", "SA", {"X"}, {SpecS32}, EMemberPresence::AllowSparse);
			FDeclId Unused = Batch.DeclareStruct("Test", "Unused2", {"X"}, {SpecS32}, EMemberPresence::AllowSparse);
			FDeclId SB = Batch.DeclareStruct("Test", "SB", {"X"}, {SpecF32}, EMemberPresence::AllowSparse);
			FMemberSpec MemberTypes[] = { S32Range(SA), S32Range(SA), SpecBool, S32Range(SpecDynamicStruct), S32Range(SA), S32Range(SpecDynamicStruct), S32Range(S32Range(SpecDynamicStruct)) };
			FDeclId Object = Batch.DeclareStruct("Test", "Object", {"Same", "Some", "None", "Diff", "SameEmpty", "DiffEmpty", "DiffNested"}, MemberTypes, EMemberPresence::AllowSparse);
		
			auto BuildStructRange = [&](FDeclId Struct, auto X)
				{
					FStructRangeBuilder Members(1);
					Members[0].Add(Batch.NameMember("X"), X);
					return Members.BuildAndReset(Scratch, Batch.Get(Struct), Batch.GetDebug());
				};

			
			FMemberBuilder O1;
			O1.AddRange(Batch.NameMember("Same"), BuildStructRange(SA, 10));
			O1.AddRange(Batch.NameMember("Some"), BuildStructRange(SA, 11));
			O1.AddRange(Batch.NameMember("Diff"), BuildStructRange(SA, 12));
			O1.AddRange(Batch.NameMember("SameEmpty"), BuildStructRange(SA, 13));
			O1.AddRange(Batch.NameMember("DiffEmpty"), BuildStructRange(SA, 14));
			FNestedRangeBuilder NestedSA(MakeStructRangeSchema(ESizeType::S32, SA), 1);
			NestedSA.Add(BuildStructRange(SA, 100));
			O1.AddRange(Batch.NameMember("DiffNested"), NestedSA.BuildAndReset(Scratch, ESizeType::S32));
			
			FMemberBuilder O2;
			O2.AddRange(Batch.NameMember("Same"), BuildStructRange(SA, 20));
			O2.AddRange(Batch.NameMember("Diff"), BuildStructRange(SB, 22.f));
			O2.AddRange(Batch.NameMember("SameEmpty"), FStructRangeBuilder(0).BuildAndReset(Scratch, Batch.Get(SA), Batch.GetDebug()));
			// PP-TEXT: Handle DiffEmpty in DynamicStructRange Test by printing dynamic type for empty ranges
			// O2.AddRange(Batch.NameMember("DiffEmpty"), FStructRangeBuilder(0).BuildAndReset(Scratch, Batch.Get(SB), Batch.GetDebug()));
			FNestedRangeBuilder NestedSB(MakeStructRangeSchema(ESizeType::S32, SB), 1);
			NestedSB.Add(BuildStructRange(SB, 200.f));
			O2.AddRange(Batch.NameMember("DiffNested"), NestedSB.BuildAndReset(Scratch, ESizeType::S32));
			
			Batch.AddObject(Object, MoveTemp(O1));
			Batch.AddObject(Object, MoveTemp(O2));
		}, 
		[](TConstArrayView<FStructView> Objects, const FTestNameReader& Names)
		{
			CHECK(Objects.Num() == 2);

			FTestMemberReader O1(Objects[0]);
			CHECK(MakeArray<FTestMemberReader>(O1.GrabRange().AsStructs())[0].GrabLeaf().AsS32() == 10);
			CHECK(MakeArray<FTestMemberReader>(O1.GrabRange().AsStructs())[0].GrabLeaf().AsS32() == 11);
			CHECK(MakeArray<FTestMemberReader>(O1.GrabRange().AsStructs())[0].GrabLeaf().AsS32() == 12);
			CHECK(MakeArray<FTestMemberReader>(O1.GrabRange().AsStructs())[0].GrabLeaf().AsS32() == 13);
			CHECK(MakeArray<FTestMemberReader>(O1.GrabRange().AsStructs())[0].GrabLeaf().AsS32() == 14);
			TArray<FRangeView> DiffNested1 = MakeArray<FRangeView>(O1.GrabRange().AsRanges());
			CHECK(MakeArray<FTestMemberReader>(DiffNested1[0].AsStructs())[0].GrabLeaf().AsS32() == 100);

			FTestMemberReader O2(Objects[1]);
			CHECK(MakeArray<FTestMemberReader>(O2.GrabRange().AsStructs())[0].GrabLeaf().AsS32() == 20);
			CHECK(MakeArray<FTestMemberReader>(O2.GrabRange().AsStructs())[0].GrabLeaf().AsFloat() == 22.f);
			CHECK(O2.GrabRange().AsStructs().Num() == 0);
			// CHECK(O2.GrabRange().AsStructs().Num() == 0);
			TArray<FRangeView> DiffNested2 = MakeArray<FRangeView>(O2.GrabRange().AsRanges());
			CHECK(MakeArray<FTestMemberReader>(DiffNested2[0].AsStructs())[0].GrabLeaf().AsFloat() == 200.f);
		});
	}

	SECTION("Inheritance")
	{
		TestSerialize([](FTestBatchBuilder& Batch, FScratchAllocator& Scratch)
		{
			FDeclId Unused = Batch.DeclareStruct("Test", "X", {"X"}, {SpecBool}, EMemberPresence::AllowSparse);
			FDeclId Low = Batch.DeclareStruct("Test", "Low", {"LInt"}, {SpecS32}, EMemberPresence::AllowSparse);
			FDeclId Mid = Batch.DeclareStruct("Test", "Mid", {"MInt", "MLow"}, {SpecS32, Low}, EMemberPresence::AllowSparse, ToOptional(Low));
			FDeclId Top = Batch.DeclareStruct("Test", "Top", {"TInt", "TLow", "TMids"}, {SpecS32, Low, S32Range(Mid)}, EMemberPresence::AllowSparse, ToOptional(Mid));
			
			FMemberBuilder Members;
			Members.Add(Batch.NameMember("LInt"), 123);
			Members.BuildSuperStruct(Scratch, Batch.Get(Low), Batch.GetDebug());
			Members.Add(Batch.NameMember("MInt"), 456);
			FMemberBuilder Nested;
			Nested.Add(Batch.NameMember("LInt"), 1000);
			Members.AddStruct(Batch.NameMember("MLow"), Low, Nested.BuildAndReset(Scratch, Batch.Get(Low), Batch.GetDebug()));
			Members.BuildSuperStruct(Scratch, Batch.Get(Mid), Batch.GetDebug());
			Members.Add(Batch.NameMember("TInt"), 789);
			Nested.Add(Batch.NameMember("LInt"), 2000);
			Members.AddStruct(Batch.NameMember("TLow"), Low, Nested.BuildAndReset(Scratch, Batch.Get(Low), Batch.GetDebug()));
			FStructRangeBuilder NestedRange(1);
			NestedRange[0].Add(Batch.NameMember("MInt"), 3000);
			Nested.Add(Batch.NameMember("LInt"), 4000);
			NestedRange[0].AddStruct(Batch.NameMember("MLow"), Low, Nested.BuildAndReset(Scratch, Batch.Get(Low), Batch.GetDebug()));
			Members.AddRange(Batch.NameMember("TMids"), NestedRange.BuildAndReset(Scratch, Batch.Get(Mid), Batch.GetDebug()));

			Batch.AddObject(Top, MoveTemp(Members));
		}, 
		[](TConstArrayView<FStructView> Objects, const FTestNameReader& Names)
		{
			CHECK(Objects.Num() == 1);

			FStructView TopView = Objects[0];
			FTestMemberReader TopIt(TopView);
			FStructView MidView = TopIt.GrabStruct();
			FTestMemberReader MidIt(MidView);
			FStructView LowView = MidIt.GrabStruct();
			FTestMemberReader LowIt(LowView);
			CHECK(LowIt.GrabLeaf().AsS32() == 123);
			CHECK(MidIt.GrabLeaf().AsS32() == 456);
			CHECK(FTestMemberReader(MidIt.GrabStruct()).GrabLeaf().AsS32() == 1000);
			CHECK(Names[TopIt.PeekName()] == "TInt");
			CHECK(TopIt.GrabLeaf().AsS32() == 789);
			CHECK(Names[TopIt.PeekName()] == "TLow");
			CHECK(FTestMemberReader(TopIt.GrabStruct()).GrabLeaf().AsS32() == 2000);
			CHECK(Names[TopIt.PeekName()] == "TMids");
			TArray<FTestMemberReader> MemberRangeIt = MakeArray<FTestMemberReader>(TopIt.GrabRange().AsStructs());
			CHECK(MemberRangeIt[0].GrabLeaf().AsS32() == 3000);
			CHECK(FTestMemberReader(MemberRangeIt[0].GrabStruct()).GrabLeaf().AsS32() == 4000);

			const FStructSchema& TopSchema = TopView.Schema.Resolve();
			const FStructSchema& MidSchema = MidView.Schema.Resolve();
			const FStructSchema& LowSchema = LowView.Schema.Resolve();
			CHECK(TopSchema.Inheritance == ESuper::Reused);
			CHECK(MidSchema.Inheritance == ESuper::Reused);
			CHECK(LowSchema.Inheritance == ESuper::No);
			CHECK(TopSchema.NumMembers == 4);
			CHECK(TopSchema.NumNames() == 3);
			CHECK(MidSchema.NumMembers == MidSchema.NumNames() + 1);
			CHECK(LowSchema.NumMembers == LowSchema.NumNames());

			FFlatMemberReader FlatIt(Objects[0]);
			CHECK(Names[FlatIt.PeekOwner().Name] == "Low");
			CHECK(FlatIt.GrabLeaf().AsS32() == 123);
			CHECK(Names[FlatIt.PeekOwner().Name] == "Mid");
			CHECK(FlatIt.GrabLeaf().AsS32() == 456);
			CHECK(Names[FlatIt.PeekOwner().Name] == "Mid");
			CHECK(FTestMemberReader(FlatIt.GrabStruct()).GrabLeaf().AsS32() == 1000);
			CHECK(Names[FlatIt.PeekOwner().Name] == "Top");
			CHECK(FlatIt.GrabLeaf().AsS32() == 789);
			CHECK(Names[FlatIt.PeekOwner().Name] == "Top");
			CHECK(FTestMemberReader(FlatIt.GrabStruct()).GrabLeaf().AsS32() == 2000);
			CHECK(Names[FlatIt.PeekOwner().Name] == "Top");
			TArray<FFlatMemberReader> FlatRangeIt = MakeArray<FFlatMemberReader>(FlatIt.GrabRange().AsStructs());
			CHECK(FlatRangeIt[0].GrabLeaf().AsS32() == 3000);
			CHECK(FFlatMemberReader(FlatRangeIt[0].GrabStruct()).GrabLeaf().AsS32() == 4000);
			CHECK(!FlatRangeIt[0].HasMore());
			CHECK(!FlatIt.HasMore());
		});
	}

	SECTION("SparseInheritance")
	{
		TestSerialize([](FTestBatchBuilder& Batch, FScratchAllocator& Scratch)
		{																													// A B C usage
			FDeclId B0 = Batch.DeclareStruct("Test", "B0", {"0"}, {SpecS32}, EMemberPresence::AllowSparse);					// - - -
			FDeclId B1 = Batch.DeclareStruct("Test", "B1", {"1"}, {SpecS32}, EMemberPresence::AllowSparse, ToOptional(B0)); // 1 1 1
			FDeclId B2 = Batch.DeclareStruct("Test", "B2", {"2"}, {SpecS32}, EMemberPresence::AllowSparse, ToOptional(B1)); // - 1 1
			FDeclId B3 = Batch.DeclareStruct("Test", "B3", {"3"}, {SpecS32}, EMemberPresence::AllowSparse, ToOptional(B2)); // - - 0
			FDeclId B4 = Batch.DeclareStruct("Test", "B4", {"4"}, {SpecS32}, EMemberPresence::AllowSparse, ToOptional(B3)); // 1 1 1
			FDeclId B5 = Batch.DeclareStruct("Test", "B5", {"5"}, {SpecS32}, EMemberPresence::AllowSparse, ToOptional(B4)); // 1 1 0
			FDeclId B6 = Batch.DeclareStruct("Test", "B6", {   }, {       }, EMemberPresence::AllowSparse, ToOptional(B5)); // 0 - 0
			FDeclId C5 = Batch.DeclareStruct("Test", "C5", {"5"}, {SpecS32}, EMemberPresence::AllowSparse, ToOptional(B4)); // - - -
			
			FMemberBuilder A;
			A.Add(Batch.NameMember("1"), 1);
			A.BuildSuperStruct(Scratch, Batch.Get(B1), Batch.GetDebug());
			A.Add(Batch.NameMember("4"), 4);
			A.BuildSuperStruct(Scratch, Batch.Get(B4), Batch.GetDebug());
			A.Add(Batch.NameMember("5"), 5);
			A.BuildSuperStruct(Scratch, Batch.Get(B5), Batch.GetDebug());

			FMemberBuilder B;
			B.Add(Batch.NameMember("1"), 10);
			B.BuildSuperStruct(Scratch, Batch.Get(B1), Batch.GetDebug());
			B.Add(Batch.NameMember("2"), 20);
			B.BuildSuperStruct(Scratch, Batch.Get(B2), Batch.GetDebug());
			B.Add(Batch.NameMember("4"), 40);
			B.BuildSuperStruct(Scratch, Batch.Get(B4), Batch.GetDebug());

			FMemberBuilder C;
			C.Add(Batch.NameMember("1"), 100);
			C.BuildSuperStruct(Scratch, Batch.Get(B1), Batch.GetDebug());
			C.Add(Batch.NameMember("2"), 200);
			C.BuildSuperStruct(Scratch, Batch.Get(B2), Batch.GetDebug());
			C.BuildSuperStruct(Scratch, Batch.Get(B3), Batch.GetDebug()); // Empty -> noop
			C.Add(Batch.NameMember("4"), 400);
			C.BuildSuperStruct(Scratch, Batch.Get(B4), Batch.GetDebug());
			C.BuildSuperStruct(Scratch, Batch.Get(B5), Batch.GetDebug()); // Empty -> noop
			
			Batch.AddObject(B6, MoveTemp(A));
			Batch.AddObject(B5, MoveTemp(B));
			Batch.AddObject(B6, MoveTemp(C));
		}, 
		[](TConstArrayView<FStructView> Objects, const FTestNameReader& Names)
		{
			CHECK(Objects.Num() == 3);
			
			const FStructSchema& Schema0 = Objects[0].Schema.Resolve();
			const FStructSchema& Schema1 = Objects[1].Schema.Resolve();
			const FStructSchema& Schema2 = Objects[2].Schema.Resolve();
			CHECK(Names[Schema0.Type.Name] == "B6");
			CHECK(Names[Schema1.Type.Name] == "B5");
			CHECK(Names[Schema2.Type.Name] == "B6");
			CHECK(Schema0.GetSuper() == Objects[1].Schema.Id);
			CHECK(Schema2.GetSuper() == Objects[1].Schema.Id);

			CHECK(Names[FMemberReader(Objects[0]).GrabStruct().Schema.Resolve().Type.Name]  == "B5");
			CHECK(Names[FMemberReader(Objects[1]).GrabStruct().Schema.Resolve().Type.Name]  == "B4");
			CHECK(Names[FMemberReader(Objects[2]).GrabStruct().Schema.Resolve().Type.Name]  == "B4");

			FSchemaBatchId BatchId = Objects[0].Schema.Batch;
			const FStructSchema& B6 = Objects[0].Schema.Resolve();
			const FStructSchema& B5 = ResolveStructSchema(BatchId, B6.GetSuper().Get());
			const FStructSchema& B4 = ResolveStructSchema(BatchId, B5.GetSuper().Get());
			const FStructSchema& B3 = ResolveStructSchema(BatchId, B4.GetSuper().Get());
			const FStructSchema& B2 = ResolveStructSchema(BatchId, B3.GetSuper().Get());
			const FStructSchema& B1 = ResolveStructSchema(BatchId, B2.GetSuper().Get());
			const FStructSchema& B0 = ResolveStructSchema(BatchId, B1.GetSuper().Get());
			// Super usage									A	B	C 	Decl
			CHECK(B0.Inheritance == ESuper::No);		//	-	-	-	-
			CHECK(B1.Inheritance == ESuper::Unused);	//	0	0	0	B0
			CHECK(B3.Inheritance == ESuper::Unused);	//	-	-	0	B2
			CHECK(B2.Inheritance == ESuper::Reused);	//	-	B1	B1	B1
			CHECK(B4.Inheritance == ESuper::Used);		//	B1	B2	B2	B3
			CHECK(B5.Inheritance == ESuper::Reused);	//	B4	B4	0	B4
			CHECK(B6.Inheritance == ESuper::Used);		//	B5	-	B4	B5

			FFlatMemberReader A(Objects[0]);
			FFlatMemberReader B(Objects[1]);
			FFlatMemberReader C(Objects[2]);
			CHECK(A.GrabLeaf().AsS32() == 1);
			CHECK(A.GrabLeaf().AsS32() == 4);
			CHECK(A.GrabLeaf().AsS32() == 5);
			CHECK(B.GrabLeaf().AsS32() == 10);
			CHECK(B.GrabLeaf().AsS32() == 20);
			CHECK(B.GrabLeaf().AsS32() == 40);
			CHECK(C.GrabLeaf().AsS32() == 100);
			CHECK(C.GrabLeaf().AsS32() == 200);
			CHECK(C.GrabLeaf().AsS32() == 400);
			CHECK(!A.HasMore());
			CHECK(!B.HasMore());
			CHECK(!C.HasMore());
		});
	}

	SECTION("SparseIndex")
	{
		TestSerialize([](FTestBatchBuilder& Batch, FScratchAllocator& Scratch)
		{
			FScopeId Unused = Batch.MakeScope("Unused");
			FScopeId NestedUnused1 = Batch.NestScope(Unused, "NestedUnused1");
			FScopeId FlatUsed = Batch.MakeScope("FlatUsed");
			FScopeId NestedUsed = Batch.NestScope(FlatUsed, "NestedUsed");
			FScopeId NestedUnused2 = Batch.NestScope(Unused, "NestedUnused2");
			FScopeId DoubleNested = Batch.NestScope(NestedUsed, "DoubleNested");
			FScopeId NestedUnused3 = Batch.NestScope(FlatUsed, "NestedUnused3");
			
			FType E1T{NestedUnused1,	Batch.MakeTypename("E1")};
			FType E2T{NestedUsed,		Batch.MakeTypename("E2")};
			FType E3T{NestedUnused2,	Batch.MakeTypename("E3")};
			
			FEnumId E1D = Batch.DeclareEnum(E1T, EEnumMode::Flat, ELeafWidth::B8, {"C1"}, {1});
			FEnumId E2D = Batch.DeclareEnum(E2T, EEnumMode::Flat, ELeafWidth::B8, {"C2"}, {2});
			FEnumId E3D = Batch.DeclareEnum(E3T, EEnumMode::Flat, ELeafWidth::B8, {"C3"}, {3});

			FType S1T{NestedUnused1,	Batch.MakeTypename("S1")};
			FType S2T{NestedUsed,		Batch.MakeTypename("S2")};
			FType S3T = Batch.MakeParametricType({NestedUnused2,	Batch.MakeTypename("S3")}, {S1T});
			FType S4T = Batch.MakeParametricType({DoubleNested,		Batch.MakeTypename("S4")}, {S2T, E2T});
			FType S5T = Batch.MakeParametricType({NestedUnused3,	Batch.MakeTypename("S5")}, {E3T, E1T, S2T});
			
			FDeclId S1D = Batch.DeclareStruct(S1T, {"M1"}, {SpecS32}, EMemberPresence::AllowSparse);
			FDeclId S2D = Batch.DeclareStruct(S2T, {"M2"}, {SpecS32}, EMemberPresence::AllowSparse);
			FDeclId S3D = Batch.DeclareStruct(S3T, {"M3"}, {SpecS32}, EMemberPresence::AllowSparse);
			FDeclId S4D = Batch.DeclareStruct(S4T, {"M4"}, {SpecS32}, EMemberPresence::AllowSparse);
			FDeclId S5D = Batch.DeclareStruct(S5T, {"M5"}, {SpecS32}, EMemberPresence::AllowSparse);

			FMemberBuilder Members;
			Members.Add(Batch.NameMember("M4"), 1);

			Batch.AddObject(S4D, MoveTemp(Members));
		}, 
		[](TConstArrayView<FStructView> Objects, const FTestNameReader& Names)
		{	
			FSchemaBatchId Batch = Objects[0].Schema.Batch;
			FType S4T = Objects[0].Schema.Resolve().Type;

			FNestedScope DoubleNested = ResolveUntranslatedNestedScope(Batch, S4T.Scope.AsNested());
			FNestedScope NestedUsed = ResolveUntranslatedNestedScope(Batch, DoubleNested.Outer.AsNested());
			FFlatScopeId FlatUsed = NestedUsed.Outer.AsFlat();
			CHECK(Names[DoubleNested.Inner.Name] == "DoubleNested");
			CHECK(Names[NestedUsed.Inner.Name] == "NestedUsed");
			CHECK(Names[FlatUsed.Name] == "FlatUsed");

			FParametricTypeView S4 = ResolveUntranslatedParametricType(Batch, S4T.Name.AsParametric());
			CHECK(Names[S4.Name.Get().Id] == "S4");
			CHECK(S4.NumParameters == 2);
			
			FType S2T = S4.Parameters[0];
			FType E2T = S4.Parameters[1];
			CHECK(S2T.Scope == DoubleNested.Outer);
			CHECK(E2T.Scope == DoubleNested.Outer);
			CHECK(Names[S2T.Name.AsConcrete().Id] == "S2");
			CHECK(Names[E2T.Name.AsConcrete().Id] == "E2");
		});
	}
}

////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FPlainPropsLoadSaveTest, "System::Core::Serialization::PlainProps::LoadSave", "[Core][PlainProps][SmokeFilter]")
{
	SECTION("Leaves")
	{}

	SECTION("Enums")
	{}
		
	SECTION("NestedStruct")
	{}
	
	SECTION("StaticArray")
	{}
	
	SECTION("LeafVariant")
	{}
	
	SECTION("BitfieldBool")
	{}

	SECTION("LeafArray")
	{}
	
	SECTION("LeafOptional")
	{}
	
	SECTION("LeafSmartPtr")
	{}

	SECTION("LeafSetWhole")
	{}

	SECTION("LeafSparseArrayAppends")
	{}

	SECTION("LeafSetOps")
	{}
	
	SECTION("SparseStructArray")
	{}

	SECTION("DenseStructArray")
	{}
	
	SECTION("SubStructArray")
	{}

	SECTION("NestedLeafArray")
	{}

	SECTION("NestedStructArray")
	{}

	SECTION("StructToSubStructMapOps")
	{}
}

} // namespace PlainProps
#endif // WITH_TESTS
