// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS && !defined(PLATFORM_COMPILER_IWYU)

#include "PlainPropsBuildSchema.h"
#include "PlainPropsCtti.h"
#include "PlainPropsIndex.h"
#include "PlainPropsInternalBuild.h"
#include "PlainPropsInternalDiff.h"
#include "PlainPropsInternalFormat.h"
#include "PlainPropsInternalParse.h"
#include "PlainPropsInternalPrint.h"
#include "PlainPropsInternalRead.h"
#include "PlainPropsInternalTest.h"
#include "PlainPropsDiff.h"
#include "PlainPropsLoad.h"
#include "PlainPropsRead.h"
#include "PlainPropsSave.h"
#include "PlainPropsVisualize.h"
#include "PlainPropsWrite.h"
#include "PlainPropsUeCoreBindings.h"
#include "Algo/Compare.h"
#include "Containers/StringConv.h"
#include "Containers/StringView.h"
#include "Containers/Map.h"
#include "Logging/LogMacros.h"
#include "Math/Transform.h"
#include "Templates/UnrealTemplate.h"
#include "Tests/TestHarnessAdapter.h"

DEFINE_LOG_CATEGORY_STATIC(LogPlainPropsUeCoreTests, Log, All);

namespace PlainProps::UE::Test
{

static TIdIndexer<FSensitiveName>	GNames;
static FDebugIds					GDebug(GNames);
static FEnumDeclarations			GEnums(GDebug);
static FSchemaBindings				GSchemas(GDebug);
static FCustomBindingsBottom		GCustoms(GDebug);
static FCustomBindingsOverlay		GDeltaCustoms(GCustoms);
static FNumeralGenerator			GNumeralGenerator(GNames);

struct FRuntimeIds
{
	static FMemberId			IndexNumeral(uint16 Numeral)		{ return GNumeralGenerator.Make(Numeral); }
	static FNameId				IndexName(FAnsiStringView Name)		{ return GNames.MakeName(FName(Name)); }
	static FMemberId			IndexMember(FAnsiStringView Name)	{ return GNames.NameMember(FName(Name)); }
	static FConcreteTypenameId	IndexTypename(FAnsiStringView Name)	{ return GNames.NameType(FName(Name)); }
	static FFlatScopeId			IndexScope(FAnsiStringView Name)	{ return GNames.NameScope(FName(Name)); }
	static FEnumId				IndexEnum(FType Type)				{ return GNames.IndexEnum(Type); }
	static FStructId			IndexStruct(FType Type)				{ return GNames.IndexStruct(Type); }
	static FIdIndexerBase&		GetIndexer()						{ return GNames; }
};

////////////////////////////////////////////////////////////////////////////////////////////////

struct FDefaultRuntime
{
	using Ids = FRuntimeIds;
	template<class T> using CustomBindings = TCustomBind<T>;

	static FEnumDeclarations&		GetEnums()			{ return GEnums; }
	static FSchemaBindings&			GetSchemas()		{ return GSchemas; }
	static FCustomBindings&			GetCustoms()		{ return GCustoms; }
};

////////////////////////////////////////////////////////////////////////////////////////////////

struct FDeltaRuntime : FDefaultRuntime
{
	template<class T> using CustomBindings = TCustomDeltaBind<T>;

	static FCustomBindings&			GetCustoms()		{ return GDeltaCustoms; }
};

////////////////////////////////////////////////////////////////////////////////////////////////

template<typename Enum, EEnumMode Mode>
using TScopedDefaultEnumDeclaration = TScopedEnumDeclaration<Enum, Mode, FDefaultRuntime>;

template<class T>
using TScopedDefaultStructBinding = TScopedStructBinding<T, FDefaultRuntime>;

////////////////////////////////////////////////////////////////////////////////////////////////


struct FNameBinding final : ICustomBinding
{
	const FDualStructId	Id;
	const FMemberId		IdxName;
	TSet<FName>			Names;

	FNameBinding()
	: Id(IndexStructDualId<FRuntimeIds, TTypename<FName>>())
	, IdxName(FRuntimeIds::IndexMember("Idx"))
	{}

	FStructDeclarationPtr Declare() const
	{
		return PlainProps::Declare({Id, NoId, 0, EMemberPresence::RequireAll, {IdxName}, {Specify<int32>()}});
	}

	virtual void SaveCustom(FMemberBuilder& Dst, const void* Src, FBaseline, const FSaveContext& Ctx) override
	{
		FSetElementId Idx = Names.Add(*static_cast<const FName*>(Src));
		Dst.Add(IdxName, Idx.AsInteger());
	}

	virtual void LoadCustom(void* Dst, FStructLoadView Src, ECustomLoadMethod) const override
	{
		FSetElementId Idx = FSetElementId::FromInteger(FMemberLoader(Src).GrabLeaf().AsS32());
		*static_cast<FName*>(Dst) = Names.Get(Idx);
	}

	virtual bool DiffCustom(const void* StructA, const void* StructB, const FBindContext&) const override
	{
		FName A = *static_cast<const FName*>(StructA);
		FName B = *static_cast<const FName*>(StructB);
		return !A.IsEqual(B, ENameCase::CaseSensitive);
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////

class FBatchSaver
{
public:
	explicit FBatchSaver(const FCustomBindings& CustomBase);
	virtual ~FBatchSaver() {}

	template<class T>
	void						Save(T&& Object);
	template<class T>
	bool						SaveDelta(const T& Object, const T& Default);

	TArray64<uint8>				Write(TArray<FStructId>* OutMemoryIds) const;

private:
	using IdBuiltStructPair = TPair<FStructId, const FBuiltStruct*>;
	TArray<IdBuiltStructPair>	SavedObjects;
	FNameBinding				SavedNames;
	FCustomBindingsOverlay		Customs;
	mutable FScratchAllocator	Scratch;
};

FBatchSaver::FBatchSaver(const FCustomBindings& Underlay)
: Customs(Underlay)
{
	Customs.BindStruct(UpCast(SavedNames.Id), SavedNames, SavedNames.Declare(), {});
}

template<class T>
void FBatchSaver::Save(T&& Object) 
{
	FBothStructId Id = IndexStructBothId<FRuntimeIds, TTypename<std::remove_reference_t<T>>>();
	SavedObjects.Emplace(Id.DeclId, SaveStruct(&Object, Id.BindId, {{GSchemas, Customs}, Scratch}));
}

template<class T>
bool FBatchSaver::SaveDelta(const T& Object, const T& Default) 
{
	FBothStructId Id = IndexStructBothId<FRuntimeIds, TTypename<std::remove_reference_t<T>>>();
	if (const FBuiltStruct* Delta = SaveStructDeltaIfDiff(&Object, &Default, Id.BindId, {{GSchemas, Customs}, Scratch}))
	{
		SavedObjects.Emplace(Id.DeclId, MoveTemp(Delta));
		return true;
	}
	return false;
}

template<typename ArrayType>
void WriteNumAndArray(TArray64<uint8>& Out, const ArrayType& Items)
{
	WriteInt(Out, IntCastChecked<uint32>(Items.Num()));
	WriteArray(Out, Items);
}

template<typename T>
TConstArrayView<T> GrabNumAndArray(/* in-out */ FByteReader& It)
{
	uint32 Num = It.Grab<uint32>();
	return MakeArrayView(reinterpret_cast<const T*>(It.GrabBytes(Num * sizeof(T))), Num);
}

inline constexpr uint32 Magics[] = { 0xFEEDF00D, 0xABCD1234, 0xDADADAAA, 0x99887766, 0xF0F1F2F3 , 0x00112233};

TArray64<uint8> FBatchSaver::Write(TArray<FStructId>* OutMemoryIds) const
{
	ESchemaFormat Format = OutMemoryIds ? ESchemaFormat::InMemoryNames : ESchemaFormat::StableNames;

	// Build partial schemas
	const FBindDeclarations Types(GEnums, Customs, GSchemas);
	FSchemasBuilder SchemaBuilders(GNames, Types,  Scratch, Format);
	for (const IdBuiltStructPair& Object : SavedObjects)
	{
		SchemaBuilders.NoteStructAndMembers(Object.Key, *Object.Value);
	}
	FBuiltSchemas Schemas = SchemaBuilders.Build();
	if (OutMemoryIds)
	{
		*OutMemoryIds = ExtractRuntimeIds(Schemas);
	}

	// Filter out declared but unused names and ids
	FWriter Writer(GNames, Types, Schemas, Format);
	TArray<FSensitiveName> UsedNames;
	UsedNames.Reserve(Writer.GetUsedNames().Num());
	for (FNameId Name : Writer.GetUsedNames())
	{
		UsedNames.Add(GNames.ResolveName(Name));
	}
	
	// Write ids. Just copying in-memory FNames, a stable format might use SaveNameBatch().
	TArray64<uint8> Out;
	WriteInt(Out, Magics[0]);
	WriteNumAndArray(Out, TArrayView<const FSensitiveName, int32>(UsedNames));

	// Write schemas
	WriteInt(Out, Magics[1]);
	WriteAlignmentPadding<uint32>(Out);
	TArray64<uint8> Tmp;
	Writer.WriteSchemas(/* Out */ Tmp);
	WriteNumAndArray(Out, TArrayView<const uint8, int64>(Tmp));
	Tmp.Reset();

	// Write objects
	WriteInt(Out, Magics[2]);
	for (const TPair<FStructId, const FBuiltStruct*>& Object : SavedObjects)
	{
		WriteInt(/* out */ Tmp, Magics[3]);
		WriteInt(/* out */ Tmp, Writer.GetWriteId(Object.Key).Get().Idx);
		Writer.WriteMembers(/* out */ Tmp, Object.Key, *Object.Value);
		WriteSkippableSlice(Out, Tmp);
		Tmp.Reset();
	}

	// Write object terminator
	WriteSkippableSlice(Out, TConstArrayView64<uint8>());
	WriteInt(Out, Magics[4]);
	
	// Write names
	WriteNumAndArray(Out, SavedNames.Names.Array());
	WriteInt(Out, Magics[5]);
		
	return Out;
}

////////////////////////////////////////////////////////////////////////////////////////////////

class FStableNameBatchIds final : public FStableBatchIds
{
	TConstArrayView<FSensitiveName> Names;
public:
	FStableNameBatchIds(FSchemaBatchId Batch, TConstArrayView<FSensitiveName> InNames) : FStableBatchIds(Batch), Names(InNames) {}
	using FStableBatchIds::AppendString;

	virtual uint32				NumNames() const override							{ return static_cast<uint32>(Names.Num()); }
	virtual void				AppendString(FUtf8Builder& Out, FNameId Name) const override { Names[Name.Idx].AppendString(Out); }
};

class FTranslationBatchIds final : public FBatchIds
{
	FIdBinding Binding;
public:
	using FBatchIds::Resolve;
	using FBatchIds::AppendString;

	FTranslationBatchIds(FSchemaBatchId Batch, FIdBinding&& InBinding) : FBatchIds(Batch), Binding(InBinding) {}

	virtual uint32				NumNames() const override												{ return GNames.NumNames(); }
	virtual void				AppendString(FUtf8Builder& Out, FNameId Name) const override			{ PlainProps::AppendString(Out, GNames.ResolveName(Name)); }
	virtual void				AppendString(FUtf8Builder& Out, FTypenameId Typename) const override	{ GNames.AppendString(Out, Typename); }
	virtual void				AppendString(FUtf8Builder& Out, FScopeId Scope) const override			{ GNames.AppendString(Out, Scope); }

	virtual uint32				NumNestedScopes() const override					{ return static_cast<uint32>(Binding.NestedScopes.Num()); }
	virtual uint32				NumParametricTypes() const override					{ return static_cast<uint32>(Binding.ParametricTypes.Num()); }
	virtual FNestedScope		Resolve(FNestedScopeId Id) const override			{ return GNames.Resolve(Binding.Remap(Id)); }
	virtual FParametricTypeView	Resolve(FParametricTypeId Id) const override		{ return GNames.Resolve(Binding.Remap(Id)); }
};

static void RoundtripText(const FBatchIds& BatchIds, TConstArrayView<FStructView> Objects, ESchemaFormat Format)
{
	// Print yaml
	TUtf8StringBuilder<4096> Yaml;
	PrintYamlBatch(Yaml, BatchIds, Objects);
	FUtf8StringView YamlView = Yaml.ToView();

	// Log yaml
	auto Wide = StringCast<TCHAR>(YamlView.GetData(), YamlView.Len());
	UE_LOGF(LogPlainPropsUeCoreTests, Log, "Schemas with %ls:\n%.*ls",
		Format == ESchemaFormat::InMemoryNames ? TEXT("InMemoryNames") : TEXT("StableNames"),
		Wide.Length(), Wide.Get());

	// Parse yaml
	TArray64<uint8> Data;
	TArray<FStructView> ParsedObjects;
	FSchemaBatchId ParsedBatch = ParseYamlBatch(Data, ParsedObjects, YamlView);

	if (Format == ESchemaFormat::StableNames)
	{
		// Diff schemas
		CHECK_FALSE(DiffSchemas(BatchIds.GetBatchId(), ParsedBatch));

		// Diff objects
		CHECK(Objects.Num() == ParsedObjects.Num());
		const int32 NumObjects = FMath::Min(Objects.Num(), ParsedObjects.Num());
		for (int32 I = 0; I < NumObjects; ++I)
		{
			FStructView In = Objects[I];
			FStructView Parsed = ParsedObjects[I];
			FReadDiffPath DiffPath;
			if (DiffStruct(In, Parsed, DiffPath))
			{
				TUtf8StringBuilder<256> Diff;
				PrintDiff(Diff, BatchIds, DiffPath);
				FAIL_CHECK(FString::Printf(TEXT("Diff in '%s' in Objects[%d]"), *Print(Diff.ToString()), I));
			}
		}
	}

	// Unmount parsed schemas
	UnmountReadSchemas(ParsedBatch);
}

static void PrintLoadStruct(const FBatchIds& BatchIds, FStructView StructView, int32 LoadIdx)
{
	TUtf8StringBuilder<4096> YamlString;
	{
		FYamlBuilderPtr YamlBuilder = MakeYamlBuilder(YamlString);
		FBatchPrinter Printer(*YamlBuilder, BatchIds);
		Printer.PrintObjects({StructView});
	}
	FUtf8StringView YamlView = YamlString.ToView();
	auto Wide = StringCast<TCHAR>(YamlView.GetData(), YamlView.Len());
	UE_LOGF(LogPlainPropsUeCoreTests, Log, "LoadStruct %d:\n%.*ls", LoadIdx, Wide.Length(), Wide.Get());
}

////////////////////////////////////////////////////////////////////////////////////////////////

class FBatchLoader
{
public:
	FBatchLoader(FMemoryView Data, const FCustomBindings& Underlay, TConstArrayView<FStructId> InRuntimeIds)
	: Customs(Underlay)
	, Format(InRuntimeIds.IsEmpty() ? ESchemaFormat::StableNames : ESchemaFormat::InMemoryNames)
	{
		// Read ids
		FByteReader It(Data);
		CHECK(It.Grab<uint32>() == Magics[0]);
		Names = GrabNumAndArray<FSensitiveName>(It);
		CHECK(Names.IsEmpty() != InRuntimeIds.IsEmpty());
		
		// Read schemas
		CHECK(It.Grab<uint32>() == Magics[1]);
		It.SkipAlignmentPadding<uint32>();
		uint32 SchemasSize = It.Grab<uint32>();
		FMemoryView	SavedSchemasView = It.GrabSlice(SchemasSize);
		SavedSchemas = ValidateSchemas(SavedSchemasView);
		CHECK(It.Grab<uint32>() == Magics[2]);


		FSchemaBatchId Batch;
		TOptional<FIdTranslator> RuntimeIds;
		if (InRuntimeIds.IsEmpty())
		{
			// Bind saved ids to runtime ids, make new schemas with new ids and mount them
			RuntimeIds.Emplace(GNames, Names, *SavedSchemas);
			TranslatedSchemas = CreateTranslatedSchemas(*SavedSchemas, RuntimeIds->Translation);
			Batch = MountReadSchemas(TranslatedSchemas);
		}
		else
		{
			// Mount saved schemas as is
			Batch = MountReadSchemas(SavedSchemas);	

			TArray<FStructId> ExpectedRuntimeIds = IndexStructRuntimeIds(*SavedSchemas, GNames);
			check(EqualItems(ExpectedRuntimeIds, InRuntimeIds));
		}
		
		// Read objects
		while (uint64 NumBytes = It.GrabVarIntU())
		{	
			FByteReader ObjIt(It.GrabSlice(NumBytes));
			CHECK(ObjIt.Grab<uint32>() == Magics[3]);
			FStructSchemaId Schema = { ObjIt.Grab<uint32>() };
			Objects.Add({ { Schema, Batch }, ObjIt });
		}
		
		CHECK(It.Grab<uint32>() == Magics[4]);
		CHECK(!Objects.IsEmpty());

		// Read names and bind custom loader
		NameBinding.Names.Append(GrabNumAndArray<FName>(It));
		Customs.BindStruct(UpCast(NameBinding.Id), NameBinding, NameBinding.Declare(), {});
		CHECK(It.Grab<uint32>() == Magics[5]);

		// Finally create load plans
		TConstArrayView<FStructId> LoadStructIds = RuntimeIds ? RuntimeIds->Translation.GetStructIds(SavedSchemas->NumStructSchemas) : InRuntimeIds;
		Plans = CreateLoadPlans(Batch, Customs, GSchemas, LoadStructIds, Format);

		// Create BatchIds for the loading phase
		if (RuntimeIds)
		{
			LoadBatchIds = MakeUnique<FTranslationBatchIds>(Batch, MoveTemp(RuntimeIds->Translation));
		}
		else
		{
			LoadBatchIds = MakeUnique<FMemoryBatchIds>(Batch, GNames);
		}
	}

	~FBatchLoader()
	{
		CHECK(LoadIdx == Objects.Num()); // Test should load all saved objects
		Plans.Reset();
		UnmountReadSchemas(Objects[0].Schema.Batch);
		if (TranslatedSchemas)
		{
			DestroyTranslatedSchemas(TranslatedSchemas);
		}
	}

	void RoundtripText() const
	{
		if (Format == ESchemaFormat::StableNames)
		{
			// Mount and use the saved schemas rather than the translated load schemas
			// in order to verify that text roundtripping creates identical serialized ids.
			FSchemaBatchId Batch = MountReadSchemas(SavedSchemas);
			TArray<FStructView> StableObjects(Objects);
			for (FStructView& Struct : StableObjects)
			{
				Struct.Schema.Batch = Batch;
			}

			FStableNameBatchIds StableBatchIds(Batch, Names);
			PlainProps::UE::Test::RoundtripText(StableBatchIds, StableObjects, Format);
			UnmountReadSchemas(Batch);
		}
		else
		{
			PlainProps::UE::Test::RoundtripText(*LoadBatchIds, Objects, Format);
		}
	}

	template<class T>
	T Load()
	{
		T Out;
		LoadInto(Out);
		return MoveTemp(Out);
	}
	
	template<class T>
	T Load(const T& Original)
	{
		T Out = Original;
		LoadInto(Out);
		return MoveTemp(Out);
	}

	template<class T>
	void LoadInto(T& Out)
	{
		FStructView In = Objects[LoadIdx];
		PrintLoadStruct(*LoadBatchIds, In, LoadIdx);
		LoadStruct(&Out, In.Values, In.Schema.Id, *Plans);
		++LoadIdx;
	}

	const FBatchIds&				GetBatchIds() { return *LoadBatchIds; }

private:
	const FSchemaBatch*				SavedSchemas;
	const FSchemaBatch*				TranslatedSchemas = nullptr;
	TConstArrayView<FSensitiveName>	Names;
	FNameBinding					NameBinding;
	TUniquePtr<FBatchIds>			LoadBatchIds;
	FCustomBindingsOverlay			Customs;
	FLoadBatchPtr					Plans;
	TArray<FStructView>				Objects;
	int32							LoadIdx = 0;
	ESchemaFormat					Format;
};

static void Run(void (*Save)(FBatchSaver&), void (*Load)(FBatchLoader&), const FCustomBindings& Customs = GCustoms)
{
	for (ESchemaFormat Format : {ESchemaFormat::StableNames, ESchemaFormat::InMemoryNames})
	{
		TArray64<uint8> Data;
		TArray<FStructId> RuntimeIds;
		{
			FBatchSaver Batch(Customs);
			Save(Batch);
			Data = Batch.Write(Format == ESchemaFormat::InMemoryNames ? &RuntimeIds : nullptr);
		}

		FBatchLoader Batch(MakeMemoryView(Data), Customs, MakeArrayView(RuntimeIds));
		Batch.RoundtripText();
		Load(Batch);
	}
}

//////////////////////////////////////////////////////////////////////////

//struct FNestedStructs
//{
//	FPt Point;
//	FPt StaticPoints[2];
//};
//UE_REFLECT_STRUCT(FNestedStructs, Point, StaticPoints);
//
//struct FLeafRanges
//{
//	TArray<int32> IntArray;
//	TOptional<uint8> MaybeByte;
//	TUniquePtr<float> FloatPtr;
//	TSet<uint16> ShortSet;
//	TSparseArray<bool> SparseBools;
//};
//
//struct FSuper
//{
//	uint16 Pad;
//	bool A;
//};
//UE_REFLECT_STRUCT(FSuper, A);
//
//struct FSub : FSuper
//{
//	bool B;
//	uint32 Pad;
//};
//UE_REFLECT_SUBSTRUCT(FSub, FSuper, B);
//
//
//struct FSubs
//{
//	TArray<FSub> Subs;
//};

struct FInt { int32 X; };
PP_REFLECT_STRUCT(PlainProps::UE::Test, FInt, void, X);
static bool operator==(FInt A, FInt B) { return A.X == B.X; }
inline uint32 GetTypeHash(FInt I) { return ::GetTypeHash(I.X); }

enum class EFlat1 : int8 { A = -1, B = 3 };
enum class EFlat2 : uint8 { A, B };
enum class EFlag1 : uint8 { A = 2, B = 8, AB = 10 };
enum class EFlag2 : uint8 { A = 1, B = 2, AB = 3 };
PP_REFLECT_ENUM(PlainProps::UE::Test, EFlat1, A, B);
PP_REFLECT_ENUM(PlainProps::UE::Test, EFlat2, A, B);
PP_REFLECT_ENUM(PlainProps::UE::Test, EFlag1, A, B);
PP_REFLECT_ENUM(PlainProps::UE::Test, EFlag2, A, B);

struct FEnums
{
	EFlat1 Flat1;
	EFlat2 Flat2;
	EFlag1 Flag1;
	EFlag2 Flag2;

	friend bool operator==(FEnums, FEnums) = default;
};
PP_REFLECT_STRUCT(PlainProps::UE::Test, FEnums, void, Flat1, Flat2, Flag1, Flag2);

enum EByteFlat : int32 { EByteFlat_A = 1, EByteFlat_B = 3 };
enum EByteFlag : uint8 { EByteFlag_A = 2, EByteFlag_B = 8, EByteFlag_AB = 10 };
PP_REFLECT_ENUM(PlainProps::UE::Test, EByteFlat, EByteFlat_A, EByteFlat_B);
PP_REFLECT_ENUM(PlainProps::UE::Test, EByteFlag, EByteFlag_A, EByteFlag_B);

struct FEnumAsBytes
{
	EByteFlat Flat;
	EByteFlag Flag;
	TEnumAsByte<EByteFlat> FlatAsByte;
	TEnumAsByte<EByteFlag> FlagAsByte;

	friend bool operator==(FEnumAsBytes, FEnumAsBytes) = default;
};
PP_REFLECT_STRUCT(PlainProps::UE::Test, FEnumAsBytes, void, Flat, Flag, FlatAsByte, FlagAsByte);

struct FLeafArrays
{
	TArray<bool> Bits;
	TArray<int>	Bobs;

	bool operator==(const FLeafArrays&) const = default;
};
PP_REFLECT_STRUCT(PlainProps::UE::Test, FLeafArrays, void, Bits, Bobs);

struct FComplexArrays
{
	TArray<char> Str;
	TArray<EFlat1> Enums;
	TArray<FLeafArrays> Misc;
	TArray<TArray<EFlat1>> Nested;

	bool operator==(const FComplexArrays&) const = default;
};
PP_REFLECT_STRUCT(PlainProps::UE::Test, FComplexArrays, void, Str, Enums, Misc, Nested);

struct FNames
{
	FName Name;
	TArray<FName> Names;

	bool operator==(const FNames&) const = default;
};
PP_REFLECT_STRUCT(PlainProps::UE::Test, FNames, void, Name, Names);

struct FSensitive
{
	int32 X;
	int32 X_0;
	int32 X_1;
	int32 x;
	int32 x_0;
	int32 x_1;
	bool operator==(const FSensitive&) const = default;
};
PP_REFLECT_STRUCT(PlainProps::UE::Test, FSensitive, void, X, X_0, X_1, x, x_0, x_1);

struct FSensitiveReversed
{
	int32 x_1;
	int32 x_0;
	int32 x;
	int32 X_1;
	int32 X_0;
	int32 X;
	bool operator==(const FSensitiveReversed&) const = default;
};
PP_REFLECT_STRUCT(PlainProps::UE::Test, FSensitiveReversed, void, x_1, x_0, x, X_1, X_0, X);

struct FStr
{
	FString S;
};
PP_REFLECT_STRUCT(PlainProps::UE::Test, FStr, void, S);

struct FNDC
{
	int X = -1;
	explicit FNDC(int I) : X(I) {}
	friend bool operator==(FNDC A, FNDC B) = default;
};
PP_REFLECT_STRUCT(PlainProps::UE::Test, FNDC, void, X);

struct FNDCBinding : ICustomBinding
{
	using Type = FNDC;
	const FMemberId MemberIds[1];

	template<class Ids>
	FNDCBinding(TCustomSpecifier<Ids, 1>& Spec)
	: MemberIds{Ids::IndexMember("X")}
	{
		Spec.Members[0] = Specify<int32>();
	}

	void Save(FMemberBuilder& Dst, const FNDC& Src, const FNDC*, const FSaveContext&) const
	{
		Dst.Add(MemberIds[0], Src.X);
	}

	void Load(FNDC& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
	{
		int X = FMemberLoader(Src).GrabLeaf().AsS32();
		if (Method == ECustomLoadMethod::Construct)
		{
			new (&Dst) FNDC { X };
		}
		else
		{
			Dst.X = X;
		}
	}

	static bool Diff(FNDC A, FNDC B, const FBindContext&)
	{
		return !(A == B);
	}
};

} namespace PlainProps {
template<> struct TCustomBind<UE::Test::FNDC> {	using Type = UE::Test::FNDCBinding; };
} namespace PlainProps::UE::Test {

struct FSets
{
	TSet<char> Leaves;
	TSet<TArray<uint8>> Ranges;
	TSet<FInt> Structs;
	TSet<FString> Strings; // ILeafRangeBinding
};
PP_REFLECT_STRUCT(PlainProps::UE::Test, FSets, void, Leaves, Ranges, Structs, Strings);


template<typename T>
bool ContainsElem(const TSet<T>& Set, const T& Elem)
{
	return Set.Contains(Elem);
}

template<typename K, typename V>
bool ContainsElem(const TMap<K,V>& Map, const TPair<K,V>& Elem)
{
	const auto* Value = Map.Find(Elem.Key);
	return Value && *Value == Elem.Value;
}

enum ECompare { Order, Content };

template<ECompare Cmp, typename ContainerType>
inline bool Equals(const ContainerType& A, const ContainerType& B)
{
	if (A.Num() != B.Num())
	{
		return false;
	}

	typename ContainerType::TConstIterator ItB(B);
	for (const auto& ElemA : A)
	{
		if (Cmp == ECompare::Order ? ElemA != *ItB : !ContainsElem(B, ElemA))
		{
			return false;
		}
		++ItB;
	}
	return true;
}
template<ECompare Cmp>
bool Same(const FSets& A, const FSets& B)
{
	return Equals<Cmp>(A.Leaves, B.Leaves) && Equals<Cmp>(A.Ranges, B.Ranges) && Equals<Cmp>(A.Structs, B.Structs) && Equals<Cmp>(A.Strings, B.Strings);
}

struct FMaps
{
	TMap<bool, bool> Leaves;
	TMap<int, TArray<char>> Ranges;
	TMap<FInt, FNDC> Structs;
	TMap<FString, FString> Strings; // ILeafRangeBinding
};
PP_REFLECT_STRUCT(PlainProps::UE::Test, FMaps, void, Leaves, Ranges, Structs, Strings);

template<ECompare Cmp>
bool Same(const FMaps& A, const FMaps& B)
{
	return Equals<Cmp>(A.Leaves, B.Leaves) && Equals<Cmp>(A.Ranges, B.Ranges) && Equals<Cmp>(A.Structs, B.Structs) && Equals<Cmp>(A.Strings, B.Strings);
}

struct FIntAlias
{
	int X;
	bool operator==(const FIntAlias&) const = default;
};

struct FSame1
{
	int X = 1;
	bool operator==(const FSame1&) const = default;
};

struct FSame2
{ 
	int Unused;
	int X = 2;
	bool operator==(const FSame2& O) const { return X == O.X; };
};


PP_REFLECT_STRUCT(PlainProps::UE::Test, FIntAlias, void, X);
PP_REFLECT_STRUCT(PlainProps::UE::Test, FSame1, void, X);
PP_REFLECT_STRUCT(PlainProps::UE::Test, FSame2, void, X);

struct FVariants
{
	TVariant<bool, float> BoolFloat;
	TVariant<float, FInt> FloatInt;
	TVariant<FEmptyVariantState, FNDC> NDC;
	TVariant<float, TArray<FInt>> FloatIntArray;
	TVariant<int32, TVariant<int32, FNDC>> IntVariantNDC;
};

PP_REFLECT_STRUCT(PlainProps::UE::Test, FVariants, void, BoolFloat, FloatInt, NDC, FloatIntArray, IntVariantNDC);

} namespace PlainProps {

template <> struct TTypename<PlainProps::UE::Test::FIntAlias>
{
	inline static constexpr std::string_view DeclName = "FInt";
	inline static constexpr std::string_view BindName = "IntAlias";
};
template <> struct TTypename<PlainProps::UE::Test::FSame1>
{
	inline static constexpr std::string_view DeclName = "Same";
	inline static constexpr std::string_view BindName = "Same1";
};
template <> struct TTypename<PlainProps::UE::Test::FSame2>
{
	inline static constexpr std::string_view DeclName = "Same";
	inline static constexpr std::string_view BindName = "Same2";
};

} namespace PlainProps::UE::Test {

struct FTypeErasure
{
	FSame1 A;
	FSame2 B;
	FIntAlias C;
	TPair<FString, TArray<char8_t>> D;
	TPair<TArray<char>, TArray<char, TInlineAllocator<8>>> E;
	bool operator==(const FTypeErasure&) const = default;
};
PP_REFLECT_STRUCT(PlainProps::UE::Test, FTypeErasure, void, A, B, C, D, E); 

//////////////////////////////////////////////////////////////////////////

struct FUniquePtrs
{
	TUniquePtr<bool> Bit;
	TUniquePtr<FInt> Struct;
	TUniquePtr<TUniquePtr<int>> IntPtr;
	TArray<TUniquePtr<double>> Doubles;
};
PP_REFLECT_STRUCT(PlainProps::UE::Test, FUniquePtrs, void, Bit, Struct, IntPtr, Doubles);

template<typename T>
bool SameValue(const TUniquePtr<T>& A, const TUniquePtr<T>& B)
{
	return !A == !B && (!A || *A == *B); 
}

static bool operator==(const FUniquePtrs& A, const FUniquePtrs& B) 
{ 
	return	SameValue(A.Bit, B.Bit) &&
			SameValue(A.Struct, B.Struct) &&
			!A.IntPtr == !B.IntPtr && (!A.IntPtr || SameValue(*A.IntPtr, *B.IntPtr)) && 
			Algo::Compare(A.Doubles, B.Doubles, [](auto& X, auto& Y) { return SameValue(X, Y); });
}

template<typename T>
TUniquePtr<T> MakeOne(T&& Value)
{
	return MakeUnique<T>(MoveTemp(Value));
}

template<typename T>
TArray<TUniquePtr<T>> MakeTwo(T&& A, T&& B)
{
	TArray<TUniquePtr<T>> Out;
	Out.Add(MakeOne(MoveTemp(A)));
	Out.Add(MakeOne(MoveTemp(B)));
	return Out;
}

//////////////////////////////////////////////////////////////////////////

struct FNDCIntrusive : FNDC
{
	FNDCIntrusive() : FNDC(-1) {}
	explicit FNDCIntrusive(FIntrusiveUnsetOptionalState) : FNDC(-1) {}
	explicit FNDCIntrusive(int I) : FNDC(I) {}
	friend bool operator==(FNDCIntrusive, FNDCIntrusive) = default;
	bool operator==(FIntrusiveUnsetOptionalState) const { return X == -1;}
};
PP_REFLECT_STRUCT(PlainProps::UE::Test, FNDCIntrusive, void, X);

struct FOpts
{
	TOptional<bool> Bit;
	TOptional<FNDC> NDC;
	TOptional<FNDCIntrusive> NDCI;

	friend bool operator==(const FOpts&, const FOpts&) = default;
};
PP_REFLECT_STRUCT(PlainProps::UE::Test, FOpts, void, Bit, NDC, NDCI);

//////////////////////////////////////////////////////////////////////////

struct FDelta
{
	bool		A = true;
	float		B = 1.0;
	FInt		C = { 2 };
	TArray<int> D;
	FString		E = "!";

	friend bool operator==(const FDelta& A, const FDelta& B) = default;
};
PP_REFLECT_STRUCT(PlainProps::UE::Test, FDelta, void, A, B, C, D, E);

struct FDenseDelta
{
	int32 X = 0;
	int32 Y = 0;
	friend bool operator==(const FDenseDelta& A, const FDenseDelta& B) = default;
};

PP_REFLECT_STRUCT(PlainProps::UE::Test, FDenseDelta, void, X, Y);

} namespace PlainProps {
template<> struct TOccupancyOf<PlainProps::UE::Test::FDenseDelta> : FRequireAll {};
} namespace PlainProps::UE::Test {

//////////////////////////////////////////////////////////////////////////
//
//struct FObject
//{
//	virtual ~FObject() {}
//	char Rtti = '\0';
//	int Id = 0;
//};
//
//struct FObjectReferenceBinding : ICustomBinding
//{
//	using Type = FObject*;
//
//	static TConstArrayView<FMemberId> GetMemberIds()
//	{
//		static FMemberId Ids[] = {Ids::IndexMember("Type"), Ids::IndexMember("Id")};
//		return MakeArrayView(Ids, 2);
//	}
//
//	virtual void SaveStruct(FMemberBuilder& Dst, const void* Src, const void*, const FSaveContext&) const override
//	{
//		if (const FObject* Object = reinterpret_cast<const FObject*>(Src))
//		{
//			Dst.AddLeaf(GetMemberIds()[0], Object->Rtti);
//			Dst.AddLeaf(GetMemberIds()[1], Object->Id);
//		}
//	}
//
//	virtual void LoadStruct(void* Dst, FStructView Src, ECustomLoadMethod, const FLoadBatch&) const override
//	{
//		FMemberReader Members(Src);
//		check(Members.PeekName() == GetMemberIds()[0]);
//		char Rtti = static_cast<char>(Members.GrabLeaf().AsChar8());
//		check(Members.PeekName() == GetMemberIds()[1]);
//		int Id = Members.GrabLeaf().AsS32();
//		check(!Members.HasMore());
//
//	}
//}
//
//PP_REFLECT_STRUCT(PlainProps::UE::Test, FObject, void, Rtti, Sibling);
//
//struct FObjectX : public FObject
//{
//	FObjectX() { Rtti = 'x'; }
//	FObject* Sibling = nullptr;
//};
//PP_REFLECT_STRUCT(PlainProps::UE::Test, FObjectX, FObject, X);
//
//struct FObjectY : public FObject
//{
//	FObjectY() { Rtti = 'y'; }
//	FObject* Sibling = nullptr;
//};
//PP_REFLECT_STRUCT(PlainProps::UE::Test, FObjectY, FObject, X);
//
//struct FOwner
//{
//	TArray<TUniquePtr<FObject>> Objects;
//};
//PP_REFLECT_STRUCT(PlainProps::UE::Test, FOwner, void, Objects);

//////////////////////////////////////////////////////////////////////////

template<typename T>
TArray<T> MakeArray(const T* Str)
{
	return TArray<T>(Str, TCString<T>::Strlen(Str));
}

template<int N, typename T>
TArray<T, TInlineAllocator<N>> MakeInlArray(const T* Str)
{
	return TArray<T, TInlineAllocator<N>>(Str, TCString<T>::Strlen(Str));
}

TEST_CASE_NAMED(FPlainPropsUeCoreTest, "System::Core::Serialization::PlainProps::UE::Core", "[Core][PlainProps][SmokeFilter]")
{
	DbgVis::FIdScope _(GNames, "SensName");

	SECTION("Basic")
	{
		TScopedDefaultStructBinding<FInt> Int;
		Run([](FBatchSaver& Batch)
			{
				Batch.Save(FInt{1234});
			}, 
			[](FBatchLoader& Batch)
			{
				CHECK(Batch.Load<FInt>().X == 1234);
			});
	}
	
	SECTION("Enum")
	{
		TScopedDefaultEnumDeclaration<EFlat1, EEnumMode::Flat> Flat1;
		TScopedDefaultEnumDeclaration<EFlat2, EEnumMode::Flat> Flat2;
		TScopedDefaultEnumDeclaration<EFlag1, EEnumMode::Flag> Flag1;
		TScopedDefaultEnumDeclaration<EFlag2, EEnumMode::Flag> Flag2;
		TScopedDefaultStructBinding<FEnums> Int;
		Run([](FBatchSaver& Batch)
			{
				Batch.Save(FEnums{EFlat1::A, EFlat2::A, EFlag1::A, EFlag2::A});
				Batch.Save(FEnums{EFlat1::A, EFlat2::A, EFlag1::B, EFlag2::B});
				Batch.Save(FEnums{EFlat1::B, EFlat2::B, EFlag1::A, EFlag2::A});
				Batch.Save(FEnums{EFlat1::B, EFlat2::B, EFlag1::B, EFlag2::B});
				Batch.Save(FEnums{EFlat1::B, EFlat2::B, EFlag1::AB, EFlag2::AB});
			}, 
			[](FBatchLoader& Batch)
			{
				CHECK(Batch.Load<FEnums>() == FEnums{EFlat1::A, EFlat2::A, EFlag1::A, EFlag2::A});
				CHECK(Batch.Load<FEnums>() == FEnums{EFlat1::A, EFlat2::A, EFlag1::B, EFlag2::B});
				CHECK(Batch.Load<FEnums>() == FEnums{EFlat1::B, EFlat2::B, EFlag1::A, EFlag2::A});
				CHECK(Batch.Load<FEnums>() == FEnums{EFlat1::B, EFlat2::B, EFlag1::B, EFlag2::B});
				CHECK(Batch.Load<FEnums>() == FEnums{EFlat1::B, EFlat2::B, EFlag1::AB, EFlag2::AB});
			});
	}

	SECTION("TEnumAsByte")
	{
		TScopedDefaultEnumDeclaration<EByteFlat, EEnumMode::Flat> Flat;
		TScopedDefaultEnumDeclaration<EByteFlag, EEnumMode::Flag> Flag;
		TScopedDefaultStructBinding<FEnumAsBytes> EnumAsBytes;
		Run([](FBatchSaver& Batch)
			{
				Batch.Save(FEnumAsBytes{EByteFlat_A, EByteFlag_A,  EByteFlat_A,  EByteFlag_A});
				Batch.Save(FEnumAsBytes{EByteFlat_B, EByteFlag_B,  EByteFlat_B,  EByteFlag_B});
				Batch.Save(FEnumAsBytes{EByteFlat_A, EByteFlag_A,  EByteFlat_A,  EByteFlag_A});
				Batch.Save(FEnumAsBytes{EByteFlat_B, EByteFlag_B,  EByteFlat_B,  EByteFlag_B});
				Batch.Save(FEnumAsBytes{EByteFlat_B, EByteFlag_AB, EByteFlat_B,  EByteFlag_AB});
			},
			[](FBatchLoader& Batch)
			{
				CHECK(Batch.Load<FEnumAsBytes>() == FEnumAsBytes{EByteFlat_A, EByteFlag_A,  EByteFlat_A, EByteFlag_A});
				CHECK(Batch.Load<FEnumAsBytes>() == FEnumAsBytes{EByteFlat_B, EByteFlag_B,  EByteFlat_B, EByteFlag_B});
				CHECK(Batch.Load<FEnumAsBytes>() == FEnumAsBytes{EByteFlat_A, EByteFlag_A,  EByteFlat_A, EByteFlag_A});
				CHECK(Batch.Load<FEnumAsBytes>() == FEnumAsBytes{EByteFlat_B, EByteFlag_B,  EByteFlat_B, EByteFlag_B});
				CHECK(Batch.Load<FEnumAsBytes>() == FEnumAsBytes{EByteFlat_B, EByteFlag_AB, EByteFlat_B, EByteFlag_AB});
			});
	}

	SECTION("TArray")
	{
		TScopedDefaultStructBinding<FLeafArrays> LeafArrays;
		Run([](FBatchSaver& Batch)
			{
				Batch.Save(FLeafArrays{{}, {}});
				Batch.Save(FLeafArrays{{false}, {1, 2}});
				Batch.Save(FLeafArrays{{true, false}, {3, 4, 5}});
				Batch.Save(FLeafArrays{{true,true,true,true,true,true,true,true,false,true}});
			}, 
			[](FBatchLoader& Batch)
			{
				CHECK(Batch.Load<FLeafArrays>() == FLeafArrays{{}, {}});
				CHECK(Batch.Load<FLeafArrays>() == FLeafArrays{{false}, {1, 2}});
				CHECK(Batch.Load<FLeafArrays>() == FLeafArrays{{true, false}, {3, 4, 5}});
				CHECK(Batch.Load<FLeafArrays>() == FLeafArrays{{true,true,true,true,true,true,true,true,false,true}});
			});
	}

	SECTION("Nesting")
	{
		TScopedDefaultEnumDeclaration<EFlat1, EEnumMode::Flat> Flat1;
		TScopedDefaultStructBinding<FLeafArrays> LeafArrays;
		TScopedDefaultStructBinding<FComplexArrays> ComplexArrays;
		Run([](FBatchSaver& Batch)
			{
				Batch.Save(FComplexArrays{});
				Batch.Save(FComplexArrays{{'a', 'b'}, {EFlat1::A}, {{}, {{true}, {2}}}, {{EFlat1::B}, {}} });
			}, 
			[](FBatchLoader& Batch)
			{
				CHECK(Batch.Load<FComplexArrays>() == FComplexArrays{});
				CHECK(Batch.Load<FComplexArrays>() == FComplexArrays{{'a', 'b'}, {EFlat1::A}, {{}, {{true}, {2}}}, {{EFlat1::B}, {}} });
			});
	}

	SECTION("TUniquePtr")
	{
		TScopedDefaultStructBinding<FInt> Int;
		TScopedDefaultStructBinding<FUniquePtrs> UniquePtrs;
		Run([](FBatchSaver& Batch)
			{
				Batch.Save(FUniquePtrs{});
				Batch.Save(FUniquePtrs{MakeOne(true), MakeOne(FInt{3}), MakeOne(MakeOne(2)), MakeTwo(1.0, 2.0)});
			}, 
			[](FBatchLoader& Batch)
			{
				CHECK(Batch.Load<FUniquePtrs>() == FUniquePtrs{});
				CHECK(Batch.Load<FUniquePtrs>() == FUniquePtrs{MakeOne(true), MakeOne(FInt{3}), MakeOne(MakeOne(2)), MakeTwo(1.0, 2.0)});
			});
	}

	SECTION("TOptional")
	{
		TScopedDefaultStructBinding<FNDC> NDC;
		TScopedDefaultStructBinding<FNDCIntrusive> NDCI;
		TScopedDefaultStructBinding<FOpts> Opts;
		Run([](FBatchSaver& Batch)
			{
				Batch.Save(FOpts{});
				Batch.Save(FOpts{{true}, {FNDC{2}}, {FNDCIntrusive{3}}});
				Batch.Save(FOpts{{true}, {FNDC{2}}, {FNDCIntrusive{3}}});
			}, 
			[](FBatchLoader& Batch)
			{
				CHECK(Batch.Load<FOpts>() == FOpts{});
				CHECK(Batch.Load<FOpts>() == FOpts{{true}, {FNDC{2}}, {FNDCIntrusive{3}}});
				CHECK(Batch.Load(FOpts{{false}, {FNDC{0}}, {FNDCIntrusive{1}}}) == FOpts{{true}, {FNDC{2}}, {FNDCIntrusive{3}}});
			});
	}

	SECTION("FName")
	{
		TScopedDefaultStructBinding<FNames> Names;
		Run([](FBatchSaver& Batch)
			{
				Batch.Save(FNames{ FName("A"), {FName("Y"), FName("A")} });
				Batch.Save(FNames{ FName("B"), {FName("B_0"), FName("B_1")} });
			}, 
			[](FBatchLoader& Batch)
			{
				CHECK(Batch.Load<FNames>() == FNames{ FName("A"), {FName("Y"), FName("A")}});
				CHECK(Batch.Load<FNames>() == FNames{ FName("B"), {FName("B_0"), FName("B_1")}});
			});
	}

	SECTION("FSensitiveName")
	{
		TScopedDefaultStructBinding<FSensitive> Sensitive;
		TScopedDefaultStructBinding<FSensitiveReversed> Sensitive2;
		Run([](FBatchSaver& Batch)
			{
				Batch.Save(FSensitive{1, 2, 3, 4, 5, 6});
				Batch.Save(FSensitiveReversed{-1, -2, -3, -4, -5, -6});
			}, 
			[](FBatchLoader& Batch)
			{
				CHECK(Batch.Load<FSensitive>() == FSensitive{1, 2, 3, 4, 5, 6});
				CHECK(Batch.Load<FSensitiveReversed>() == FSensitiveReversed{-1, -2, -3, -4, -5, -6});
			});
	}
	
	SECTION("FString")
	{
		TScopedDefaultStructBinding<FStr> Str;
		Run([](FBatchSaver& Batch)
			{
				Batch.Save(FStr{});
				Batch.Save(FStr{"ABC"});
				if constexpr (sizeof(TCHAR) > 1)
				{
					Batch.Save(FStr{TEXT("\x7FF")});
					Batch.Save(FStr{TEXT("\x3300")});
					Batch.Save(FStr{TEXT("\xFE30")});
					Batch.Save(FStr{TEXT("\xD83D\xDC69")});
				}
			}, 
			[](FBatchLoader& Batch)
			{
				CHECK(Batch.Load<FStr>().S.IsEmpty());
				CHECK(Batch.Load<FStr>().S == "ABC");
				if constexpr (sizeof(TCHAR) > 1)
				{
					CHECK(Batch.Load<FStr>().S == TEXT("\x7FF"));
					CHECK(Batch.Load<FStr>().S == TEXT("\x3300"));
					CHECK(Batch.Load<FStr>().S == TEXT("\xFE30"));
					CHECK(Batch.Load<FStr>().S == TEXT("\xD83D\xDC69"));
				}
			});	
	}

	SECTION("TypeErasure")
	{
		TScopedDefaultStructBinding<FSame1> Same1;
		TScopedDefaultStructBinding<FSame2> Same2;	
		TScopedDefaultStructBinding<FIntAlias> IntAlias;
		TScopedDefaultStructBinding<TPair<FString, TArray<char8_t>>> X;
		TScopedDefaultStructBinding<TPair<TArray<char>, TArray<char, TInlineAllocator<8>>>> Y;
		TScopedDefaultStructBinding<FTypeErasure> TypeErasure;

		Run([](FBatchSaver& Batch)
			{
				Batch.Save(FTypeErasure{});
				Batch.Save(FTypeErasure{{10}, {0,20}, {30}, {"a", TArray<char8_t>({u8'b'})}, {MakeArray("c"), MakeInlArray<8>("d")}});
			}, 
			[](FBatchLoader& Batch)
			{
				CHECK(Batch.Load<FTypeErasure>() == FTypeErasure{});
				CHECK(Batch.Load<FTypeErasure>() == FTypeErasure{FTypeErasure{{10}, {0,20}, {30}, {"a", TArray<char8_t>({u8'b'})}, {MakeArray("c"), MakeInlArray<8>("d")}}});
			});
	}

	SECTION("TSet")
	{
		TScopedDefaultStructBinding<FInt> Int;
		TScopedDefaultStructBinding<FSets> Sets;
		Run([](FBatchSaver& Batch)
			{
				Batch.Save(FSets{{'H','i'}, {{uint8(10)}, {}}, {{123}}});
				
				// Test order preservation
				Batch.Save(FSets{{'a','b'}});
				Batch.Save(FSets{{'b','a'}});

				// Test non-compact set
				FSets Sparse = FSets{{'w','z','a','p','?','!'}};
				Sparse.Leaves.Remove('w');
				Sparse.Leaves.Remove('p');
				Sparse.Leaves.Remove('!');
				Batch.Save(Sparse);
			}, 
			[](FBatchLoader& Batch)
			{
				CHECK(Same<ECompare::Order>(Batch.Load<FSets>(), FSets{{'H','i'}, {{uint8(10)}, {}}, {{123}}}));
				CHECK(!Same<ECompare::Order>(FSets{{'a','b'}}, FSets{{'b','a'}}));
				CHECK(Same<ECompare::Order>(Batch.Load<FSets>(), FSets{{'a','b'}}));
				CHECK(Same<ECompare::Order>(Batch.Load<FSets>(), FSets{{'b','a'}}));
				CHECK(Same<ECompare::Content>(Batch.Load<FSets>(), FSets{{'z','a','?'}}));
			});
	}

	SECTION("TMap")
	{
		TScopedDefaultStructBinding<FInt> Int;
		TScopedDefaultStructBinding<FNDC> NDC;
		TScopedDefaultStructBinding<FMaps> Maps;
		// Todo: Recursive auto-bind mechanism?
		TScopedDefaultStructBinding<TPair<bool, bool>> BoolBoolPair;
		TScopedDefaultStructBinding<TPair<int, TArray<char>>> IntStringPair;
		TScopedDefaultStructBinding<TPair<FInt, FNDC>> IntNDCPair;
		TScopedDefaultStructBinding<TPair<FString, FString>> StrStrPair;
		
		Run([](FBatchSaver& Batch)
			{
				TPair<FString, FString> abc = {"a", "bc"};
				Batch.Save(FMaps{});
				Batch.Save(FMaps{{{true, true}, {false, false}}, {{5, {'h', 'i'}}}, {{{7}, FNDC{8}}}, {{"a", "bc"}}});
			}, 
			[](FBatchLoader& Batch)
			{
				CHECK(Same<ECompare::Order>(Batch.Load<FMaps>(), FMaps{}));
				CHECK(Same<ECompare::Order>(Batch.Load<FMaps>(), FMaps{{{true, true}, {false, false}}, {{5, {'h', 'i'}}}, {{{7}, FNDC{8}}}, {{"a", "bc"}}}));
			});
	}

	SECTION("Delta")
	{
		TScopedDefaultStructBinding<FInt> Int;
		TScopedDefaultStructBinding<FDelta> Delta;
		TScopedDefaultStructBinding<FDenseDelta> DenseDelta;
		Run([](FBatchSaver& Batch)
			{
				FDelta Zero = {false, 0, {}, {}, {}};
				CHECK(!Batch.SaveDelta(FInt{123},FInt{123}));
				CHECK(!Batch.SaveDelta(FDelta{},FDelta{}));
				CHECK(!Batch.SaveDelta(Zero, Zero));
				CHECK(!Batch.SaveDelta(FDenseDelta{}, FDenseDelta{}));
				CHECK(!Batch.SaveDelta(FDenseDelta{1, 2}, FDenseDelta{1, 2}));
					
				Batch.SaveDelta({}, Zero);
				Batch.SaveDelta(Zero, {});
				Batch.SaveDelta(FDelta{.B = 123}, {});
				Batch.SaveDelta(FDelta{.C = {321}}, {});
				Batch.SaveDelta(FDelta{.D = {0}}, {});
				Batch.SaveDelta(FDelta{.E = "!!"}, {});
				Batch.SaveDelta(FDenseDelta{.X = 1}, {});
				Batch.SaveDelta(FDenseDelta{.Y = 2}, {});
				Batch.SaveDelta(FDenseDelta{1, 2}, {});
			}, 
			[](FBatchLoader& Batch)
			{
				const FDelta Zero = {false, 0, {}, {}, {}};
				CHECK(Batch.Load(Zero) == FDelta{});
				CHECK(Batch.Load<FDelta>() == Zero);
				CHECK(Batch.Load<FDelta>() == FDelta{.B = 123});
				CHECK(Batch.Load<FDelta>() == FDelta{.C = {321}});
				CHECK(Batch.Load<FDelta>() == FDelta{.D = {0}});
				CHECK(Batch.Load<FDelta>() == FDelta{.E = "!!"});
				CHECK(Batch.Load<FDenseDelta>() == FDenseDelta{.X = 1});
				CHECK(Batch.Load<FDenseDelta>() == FDenseDelta{.Y = 2});
				CHECK(Batch.Load<FDenseDelta>() == FDenseDelta{1, 2});
			});
	}

	SECTION("TSetDelta")
	{
		TScopedDefaultStructBinding<FInt> Int;
		TScopedStructBinding<FSets, FDeltaRuntime> Sets;
		Run([](FBatchSaver& Batch)
			{
				Batch.Save(FSets{});
				Batch.Save(FSets{{'l'}, {{1}}, {{2}}, {"s"}});
				const FSets Default = {{'a'}, {{1}}, {{1}}, {"a"}};
				CHECK(!Batch.SaveDelta(FSets{}, FSets{}));
				CHECK(Batch.SaveDelta(Default, FSets{}));
				CHECK(Batch.SaveDelta(FSets{}, Default)); // Wipe
				CHECK(!Batch.SaveDelta(Default, Default));
				CHECK(Batch.SaveDelta(FSets{{'a'}, {{0,1,2}}, {{2}}, {}}, Default)); // Mixed changes
			}, 
			[](FBatchLoader& Batch)
			{
				CHECK(Same<ECompare::Content>(Batch.Load<FSets>(), FSets{}));
				CHECK(Same<ECompare::Content>(Batch.Load<FSets>(), FSets{{'l'}, {{1}}, {{2}}, {"s"}}));
				
				const FSets Default = {{'a'}, {{1}}, {{1}}, {"a"}};
				CHECK(Same<ECompare::Content>(Batch.Load<FSets>(), Default));
				CHECK(Same<ECompare::Content>(Batch.Load(Default), FSets{})); // Wipe
				CHECK(Same<ECompare::Content>(Batch.Load(Default), FSets{{'a'}, {{0,1,2}}, {{2}}, {}})); // Mixed changes
			},
			GDeltaCustoms);
	}

	SECTION("TMapDelta")
	{
		TScopedDefaultStructBinding<FInt> Int;
		TScopedDefaultStructBinding<FNDC> NDC;
		// Todo: Recursive auto-bind mechanism?
		TScopedDefaultStructBinding<TPair<bool, bool>> BoolBoolPair;
		TScopedDefaultStructBinding<TPair<int, TArray<char>>> IntStringPair;
		TScopedDefaultStructBinding<TPair<FInt, FNDC>> IntNDCPair;
		TScopedDefaultStructBinding<TPair<FString, FString>> StrStrPair;
		TScopedStructBinding<FMaps, FDeltaRuntime> Maps;

		Run([](FBatchSaver& Batch)
			{
				const FMaps Default = {{{true,true}}, {{1, {'a'}}},	{{{2}, FNDC{3}}},	{{"hi", "lo"}}};
				const FMaps Changes = {{},	{{0,{'a'}}, {2,{'a'}}},	{{{2}, FNDC{4}}},	{{"hi", "hi"}}};
				CHECK(!Batch.SaveDelta(FMaps{}, FMaps{}));
				CHECK(Batch.SaveDelta(Default, FMaps{}));
				CHECK(Batch.SaveDelta(FMaps{}, Default)); // Wipe defaults
				CHECK(!Batch.SaveDelta(Default, Default));
				CHECK(Batch.SaveDelta(Changes, Default));
			}, 
			[](FBatchLoader& Batch)
			{
				const FMaps Default = {{{true,true}}, {{1, {'a'}}},	{{{2}, FNDC{3}}},	{{"hi", "lo"}}};
				const FMaps Changes = {{},	{{0,{'a'}}, {2,{'a'}}},	{{{2}, FNDC{4}}},	{{"hi", "hi"}}};
				CHECK(Same<ECompare::Content>(Batch.Load<FMaps>(), Default));
				CHECK(Same<ECompare::Content>(Batch.Load<FMaps>(), FMaps{})); // Wipe defaults
				CHECK(Same<ECompare::Content>(Batch.Load(Default), Changes));
			},
			GDeltaCustoms);
	}

	SECTION("Transform")
	{
		TScopedDefaultStructBinding<FVector> Vector;
		TScopedDefaultStructBinding<FQuat> Quat;
		TScopedDefaultStructBinding<FTransform> Transform;

		Run([](FBatchSaver& Batch)
			{
				CHECK(!Batch.SaveDelta(FTransform(),FTransform()));
				CHECK(!Batch.SaveDelta(FTransform(FVector::UnitY()), FTransform(FVector::UnitY())));

				Batch.Save(FTransform());

				// This should only save translation
				Batch.SaveDelta(FTransform(FVector::UnitY()), FTransform());
			}, 
			[](FBatchLoader& Batch)
			{
				CHECK(Batch.Load<FTransform>().Equals(FTransform(), 0.0));

				FTransform TranslateY(				FQuat(1, 2, 3, 4), FVector(5, 5, 5), FVector(6, 7, 8));
				Batch.LoadInto(TranslateY);
				CHECK(TranslateY.Equals(FTransform(	FQuat(1, 2, 3, 4), FVector::UnitY(), FVector(6, 7, 8)), 0.0));
			});
	}

	SECTION("TVariant")
	{
		TScopedDefaultStructBinding<FInt> Int;
		TScopedDefaultStructBinding<FNDC> NDC;
		TScopedDefaultStructBinding<FEmptyVariantState> EmptyVariantState;
		TScopedDefaultStructBinding<TVariant<int32, FNDC>> InnerVariant;
		TScopedDefaultStructBinding<FVariants> Variants;
		Run([](FBatchSaver& Batch)
			{
				FVariants Var1 {
					TVariant<bool, float>{TInPlaceType<bool>(), true},
					TVariant<float, FInt>{TInPlaceType<float>(), 2.2f},
					TVariant<FEmptyVariantState, FNDC>{},
					TVariant<float, TArray<FInt>>{TInPlaceType<float>(), 4.4f},
					TVariant<int32, TVariant<int32, FNDC>>{TInPlaceType<int32>(), 5}
				};
				FVariants Var2 {
					TVariant<bool, float>{TInPlaceType<float>(), 1.1f},
					TVariant<float, FInt>{TInPlaceType<FInt>(), 2},
					TVariant<FEmptyVariantState, FNDC>{TInPlaceType<FNDC>(), 3},
					TVariant<float, TArray<FInt>>{TInPlaceType<TArray<FInt>>(), MakeArrayView({FInt{4}, FInt{5}})},
					TVariant<int32, TVariant<int32, FNDC>>{TInPlaceType<TVariant<int32, FNDC>>(), TVariant<int32, FNDC>{TInPlaceType<FNDC>(), 6}}
				};
				Batch.Save(Var1);
				Batch.Save(Var2);
			}, 
			[](FBatchLoader& Batch)
			{
				FVariants Var1 {
					TVariant<bool, float>{TInPlaceType<bool>(), true},
					TVariant<float, FInt>{TInPlaceType<float>(), 2.2f},
					TVariant<FEmptyVariantState, FNDC>{},
					TVariant<float, TArray<FInt>>{TInPlaceType<float>(), 4.4f},
					TVariant<int32, TVariant<int32, FNDC>>{TInPlaceType<int32>(), 5}
				};
				FVariants Var2 {
					TVariant<bool, float>{TInPlaceType<float>(), 1.1f},
					TVariant<float, FInt>{TInPlaceType<FInt>(), 2},
					TVariant<FEmptyVariantState, FNDC>{TInPlaceType<FNDC>(), 3},
					TVariant<float, TArray<FInt>>{TInPlaceType<TArray<FInt>>(), MakeArrayView({FInt{4}, FInt{5}})},
					TVariant<int32, TVariant<int32, FNDC>>{TInPlaceType<TVariant<int32, FNDC>>(), TVariant<int32, FNDC>{TInPlaceType<FNDC>(), 6}}
				};

				FVariants LoadedVar1 = Batch.Load<FVariants>();
				FVariants LoadedVar2 = Batch.Load<FVariants>();

				CHECK(LoadedVar1.BoolFloat.Get<bool>()				== Var1.BoolFloat.Get<bool>());
				CHECK(LoadedVar1.FloatInt.Get<float>()				== Var1.FloatInt.Get<float>());
				CHECK(LoadedVar1.NDC.IsType<FEmptyVariantState>());
				CHECK(LoadedVar1.FloatIntArray.Get<float>()			== Var1.FloatIntArray.Get<float>());
				CHECK(LoadedVar1.IntVariantNDC.Get<int32>()			== Var1.IntVariantNDC.Get<int32>());

				CHECK(LoadedVar2.BoolFloat.Get<float>()				== Var2.BoolFloat.Get<float>());
				CHECK(LoadedVar2.FloatInt.Get<FInt>()				== Var2.FloatInt.Get<FInt>());
				CHECK(LoadedVar2.NDC.Get<FNDC>()					== Var2.NDC.Get<FNDC>());
				CHECK(LoadedVar2.FloatIntArray.Get<TArray<FInt>>()	== Var2.FloatIntArray.Get<TArray<FInt>>());
				CHECK(LoadedVar2.IntVariantNDC.Get<TVariant<int32,FNDC>>().Get<FNDC>() ==
						    Var2.IntVariantNDC.Get<TVariant<int32,FNDC>>().Get<FNDC>());
			});
	}

	SECTION("Reference")
	{}
}

} // namespace PlainProps::UE::Test

#endif // WITH_TESTS
