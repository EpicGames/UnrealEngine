// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS
#include "PlainPropsCtti.h"
#include "PlainPropsIndex.h"
#include "PlainPropsInternalBind.h"
#include "PlainPropsInternalBuild.h"
#include "PlainPropsInternalDiff.h"
#include "PlainPropsInternalFormat.h"
#include "PlainPropsInternalRead.h"
#include "PlainPropsInternalTest.h"
#include "PlainPropsInternalUpgrade.h"
#include "PlainPropsDiff.h"
#include "PlainPropsBuild.h"
#include "PlainPropsBuildSchema.h"
#include "PlainPropsLoad.h"
#include "PlainPropsRead.h"
#include "PlainPropsSave.h"
#include "PlainPropsUpgrade.h"
#include "PlainPropsVisualize.h"
#include "PlainPropsUeCoreBindings.h"
#include "PlainPropsWrite.h"
#include "Containers/AnsiString.h"
#include "Containers/Map.h"
#include "Logging/StructuredLog.h"
#include "Templates/UnrealTemplate.h"
#include "Tests/TestHarnessAdapter.h"

DEFINE_LOG_CATEGORY_STATIC(LogPlainPropsUpgradeTests, Log, All);

namespace PlainProps::Upgrade::Test
{

static TIdIndexer<FAnsiString>	GIds;
static FDebugIds				GDebug(GIds);
static FEnumDeclarations		GEnums(GDebug);
static FSchemaBindings			GSchemas(GDebug);
static FCustomBindingsBottom	GCustoms(GDebug);
static FCustomBindingsOverlay	GDeltaCustoms(GCustoms);
static FBindDeclarations		GDeltaDeclarations(GEnums, GDeltaCustoms, GSchemas);

struct FFixture
{
	FFixture() : Vis(GIds, "AnsiStr") {}
	DbgVis::FIdScope Vis;
};

struct FRuntimeIds
{
	static FNameId				IndexName(FAnsiStringView Name)		{ return GIds.MakeName(Name); }
	static FMemberId			IndexMember(FAnsiStringView Name)	{ return GIds.NameMember(Name); }
	static FConcreteTypenameId	IndexTypename(FAnsiStringView Name)	{ return GIds.NameType(Name); }
	static FFlatScopeId			IndexScope(FAnsiStringView Name)	{ return GIds.NameScope(Name); }
	static FEnumId				IndexEnum(FType Type)				{ return GIds.IndexEnum(Type); }
	static FStructId			IndexStruct(FType Type)				{ return GIds.IndexStruct(Type); }
	static FIdIndexerBase&		GetIndexer()						{ return GIds; }
};

struct FDefaultRuntime
{
	using Ids = FRuntimeIds;
	template<class T> using CustomBindings = TCustomBind<T>;

	static FEnumDeclarations&		GetEnums()			{ return GEnums; }
	static FSchemaBindings&			GetSchemas()		{ return GSchemas; }
	static FCustomBindings&			GetCustoms()		{ return GCustoms; }
};

struct FDeltaRuntime : FDefaultRuntime
{
	template<class T> using CustomBindings = TCustomDeltaBind<T>;

	static FCustomBindings&			GetCustoms()		{ return GDeltaCustoms; }
};

////////////////////////////////////////////////////////////////////////////////////////////////

struct FUnboundDeclarations : IDeclarations
{
public:
	FUnboundDeclarations() : Enums(GDebug) {}

	const FStructDeclaration&				Declare(const char* Scope, const char* Name, std::initializer_list<const char*> MemberNames, TConstArrayView<FMemberSpec> MemberTypes, EMemberPresence Occupancy, FOptionalDeclId Super = NoId)
	{
		FDeclId Id = GIds.IndexDeclId(GIds.MakeType(Scope, Name));
		return *Structs.Emplace(Id, PlainProps::Declare({Id, Super, /* v */ 0, Occupancy, NameMembers(MemberNames), MemberTypes}));	
	}

private:
	virtual const FEnumDeclaration*			Find(FEnumId Id) const override		{ return Enums.Find(Id); }
	virtual const FStructDeclaration*		Find(FStructId Id) const override	{ return Structs.FindChecked(FDeclId(Id)); }
	virtual FDeclId							Lower(FBindId Id) const override	{ unimplemented(); return LowerCast(Id);	}

	FEnumDeclarations						Enums;
	TMap<FStructId, FStructDeclarationPtr>	Structs;

	static TArray<FMemberId> NameMembers(std::initializer_list<const char*> Members)
	{
		TArray<FMemberId> Out;
		Out.Reserve(Members.size());
		for (const char* Member : Members)
		{
			Out.Add(GIds.NameMember(Member));
		}
		return Out;
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////

FAnsiStringView	ToStr(EMismatch Mismatch)
{
	switch (Mismatch)
	{
	case EMismatch::Inners:		return "inners of";
	case EMismatch::Type:		return "type of";
	case EMismatch::Duplicate:	return "duplicated";
	case EMismatch::Surplus:	return "surplus";
	case EMismatch::Missing:	return "missing";
	}
	return "UNKNOWN MISMATCH";
}

static void CheckMismatches(TConstArrayView<FUpgradedInstance> Instances, FDebugPrinter Debug)
{
	for (FUpgradedInstance Instance : Instances)
	{
		if (Instance.Fixed)
		{
			continue;
		}

		TMap<FParameterId, FLooseType> Missing;
		TUtf8StringBuilder<256> Msg;
		for (FLooseMember Member : Instance.Loose)
		{
			EMismatch Mismatch = Member.GetValue().Meta.Mismatch;
			FParameterId Name = Member.GetName();
			if (Mismatch == EMismatch::No)
			{
				if (!Member)
				{
					Missing.Emplace(Name, Member.GetType()); 
				}
				continue;
			}

			if (Msg.Len())
			{
				Msg << ", ";
			}
			
			Msg << ToStr(Mismatch) << " '" << Debug.Print(Instance.Type, Name) << "' " ;
			if (const FLooseType* Expected = Missing.Find(Name))
			{
				Msg << "from " << Debug.Print(*Expected) << " to ";
			}
			Msg << Debug.Print(Member.GetType());
		}

		UE_LOGFMT(LogPlainPropsUpgradeTests, Fatal, "Missing upgrade(s) in '{Struct}' that transforms {Msg}", Debug.Print(Instance.Type), Msg);
	}
}

// FBatchWriter/FBatchSaver helper
struct FSavedStruct
{
	FDeclId						Type;
	const FBuiltStruct*			Value;
};

class FBatchWriterBase
{
public:
	FBatchWriterBase(FScratchAllocator& InScratch) : Scratch(InScratch) {}
		
	void						SetRoots(TConstArrayView<FUpgradedInstance> Instances)
	{ 
		Roots.Empty(Instances.Num());
		for (FUpgradedInstance Instance : Instances)
		{
			check(Instance.Fixed);
			Roots.Emplace(Instance.Type, Instance.Fixed);
		}
	}

protected:
	TArray<FSavedStruct>		Roots;
	FScratchAllocator&			Scratch;

	TArray64<uint8>				Write(const IDeclarations& Types) const;
};

////////////////////////////////////////////////////////////////////////////////////////////////

// For untyped unit tests without load/save/bind layer
class FBatchWriter : public FBatchWriterBase
{
public:
	using FBatchWriterBase::FBatchWriterBase;

	FBuiltStruct*				Build(FMemberBuilder& Members, const FStructDeclaration& Decl) const	{ return Members.BuildAndReset(Scratch, Decl, GDebug); }
	void						Save(FDeclId Type, const FBuiltStruct* Value)							{ Roots.Emplace(Type, Value); }
	TArray64<uint8>				Write() const															{ return FBatchWriterBase::Write(OldTypes); }
	
	FUnboundDeclarations		OldTypes;
};

////////////////////////////////////////////////////////////////////////////////////////////////

// Temporarily bind obsolete test type to save it
struct ITempBinding { virtual ~ITempBinding() {} };
template<class T>
struct TTempBinding : ITempBinding, TScopedStructBinding<T, FDefaultRuntime> {};
using FTempBindingPtr = TUniquePtr<ITempBinding>;

// For typed unit tests using load/save/bind layer
class FBatchSaver : public FBatchWriterBase
{
public:
	using FBatchWriterBase::FBatchWriterBase;

	template<class InT>
	void						Save(InT&& Object, InT&& Default = {}) 
	{
		using T = std::remove_reference_t<InT>;
		FDualStructId Id = IndexStructDualId<FRuntimeIds, TTypename<T>>();
		BindObsolete<T>(Id);
		Roots.Emplace(FDeclId(Id), SaveStructDelta(&Object, FBaseline(&Default), Id, {{GSchemas, GCustoms}, {Scratch}}));
	}

	TArray64<uint8>				Write() const { return FBatchWriterBase::Write(GDeltaDeclarations); }

private:
	TArray<FTempBindingPtr>		Obsoletes;

	template<typename T>
	void BindObsolete(FDualStructId Id)
	{
		if (!GDeltaDeclarations.Find(Id))
		{
			Obsoletes.Emplace(new TTempBinding<T>);
		}
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////

inline constexpr uint32 TestMagics[] = { 0xFEEDF00D, 0xABCD1234, 0xDADADAAA, 0x99887766, 0xF0F1F2F3 };

TArray64<uint8> FBatchWriterBase::Write(const IDeclarations& Types) const
{
	// Build partial schemas
	FSchemasBuilder SchemaBuilders(GIds, Types, Scratch, ESchemaFormat::InMemoryNames);
	for (FSavedStruct Root : Roots)
	{
		SchemaBuilders.NoteStructAndMembers(Root.Type, *Root.Value);
	}
	FBuiltSchemas Schemas = SchemaBuilders.Build(); 

	// Filter out declared but unused names and ids
	FWriter Writer(GIds, Types, Schemas, ESchemaFormat::InMemoryNames);

	// Write schemas
	TArray64<uint8> Out;
	TArray64<uint8> Tmp;
	WriteInt(Out, TestMagics[1]);
	Writer.WriteSchemas(/* Out */ Tmp);
	WriteAlignmentPadding<uint32>(Out);
	WriteInt(Out, IntCastChecked<uint32>(Tmp.Num()));
	WriteArray(Out, Tmp);
	Tmp.Reset();

	// Write roots
	WriteInt(Out, TestMagics[2]);
	for (FSavedStruct Root : Roots)
	{
		WriteInt(/* out */ Tmp, TestMagics[3]);
		WriteInt(/* out */ Tmp, Writer.GetWriteId(Root.Type).Get().Idx);
		Writer.WriteMembers(/* out */ Tmp, Root.Type, *Root.Value);
		WriteSkippableSlice(Out, Tmp);
		Tmp.Reset();
	}

	// Write object terminator
	WriteSkippableSlice(Out, TConstArrayView64<uint8>());
	WriteInt(Out, TestMagics[4]);
		
	return Out;
}

////////////////////////////////////////////////////////////////////////////////////////////////

class FBatchLoader
{
public:
	FBatchLoader(FMemoryView Data)
	{
		// Read schemas
		FByteReader It(Data);
		CHECK(It.Grab<uint32>() == TestMagics[1]);
		It.SkipAlignmentPadding<uint32>();
		uint32 SchemasSize = It.Grab<uint32>();
		FMemoryView SchemasView = It.GrabSlice(SchemasSize);
		Schemas = ValidateSchemas(SchemasView);
		FSchemaBatchId Batch = MountReadSchemas(Schemas);
		CHECK(It.Grab<uint32>() == TestMagics[2]);
		
		// Read roots
		while (uint64 NumBytes = It.GrabVarIntU())
		{	
			FByteReader RootIt(It.GrabSlice(NumBytes));
			CHECK(RootIt.Grab<uint32>() == TestMagics[3]);
			FStructSchemaId Schema = { RootIt.Grab<uint32>() };
			Roots.Add({ { Schema, Batch }, RootIt });
		}
		
		CHECK(It.Grab<uint32>() == TestMagics[4]);
		check(!Roots.IsEmpty());

		// Finally create load plans
		RuntimeIds = { IndexAllRuntimeIds(*Schemas, GIds) };
		TConstArrayView<FStructId> StructIds = FInnerIds{RuntimeIds}.GetStructIds(Schemas->NumStructSchemas);
		Plans = CreateLoadPlans(Batch, GCustoms, GSchemas, StructIds, ESchemaFormat::InMemoryNames);
	}

	~FBatchLoader()
	{
		UnmountReadSchemas(Roots[0].Schema.Batch);
	}
	
	const FSchemaBatch&					GetSchemas() const			{ return *Schemas; }
	TConstArrayView<FInnerId>			GetRuntimeIds() const		{ return RuntimeIds; }
	TConstArrayView<FStructView>		GetRoots() const			{ return Roots;	}

	template<class T>
	T Load(int32 Idx) const
	{
		T Out;
		LoadStruct(&Out, Roots[Idx].Values, Roots[Idx].Schema.Id, *Plans);
		return MoveTemp(Out);
	}

private:
	const FSchemaBatch*			Schemas = nullptr;
	TArray<FStructView>			Roots;
	TArray<FInnerId>			RuntimeIds;
	FLoadBatchPtr				Plans;
};

////////////////////////////////////////////////////////////////////////////////////////////////


static void Run(FHistory& History, void (*SaveOld)(FBatchSaver&), void (*TestNew)(FBatchLoader&))
{
	// Generate old data
	TArray64<uint8> OldData;
	{
		FScratchAllocator Scratch;
		FBatchSaver OldBatch(Scratch);
		SaveOld(OldBatch);
		OldData = OldBatch.Write();
	}

	TArray64<uint8> NewData;
	{
		// Mount old schemas
		FBatchLoader OldBatch(MakeMemoryView(OldData));

		// Upgrade old instances
		FBatchUpgrader Upgrader(OldBatch.GetSchemas(), OldBatch.GetRuntimeIds(), GIds);
		FBindDeclarations Declarations(GEnums, GCustoms, GSchemas);
		CHECK(!Upgrader.MatchSchemas(Declarations));
		FScratchAllocator Scratch;
		TArray<FUpgradedInstance> NewRoots = Upgrader.Upgrade(OldBatch.GetRoots(), History, Scratch);
		CheckMismatches(NewRoots, FDebugPrinter(History.Ids, History.LooseIds));

		// Write new schemas and instances
		FBatchSaver NewBatch(Scratch);
		NewBatch.SetRoots(NewRoots);
		NewData = NewBatch.Write();
	}
	
	// Test new data
	FBatchLoader NewBatch(MakeMemoryView(NewData));
	TestNew(NewBatch);
}

// RunUntyped helper
struct FTestUpgrades
{
	FUnboundDeclarations&	NewTypes;
	FHistory&				History;
};

// RunUntyped helper
struct FUpgradedBatch
{
	TConstArrayView<FUpgradedInstance>		Roots;
	FLooseIndexers&							LooseIds;

	FAnsiString ResolveName(FDeclId Id, FLooseMember Member) const { return GIds.ResolveName(LooseIds.Get(Id).Resolve(Member.GetName()).Get().Id); }
};

static void RunUntyped(void (*WriteUntyped)(FBatchWriter&), void (*ChronicleUntyped)(FTestUpgrades), void (*TestUntyped)(FUpgradedBatch))
{
	// Generate old data
	TArray64<uint8> OldData;
	{
		FScratchAllocator Scratch;
		FBatchWriter OldBatch(Scratch);
		WriteUntyped(OldBatch);
		OldData = OldBatch.Write();
	}

	// Register latest declarations and upgrade ops
	FUnboundDeclarations NewTypes;
	FHistoryPtr History = MakeHistory(NewTypes, GIds);
	ChronicleUntyped({NewTypes, *History});

	// Run upgrades
	FScratchAllocator Scratch;
	TArray<FUpgradedInstance> Roots;
	{
		// Mount old schemas
		FBatchLoader OldBatch(MakeMemoryView(OldData));

		// Upgrade old instances
		FBatchUpgrader Upgrader(OldBatch.GetSchemas(), OldBatch.GetRuntimeIds(), GIds);
		CHECK(!Upgrader.MatchSchemas(NewTypes));
		
		Roots = Upgrader.Upgrade(OldBatch.GetRoots(), *History, Scratch);
	}
	
	// Finally test upgraded output
	TestUntyped({Roots, History->LooseIds});
}

////////////////////////////////////////////////////////////////////////////////////////////////

// Tests that everything was read
struct FTestMemberReader : public FMemberReader
{
	using FMemberReader::FMemberReader;
	
	~FTestMemberReader()
	{
		CHECK(MemberIdx == NumMembers); // Must read all members
		CHECK(RangeTypeIdx == NumRangeTypes); // Must read all ranges
#if DO_CHECK
		CHECK(InnerSchemaIdx == NumInnerSchemas); // Must read all schema ids
#endif
	}
};

template<typename OutType, typename InType>
TArray<OutType> MakeArray(const InType& Items)
{
	TArray<OutType> Out;
	Out.Reserve(IntCastChecked<int32>(Items.Num()));
	for (const auto& Item : Items)
	{
		Out.Emplace(Item);
	}
	return Out;
}

////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////

// No ordering impact - only helps compile out old upgraders
enum ERelease { R0, R1, R2, R3 };

enum class ERename0 : uint8 {Same, OldA, OldB1, OldB2};
enum class ERename : uint8 {Same, NewA, NewB};
enum class EFlagify : uint8 {A, B};
enum class EFlatten : uint8 {None, A, B, AB};
PP_REFLECT_ENUM(PlainProps::Upgrade::Test, ERename0, Same, OldA, OldB1, OldB2);
PP_REFLECT_ENUM(PlainProps::Upgrade::Test, ERename, Same, NewA, NewB);
PP_REFLECT_ENUM(PlainProps::Upgrade::Test, EFlagify, A, B);
PP_REFLECT_ENUM(PlainProps::Upgrade::Test, EFlatten, A, B, AB);
//
//template<> PlainProps::Upgrade::Chronicle<ERename>(FHistory& Out)
//{
//	FEnumChronicle Rename(Out, "", "ERename");
//
//	//Rename<R0>.Rename("OldA", "NewA");
//	//Rename<R0>.Rename("OldB1", "NewB");
//	//Rename<R0>.Rename("OldB2", "NewB");
//}
//
//template<> PlainProps::Upgrade::Chronicle<EFlagify>(FHistory& Out)
//{
//	FEnumChronicle Flagify(Out, "", "EFlagify");
//	Flagify.Record<R0>([](FEnumChronicle& Out)
//		{
//			//Out.
//		});
//	//Flagify<R0>.Flagify("None", {});
//	//Flagify<R0>.Flagify("AB", {"A", "B"});
//}
//
//template<> PlainProps::Upgrade::Chronicle<EFlatten>(FHistory& Out)
//{
//	FEnumChronicle Flatten(Out, "", "EFlatten");
//	//Flatten<R0>.Flatten({}, "None");
//	//Flatten<R0>.Flatten({"A", "B"}, "AB");
//}

////////////////////////////////////////////////////////////////////////////////////////////////

enum class EFlag : uint8 {None = 0, A = 8, B = 32, AB = 40};
PP_REFLECT_ENUM(PlainProps::Upgrade::Test, EFlag, A, B);

struct FPoint { float X, Y; };
PP_REFLECT_STRUCT(PlainProps::Upgrade::Test, FPoint, void, X, Y); 
inline bool operator==(FPoint A, FPoint B) { return A.X == B.X && A.Y == B.Y; }

struct FBasicOld
{
	uint32		Widen; // Could be handled automatically by loosen + fasten
	int32		Rename;
	uint16		Reorder; // Handled automatically by loosen + fasten
	double		Drop;
	double		Unchanged;
	bool		FlagA;
	bool		FlagB;
	float		PointX;
	float		PointY;
	FGuid		Guid; // Renamed to GuidRenamed to test struct renaming
};
PP_REFLECT_STRUCT(PlainProps::Upgrade::Test, FBasicOld, void, Widen, Rename, Reorder, Drop, Unchanged, FlagA, FlagB, PointX, PointY, Guid); 

struct FBasic
{
	uint64		Widen;
	uint16		Reorder;
	int32		Renamed;
	double		Unchanged;
	EFlag		Flags;
	FPoint		Point;
	uint64		GuidAB; // Todo: Switch to uint64 Guid[2] once static array support is built out
	uint64		GuidCD;

	bool operator==(const FBasic&) const = default;
};
PP_REFLECT_STRUCT(PlainProps::Upgrade::Test, FBasic, void, Widen, Reorder, Renamed, Unchanged, Flags, Point, GuidAB, GuidCD);


// Sugar-free API
void Record(TChroniclesOf<FBasic> Out)
{
	FChronicler BasicOld(Out, "PlainProps::Upgrade::Test", "FBasicOld");

	BasicOld.In<R1>([](IOps& Out)
	{	
		Out.Add({"Widen"_U32}, {"Widen"_U64}, [](FIns In, FOuts Out, FContext Ctx)
			{
				Out[0].U64() = In[0].U32();
			});
		
		Out.Add({"Rename"_S32}, {"Renamed"_S32}, [](FIns In, FOuts Out, FContext Ctx) 
			{ 
				Out[0].S32() = In[0].S32();
			});

		Out.Add({"Drop"_F64}, {}, [](FIns, FOuts, FContext) {});
		
		// Partial declaration of the members / enumerators that will be accessed
		// Note a chronicle might need multiple declarations for a type if it changed during chronicle
		FEnumFlagNames AB = Out.NameFlags({"PlainProps::Upgrade::Test", "EFlag"}, {"A", "B"});

		// Executed last if only FlagA or FlagB was sparsely saved
		Out.Add({~"FlagA"_B1, ~"FlagB"_B1}, {"Flags"_Enum(AB)}, [](FIns In, FOuts Out, FContext)
			{
				FOutFlags Flags = Out[0].EnumFlags();
				Flags[/* A */ 0] = In[/* FlagA */ 0].OptBool();
				Flags[/* B */ 1] = In[/* FlagB */ 1].OptBool();
			});
	});
	
	FChronicler Basic(Out, "PlainProps::Upgrade::Test", "FBasic");
	BasicOld.Rename<R1>(Basic.GetName());
	Basic.In<R1>([](IOps& Out)
	{
		// Partial declaration of the members / enumerators that will be accessed
		// Note a chronicle might need multiple declarations for a type if it changed during chronicle
		
		FStructMemberNames XY = Out.NameMembers({"PlainProps::Upgrade::Test", "FPoint"}, {"X"_F32, "Y"_F32});
		Out.Add({ ~"PointX"_F32, ~"PointY"_F32 }, { "Point"_Struct(XY) }, [](FIns In, FOuts Out, FContext)
			{
				FOutMembers Point = Out[0].StructMembers();
				Point[/* X */ 0].F32() = In[/* PointX */ 0].OptF32().Get(0.f);
				Point[/* Y */ 1].F32() = In[/* PointY */ 1].OptF32().Get(0.f);
			});
		
		FStructId GuidId = Out.NameStruct({/*"/Script/CoreUObject"*/ "", "Guid"});
		Out.Add({"Guid"_Struct(GuidId)}, {"GuidRenamed"_Struct(GuidId)}, [](FIns In, FOuts Out, FContext Ctx) 
			{ 
				Out[0].Struct() = In[0].Struct();
			});

		FStructMemberNames ABCD = Out.NameMembers(GuidId, {"A"_U32, "B"_U32, "C"_U32, "D"_U32});
		Out.Add({ "GuidRenamed"_Struct(ABCD) }, { "GuidAB"_U64, "GuidCD"_U64 }, [](FIns In, FOuts Out, FContext)
			{
				FInMembers ABCD = In[0].StructMembers();

				uint64 A = ABCD[0].U32();
				uint64 B = ABCD[1].U32();
				uint64 C = ABCD[2].U32();
				uint64 D = ABCD[3].U32();
				
				Out[/* GuidAB */ 0].U64() = (A << 32) | B;
				Out[/* GuidCD */ 1].U64() = (C << 32) | D;
			});
	});
}

// Sugar / template sorcery (up to a few members) for type-safety & usability
//void Record(TChroniclesOf<FBasic> Out)
//{
//	FChronicler BasicOld(Out, "PlainProps::Upgrade::Test", "FBasicOld");	
//	BasicOld.In<R1>([](IOps& Out)
//	{
//		Out.Convert<uint32>("Widen"_U64);
//		Out.Rename("Rename"_S32, "Renamed");
//		Out.Drop("Drop"_F64);
//
//		FEnumFlagNames Flag = Out.NameFlags({"PlainProps::Upgrade::Test", "EFlag"}, {"A", "B"});
//		Out.Enumify({~"FlagA"_B1, ~"FlagB"_B1}, "Flags"_Enum(Flag));
//	});
//	
//	FChronicler Basic(Out, "PlainProps::Upgrade::Test", "FBasic");
//	BasicOld.Rename<R1>(Basic.GetName());
//	Basic.In<R1>([](IOps& Out)
//	{
//		FStructMemberNames XY = Out.NameMembers({"PlainProps::Upgrade::Test", "FPoint"}, {"X"_F32, "Y"_F32});
//		Out.Structify({ ~"PointX"_F32, ~"PointY"_F32 }, "Point"_Struct(XY));
//		
//		FStructId GuidId = Out.NameStruct({/*"/Script/CoreUObject"*/ "", "Guid"});
//		Out.Rename("Guid"_Struct(GuidId), "GuidRenamed");
//
//		FStructMemberNames ABCD = Out.NameMembers(GuidId, {"A"_U32, "B"_U32, "C"_U32, "D"_U32});
//		Out.Destructify("GuidRenamed"_Struct(ABCD), { "GuidAB"_U64, "GuidCD"_U64 }, 
//			[](uint32 A, uint32 B, uint32 C, uint32 D, TOut<2, uint64> Out, FContext Ctx)
//			{
//				Out[/* GuidAB */ 0].U64() = (A << 32) | B;
//				Out[/* GuidCD */ 1].U64() = (C << 32) | D;
//			});
//	});
//}


// Sugar-free but with type-checked inputs
//
// Can type-check inputs and outputs too if we specify number of input params, i.e. AddTypechecked<4>
//
//void Record(TChroniclesOf<FBasic> Out)
//{
//	FChronicler BasicOld(Out, "PlainProps::Upgrade::Test", "FBasicOld");
//
//	BasicOld.In<R1>([](IOps& Out)
//	{	
//		Out.AddTypechecked({"Widen"_U32}, {"Widen"_U64}, [](uint32 Widen, FOuts Out, FContext Ctx)
//			{
//				Out[0].U64() = Widen;
//			});
//		
//		Out.AddTypechecked({"Rename"_S32}, {"Renamed"_S32}, [](int32 Rename, FOuts Out, FContext Ctx) 
//			{ 
//				Out[0].S32() = Rename;
//			});
//
//		Out.AddTypechecked({"Drop"_F64}, {}, [](double Drop, FOuts, FContext) {});
//		
//		FEnumFlagNames AB = Out.NameFlags({"PlainProps::Upgrade::Test", "EFlag"}, {"A", "B"});
//		Out.AddTypechecked({~"FlagA"_B1, ~"FlagB"_B1}, {"Flags"_Enum(AB)}, 
//			[](TOptional<bool> FlagA, TOptional<bool> FlagB, FOuts Out, FContext)
//			{
//				FOutFlags& Flags = Out[0].EnumFlags();
//				Flags[/* A */ 0] = FlagA;
//				Flags[/* B */ 1] = FlagB;
//			});
//	});
//	
//	FChronicler Basic(Out, "PlainProps::Upgrade::Test", "FBasic");
//	BasicOld.Rename<R1>(Basic.GetName());
//	Basic.In<R1>([](IOps& Out)
//	{
//		// Partial declaration of the members / enumerators that will be accessed
//		// Note a chronicle might need multiple declarations for a type if it changed during chronicle
//		
//		FStructMemberNames XY = Out.NameMembers({"PlainProps::Upgrade::Test", "FPoint"}, {"X"_F32, "Y"_F32});
//		Out.AddTypechecked({ ~"PointX"_F32, ~"PointY"_F32 }, { "Point"_Struct(XY) },
//			[](TOptional<float> PointX, TOptional<float> PointY, FOuts Out, FContext)
//			{
//				TOuts<2> Point = Out[0].StructMembers<2>();
//				Point[/* X */ 0].F32() = PointX.Get(0.f);
//				Point[/* Y */ 1].F32() = PointY.Get(0.f);
//			});
//		
//		FStructId GuidId = Out.NameStruct({/*"/Script/CoreUObject"*/ "", "Guid"});
//		Out.AddTypechecked({"Guid"_Struct(GuidId)}, {"GuidRenamed"_Struct(GuidId)}, [](FInStruct Guid, FOuts Out, FContext Ctx) 
//			{ 
//				Out[0].Struct() = Guid;
//			});
//
//		FStructMemberNames ABCD = Out.NameMembers(GuidId, {"A"_U32, "B"_U32, "C"_U32, "D"_U32});
//		Out.AddTypechecked({ "GuidRenamed"_Struct(ABCD) }, { "GuidAB"_U64, "GuidCD"_U64 }, [](TIns<4> ABCD, FOuts Out, FContext)
//			{
//				uint64 A = ABCD[0].U32();
//				uint64 B = ABCD[1].U32();
//				uint64 C = ABCD[2].U32();
//				uint64 D = ABCD[3].U32();
//				
//				Out[/* GuidAB */ 0].U64() = (A << 32) | B;
//				Out[/* GuidCD */ 1].U64() = (C << 32) | D;
//			});
//	});
//}

////////////////////////////////////////////////////////////////////////////////////////////////

enum class EFlat : uint8 {A=3, B=9, C=0};
PP_REFLECT_ENUM(PlainProps::Upgrade::Test, EFlat, A, B, C);

struct FStr
{
	TArray<char8_t> Data;
	bool operator==(const FStr&) const = default;
};
PP_REFLECT_STRUCT(PlainProps::Upgrade::Test, FStr, void, Data);

struct FPoints
{
	TArray<float> Xs;
	TArray<float> Ys;
	bool operator==(const FPoints&) const = default;
};
PP_REFLECT_STRUCT(PlainProps::Upgrade::Test, FPoints, void, Xs, Ys); 

// Test convert FRangesA into FRangesB and vice versa
struct FRangesA
{
	TArray<uint8>			Leaves;	// Int <-> Enum
	TArray<char8_t>			String; // Structify <-> Destructify
	TArray<FPoint>			Points; // SoA <-> AoS
	TArray<TArray<FPoint>>	Series; // Join <-> Split
	TArray<char>			Unterminated; // Add <-> Remove null terminator
	TArray<EFlat>			Sorted; // Reverse opaque enums
	TArray<EFlat>			Enums;	// A->B, B->C, C->A
	TArray<bool>			Bools;	// Negate
};
PP_REFLECT_STRUCT(PlainProps::Upgrade::Test, FRangesA, void, Leaves, String, Points, Series, Unterminated, Sorted, Enums, Bools); 

struct FRangesB
{
	TArray<EFlag>			Leaves;
	FStr					String;
	FPoints					Points;
	TArray<FPoint>			Series;
	TArray<int32>			SeriesStarts; // Series indices
	TArray<char>			Terminated;
	TArray<EFlat>			Reversed;
	TArray<EFlat>			Cycled;
	TArray<bool>			Negated;
	bool operator==(const FRangesB&) const = default;
};
PP_REFLECT_STRUCT(PlainProps::Upgrade::Test, FRangesB, void, Leaves, String, Points, Series, SeriesStarts, Terminated, Reversed, Cycled, Negated);

void Record(TChroniclesOf<FRangesB> Out)
{
	FChronicler RangesA(Out, "PlainProps::Upgrade::Test", "FRangesA"); // ~ registers chronicle once we've collected all member names
	
	RangesA.In<R0>([](IOps& Out)
	{
		FEnumFlagNames AB = Out.NameFlags({"PlainProps::Upgrade::Test", "EFlag"}, {"A", "B"});
		Out.Add({"Leaves"_U8s}, {"Leaves"_Enums(AB)}, [](FIns In, FOuts Out, FContext Ctx)
		{
			TInItems<uint8> Ints = In[0].U8s().Items();
			TFiller<FOutFlags> Enums = Out[0].EnumsFlags(2).New(Ints.Num(), Ctx);
			for (uint8 Int : Ints)
			{
				Enums << uint64((/* A */ Int & 1) | (/* B */ Int & 2));
			}
		});
		
		FStructMemberNames Data = Out.NameMembers({"PlainProps::Upgrade::Test", "FStr"}, {"Data"_Str});
		Out.Add({"String"_Str}, {"String"_Struct(Data)}, [](FIns In, FOuts Out, FContext Ctx)
		{
			Out[/* String */ 0].StructMembers()[/* Data */ 0].Str() = In[/* String */ 0].Str();
		});
		
		FStructMemberNames XY = Out.NameMembers({"PlainProps::Upgrade::Test", "FPoint"}, {"X"_F32, "Y"_F32});
		FStructMemberNames XsYs = Out.NameMembers({"PlainProps::Upgrade::Test", "FPoints"}, {"Xs"_F32s, "Ys"_F32s});
		Out.Add({"Points"_Structs(XY)}, {"Points"_Struct(XsYs)}, [](FIns In, FOuts Out, FContext Ctx)
		{
			TInItems<FInMembers> InPoints = In[0].StructsMembers(2).Items();
			FOutMembers OutPoints = Out[0].StructMembers();
			TFiller<float> Xs = OutPoints[0].F32s().New(InPoints.Num(), Ctx);
			TFiller<float> Ys = OutPoints[1].F32s().New(InPoints.Num(), Ctx);
			for (FInMembers XY : InPoints)
			{
				Xs << XY[0].F32();
				Ys << XY[1].F32();
			}
		});

		FStructId Point = Out.NameStruct({"PlainProps::Upgrade::Test", "FPoint"});
		Out.Add({"Series"_Structs(Point).Range()}, {"Series"_Structs(Point), "SeriesStarts"_S32s}, [](FIns In, FOuts Out, FContext Ctx)
		{
			TInItems<FInStruct, 2> InSeries = In[0].Range<FInStruct, 2>().Items();
			TFiller<int32> OutStarts = Out[1].S32s().New(InSeries.Num(), Ctx);
			int32 Start = 0;
			for (TInItems<FInStruct> InSerie : InSeries)
			{
				OutStarts << Start;
				Start += InSerie.Num();
			}

			TFiller<FOutStruct> OutSeries = Out[0].Structs().New(Start, Ctx);
			for (TInItems<FInStruct> InSerie : InSeries)
			{
				OutSeries << InSerie;
			}
		});

		Out.Add({"Unterminated"_Str}, {"Terminated"_Str}, [](FIns In, FOuts Out, FContext Ctx)
		{
			TInItems<char8_t> Unterminated = In[0].Str().Items();
			TFiller<char8_t> Terminated = Out[0].Str().New(Unterminated.Num() + 1, Ctx);
			Terminated << Unterminated << '\0';
		});

		FOpaqueEnum Flat = Out.NameFlatEnum({"PlainProps::Upgrade::Test", "EFlat"});
		Out.Add({"Sorted"_Enums(Flat)}, {"Reversed"_Enums(Flat)}, [](FIns In, FOuts Out, FContext Ctx)
		{
			TInItems<FInEnum> Sorted = In[0].Enums().Items();
			TOutItems<FOutEnum> Reversed = Out[0].Enums().New(Sorted.Num(), Ctx);
			for (uint64 Idx = Sorted.Num(); FInEnum Enum : Sorted)
			{
				Reversed[--Idx] = Enum;
			}
		});

		FEnumeratorNames ABC = Out.NameEnumerators({"PlainProps::Upgrade::Test", "EFlat"}, {"A","B","C"});
		Out.Add({"Enums"_Enums(ABC)}, {"Cycled"_Enums(ABC)}, [](FIns In, FOuts Out, FContext Ctx)
		{
			TInItems<FInEnumerator> Enums = In[0].Enumerators(3).Items();
			TFiller<FOutEnumerator> Cycled = Out[0].Enumerators(3).New(Enums.Num(), Ctx);
			for (FInEnumerator Enum : Enums)
			{
				Cycled << (Enum.GetIndex() + 1) % 3;
			}
		});

		Out.Add({"Bools"_Bools}, {"Negated"_Bools}, [](FIns In, FOuts Out, FContext Ctx)
		{
			TInItems<bool> Bools = In[0].Bools().Items();
			TFiller<bool> Negated = Out[0].Bools().New(Bools.Num(), Ctx);
			for (bool Bool : Bools)
			{
				Negated << !Bool;
			}
		});

	});
	RangesA.Rename<R0>({"PlainProps::Upgrade::Test", "FRangesB"});
}
		//Out.Transform("Leaves"_U8, "Leaves"_Enum(Flag)), [](uint8 In, FOutEnumFlag Out)
		//{
		//	Out[/* A */ 0] = In & 1;
		//	Out[/* B */ 1] = In & 2;
		//});

		//TEnum<2, R0> Flag(Out, "PlainProps::Upgrade::Test", "EFlag", {"A", "B"});
		//TStruct<2, R0> Point(Out, "PlainProps::Upgrade::Test", "FPoint", {"X", "Y"});
		//TStruct<2, R0> Points(Out, "PlainProps::Upgrade::Test", "FPoints", {"Xs", "Ys"});
		//TStruct<2, R0> Str(Out, "PlainProps::Upgrade::Test", "FStr", {"Data"});


	//		RangesA.Transform("Leaves"_U8, "Leaves"_Enum(Flag)),
	//	[](uint8 In, FOutEnum Out)
	//	{
	//		Out.EmitIf(/* A */ 0, !!(In & 1));
	//		Out.EmitIf(/* B */ 1, !!(In & 2));
	//	});
	//RangesA.Structify("String"_UTF8, "String"_Struct(Str));  // overload without lambda
	//RangesA.Structify("Points"_Range.Of(Point), "Points"_Struct(Points),
	//	[](FInStructs Points, FOutFloats Xs, FOutFloats Ys)
	//	{
	//		FFloatWriter XIt = Xs.Emit(In.Num());
	//		FFloatWriter YIt = Ys.Emit(In.Num());
	//		for (FInStruct Point : Points)
	//		{
	//			XIt.Add(Point[/* X */ 0].AsRequired<float>());
	//			YIt.Add(Point[/* Y */ 1].AsRequired<float>());
	//		}
	//	});

	//RangesA.Upgrade("Series"_Range.Nest().Of(Point), "Series"_Range.Of(Point), "SeriesLengths"_Range.Of<int32>(),
	//	[](FInNestedStructs SeriesOfSeries, FOutStructs Series, FOutI32s SeriesLengths)
	//	{
	//		FStructWriter SeriesIt = Series.Emit(SumNums(SeriesOfSeries));
	//		FI32Writer LengthsIt = SeriesLengths.Emit(SeriesOfSeries.Num());
	//		for (FInStructRange Points : SeriesOfSeries)
	//		{
	//			LengthsIt.Add(IntCastChecked<int32>(Points.Num()));
	//			for (FInStruct Point : Points)
	//			{
	//				SeriesIt.Add(Point);
	//			}
	//		}
	//	});

	//RangesA.Upgrade<FInUtf8s>("Unterminated", "Terminated"),
	//	[](FInUtf8s Unterminated, FOutUtf8s Terminated)
	//	{
	//		FUtf8s OutData = Terminated.Emit(Unterminated.Num() + 1);
	//		OutData.EmitSlice(0, Unterminated);
	//		OutData.EmitItem(Unterminated.Num, '\0');
	//	});
	//});

//// Sugar (move down)
//template<> PlainProps::Upgrade::Chronicle<FRangesB>(FUpgrades& Out)
//{
//	TEnum<2, R0> Flag(Out, "", "EFlag", {"A", "B"});
//	TStruct<2, R0> Point(Out, "", "FPoint", {"X", "Y"});
//	TStruct<2, R0> Points(Out, "", "FPoints", {"Xs", "Ys"});
//	TStruct<2, R0> Str(Out, "", "FStr", {"Data"});
//
//	FChronicle RangesB(Out, "", "FRangesB"); // ~ registers chronicle once we've collected all member names
//	
//	RangesB.Transform<FInEnum>(MakeIn(Flag, "Leaves"), "Leaves"),
//		[](FInEnum In, FOutU8 Out)
//		{
//			OutU8.Emit(uint8(In[/* A */ 0]) + uint8(In[/* B */ 1])*2);
//		});
//	RangesB.Destructify<FInUtf8s>(MakeInStruct("String"), "String"); // overload without lambda
//	
//	RangesB.Destructify<FInFloats, FInFloats>(MakeInStruct(Points, "Points"), MakeOutStructs(Point, "Points")),
//		[](FInFloats Xs, FInFloats Ys, FOutNestedStructs Points)
//		{
//			uint64 Num = Xs.Num();
//			if (Num != Ys.Num())
//			{
//				Ctx.LogDataLoss();
//				Num = FMath::Min(Xs.Num(), Ys.Num());
//			}
//
//			FStructsWriter Out = Points.Emit(Xs.Num());
//			for (uint64 Idx = 0; Idx < Num; ++Idx)
//			{
//				Out[Idx][0].Emit(Xs[Idx]);
//				Out[Idx][1].Emit(Ys[Idx]);
//			}
//		});
//	
//	RangesB.Upgrade<FInStructs, FI32s>(MakeIn(Point, "Series"), "SeriesLengths", MakeOutStructs(Point, "Series"),
//		[](FInStructs Series, FInI32s SeriesLengths, FOutStructs Out)
//		{
//			FStructWriter* OutIt = Out.Emit(SeriesOfSeries.Num()).GetData();
//			uint64 SeriesIdx = 0;
//			for (uint64 Len : SeriesOfSeries)
//			{
//				FOutStruct OutPoint = *OutIt++;
//				OutPoint[0].Emit(Series.Slice(SeriesIdx, Len));
//				SeriesIdx += Len;
//			}
//		});
//
//	RangesB.Upgrade<FInUtf8s>("Terminated", "Unterminated"),
//		[](FInUtf8s Terminated, FOutUtf8s Unterminated)
//		{
//			check(Terminated.Num());
//			check(Terminated.Last() == '\0');
//			FUtf8s OutData = Unterminated.Emit(Terminated.Num() - 1);
//			OutData.EmitSlice(0, Terminated, Terminated.Num() - 1);
//		});
//}

template<class T>
struct TScopedStructUpgradeBinding 
{
	explicit TScopedStructUpgradeBinding(FHistory& Hist) : History(Hist)
	{
		Record(static_cast<TChroniclesOf<T>>(FChronicles(/* out */ History, Binding.DeclId)));
	}
	
	~TScopedStructUpgradeBinding()
	{
		EraseChroniclesOf(History, Binding.DeclId);
	}
	FHistory&									History;
	TScopedStructBinding<T, FDefaultRuntime>	Binding;
};

////////////////////////////////////////////////////////////////////////////////////////////////

TEST_CASE_NAMED(FPlainPropsUpgradeTest, "System::PlainProps::Upgrade", "[PlainProps][SmokeFilter]")
{
	FFixture Fixture;

	SECTION("Enums")
	{
	}

	SECTION("Basics")
	{
		FHistoryPtr History = MakeHistory(GDeltaDeclarations, GIds);
		TScopedStructUpgradeBinding<FBasic> Basic(*History);
		TScopedEnumDeclaration<EFlag, EEnumMode::Flag, FDefaultRuntime> Flag;
		TScopedStructBinding<FPoint, FDefaultRuntime> Point;

		Run(*History,
			[](FBatchSaver& Batch)
			{
				Batch.Save(FBasicOld{
					.Widen=1, .Rename=2, .Reorder=3, .Drop=4, .Unchanged=5, .FlagA=false, .FlagB=true, 
					.PointX=6.f, .PointY=7.f, .Guid=FGuid(0x01234567, 0x89ABCDEF,0,0xFFFFFFFF)});
			}, 
			[](FBatchLoader& Batch)
			{
				CHECK(Batch.Load<FBasic>(0) == FBasic{
					.Widen=1, .Reorder=3, .Renamed=2, .Unchanged=5, .Flags=EFlag::B, 
					.Point={6.f, 7.f}, .GuidAB=0x0123456789ABCDEF, .GuidCD=0xFFFFFFFF});
			});
	}

	SECTION("Ranges")
	{
		TScopedStructBinding<FPoint, FDefaultRuntime> Point;
		TScopedStructBinding<FPoints, FDefaultRuntime> Points;
		TScopedStructBinding<FStr, FDefaultRuntime> Str;
		TScopedEnumDeclaration<EFlat, EEnumMode::Flat, FDefaultRuntime> Flat;
		TScopedEnumDeclaration<EFlag, EEnumMode::Flag, FDefaultRuntime> Flag;

		FHistoryPtr History = MakeHistory(GDeltaDeclarations, GIds);
		TScopedStructUpgradeBinding<FRangesB> RangesB(*History);
		Run(*History,
			[](FBatchSaver& Batch)
			{
				Batch.Save(FRangesA{.Leaves = {0,1,2,3}, .String = {'h','i'}, .Points = {{1,5}, {2,6}, {3,7}}, 
									.Series = {{}, {{1,2}, {3,4}, {5,6}}, {}, {{7,8}}, {}},
									.Unterminated = {'l','o'}, .Sorted = {EFlat::A, EFlat::B, EFlat::C},
									.Enums = {EFlat::A, EFlat::C}, .Bools = {true, false, true} });
			}, 
			[](FBatchLoader& Batch)
			{
				FRangesB B = Batch.Load<FRangesB>(0);
				CHECK(B == FRangesB{.Leaves = {EFlag::None, EFlag::A, EFlag::B, EFlag::AB}, .String = {{'h', 'i',}}, .Points = {{1,2,3},{5,6,7}},
									.Series = {{1,2}, {3,4}, {5,6}, {7,8}}, .SeriesStarts = {0,0,3,3,4}, 
									.Terminated = {'l','o','\0'}, .Reversed = {EFlat::C, EFlat::B, EFlat::A},
									.Cycled = {EFlat::B, EFlat::A}, .Negated = {false, true, false}});
			});
	}
	
	SECTION("Versions")
	{
	}

	SECTION("Inheritance")
	{
	}

	SECTION("Unknowns")
	{
	}

	SECTION("Mismatches")
	{
		RunUntyped([](FBatchWriter& Batch)
		{
			const FStructDeclaration& InputDecl = Batch.OldTypes.Declare("Test", "Input", {"A","B","X"}, {SpecF64, SpecF32, SpecF32}, EMemberPresence::AllowSparse);
			const FStructDeclaration& MismatchDecl = Batch.OldTypes.Declare("Test", "Mismatch", 
				{"UnusedOld", "Unknown", "Mistyped", "Ok", "Inputs"}, 
				{SpecS32,	SpecS32,	SpecS32,	SpecS32, FMemberSpec(InputDecl.Id)}, EMemberPresence::AllowSparse);

			FMemberBuilder Members;
			Members.Add(MismatchDecl.MemberNames[1], 123);	// Unknown
			Members.Add(MismatchDecl.MemberNames[2], 456);	// Mistyped
			Members.Add(MismatchDecl.MemberNames[3], 789);	// Ok

			FMemberBuilder Inputs;
			Inputs.Add(GIds.IndexMemberLiteral("A"), 1.0);	// Wrong type
			Inputs.Add(GIds.IndexMemberLiteral("B"), 2.0f); // Ok
			Inputs.Add(GIds.IndexMemberLiteral("X"), 3.0f); // Surplus
															// C is missing			
			Members.AddStruct(MismatchDecl.MemberNames[4], InputDecl.Id, Batch.Build(Inputs, InputDecl));

			Batch.Save(MismatchDecl.Id, Batch.Build(Members, MismatchDecl));
		}, 
		[](FTestUpgrades Upgrades)
		{
			// First declare types
			const FStructDeclaration& InputDecl = Upgrades.NewTypes.Declare("Test", "Input", {"A","B","C"}, {SpecF32, SpecF32, SpecF32}, EMemberPresence::AllowSparse);
			const FStructDeclaration& MismatchDecl = Upgrades.NewTypes.Declare("Test", "Mismatch", 
				{"Duplicate", "Mistyped", "Ok",	"Inputs"}, 
				{SpecS32,		SpecS64, SpecS32, FMemberSpec(InputDecl.Id)}, EMemberPresence::AllowSparse);
		
			FChronicles MismatchChronicles(Upgrades.History, MismatchDecl.Id);
			IOps& MismatchOps = *MismatchChronicles.CreateVersion({"Test", "Mismatch"}, 0);

			// Add two ops which both reemit Ok but also emit one Duplicate
			MismatchOps.Add({"Ok"_S32}, {"Ok"_S32, "Duplicate"_S32}, [](FIns In, FOuts Out, FContext Ctx)
			{
				Out[0].S32() = In[0].S32();
				Out[1].S32() = 111;
			});

			MismatchOps.Add({"Ok"_S32}, {"Ok"_S32, "Duplicate"_S32}, [](FIns In, FOuts Out, FContext Ctx)
			{
				Out[0].S32() = In[0].S32();
				Out[1].S32() = 222;
			});
			
			FStructId InputId = MismatchOps.NameStruct({"Test", "Input"});
			FStructMemberNames ABC = MismatchOps.NameMembers(InputId, {"A"_F32, "B"_F32, "C"_F32});
			MismatchOps.Add({"Inputs"_Struct(ABC)}, {}, [](FIns In, FOuts Out, FContext)
				{
					CHECK(false); // Shouldn't execute op with mismatching inputs
				});

			MismatchChronicles.CommitVersion(&MismatchOps);

			// Test.Input chronicles are missing X -> C rename and A's F64 ->F32 conversion
			FChronicles InputChronicles(Upgrades.History, InputDecl.Id);
			IOps& InputOps = *InputChronicles.CreateVersion({"Test", "Input"}, 0);
			InputChronicles.CommitVersion(&InputOps);
		}, 
		[](FUpgradedBatch Batch)
		{
			CHECK(Batch.Roots.Num() == 1);
			CHECK(Batch.Roots[0].Fixed == nullptr);
			CHECK(Batch.Roots[0].Loose.Num() == 6);
		
			FDeclId MismatchId = Batch.Roots[0].Type;
			TConstArrayView<FLooseMember> Members = Batch.Roots[0].Loose;

			//TArray<FAnsiString> MemberNames;
			//for (FLooseMember Member : Members)
			//{
			//	MemberNames.Emplace(Batch.ResolveName(MismatchId, Member));
			//}

			// Matching members come first
			// Mismatching member order doesn't matter, this is a hacky test of a temporary / ad-hoc API
			CHECK(Batch.ResolveName(MismatchId, Members[0]) == "Ok");
			CHECK(Batch.ResolveName(MismatchId, Members[1]) == "Duplicate"); 
			CHECK(Batch.ResolveName(MismatchId, Members[2]) == "Unknown");
			CHECK(Batch.ResolveName(MismatchId, Members[3]) == "Mistyped");
			CHECK(Batch.ResolveName(MismatchId, Members[4]) == "Duplicate");
			CHECK(Batch.ResolveName(MismatchId, Members[5]) == "Inputs");

			// 222 is given first despite 111 being output first due to current greedy execution that might change
			CHECK(Members[0].GetValue().Meta.Mismatch == EMismatch::No);
			CHECK(Members[0].GetValue().Data.Arithmetic == 789);
			CHECK(Members[1].GetValue().Meta.Mismatch == EMismatch::Duplicate);
			CHECK(Members[1].GetValue().Data.Arithmetic == 222);				
			CHECK(Members[2].GetValue().Meta.Mismatch == EMismatch::Surplus);
			CHECK(Members[2].GetValue().Data.Arithmetic == 123);
			CHECK(Members[3].GetValue().Meta.Mismatch == EMismatch::Type);
			CHECK(Members[3].GetValue().Data.Arithmetic == 456);
			CHECK(Members[4].GetValue().Meta.Mismatch == EMismatch::Duplicate);
			CHECK(Members[4].GetValue().Data.Arithmetic == 111);
			CHECK(Members[5].GetValue().Meta.Mismatch == EMismatch::Inputs);

			FLooseMember Inputs = Members[5];
			CHECK(Inputs.GetType().IsStruct());

			FDeclId InputId = Inputs.GetType().InnermostId.Get().AsStructDeclId();
			TConstArrayView<FLooseMember> Inners(Inputs.GetValue().Data.Struct.Loose, Inputs.GetValue().Meta.Num);
			CHECK(Inners.Num() == 4);			
			CHECK(Batch.ResolveName(InputId, Inners[0]) == "B");
			CHECK(Batch.ResolveName(InputId, Inners[1]) == "A");
			CHECK(Batch.ResolveName(InputId, Inners[2]) == "X");
			CHECK(Batch.ResolveName(InputId, Inners[3]) == "C");

			CHECK(Inners[0].GetValue().Meta.Mismatch == EMismatch::No);
			CHECK(Inners[1].GetValue().Meta.Mismatch == EMismatch::Type);
			CHECK(Inners[2].GetValue().Meta.Mismatch == EMismatch::Surplus);
			CHECK(Inners[3].GetValue().Meta.Mismatch == EMismatch::Missing);
		});
	}

	SECTION("Defaults")
	{
	}
}

} // namespace PlainProps::Upgrade::Test
#endif // WITH_TESTS